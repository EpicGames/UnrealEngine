{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": ["WinGDK"],
	"Modules": [
		{
			"Name": "OnlineSubsystemGDK",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": ["WinGDK"]
		}
	]
}