{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem GDK",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to GDK platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	"RequiresBuildPlatform": false,
	"HasExplicitPlatforms": true,	
	"ExplicitlyLoaded":  true,
	"SupportedTargetPlatforms": ["Win64"],

	"Modules": [
		{
			"Name": "OnlineSubsystemGDK",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "Default",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemGDK",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "GDKNetDriver",
			"Enabled": true
		},
		{
			"Name": "PlayFabParty",
			"Enabled": true
		}
	]
}