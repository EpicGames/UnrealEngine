// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Diagnostics;
using System.IO;
using UnrealBuildTool;

public class OnlineSubsystemGDK : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Engine, "PlayFab")]
	bool EnablePlayfabMatchmaking = false;

	public OnlineSubsystemGDK(ReadOnlyTargetRules Target) : base(Target)
    {
		PrivateDefinitions.Add("ONLINESUBSYSTEMGDK_PACKAGE=1");
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Engine",
				"OnlineSubsystemUtils",
				"CoreUObject",
				"Voice",
				"GDKNetDriver",
				"HTTP",
			});
		}

        // Modules our Privates require
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"Sockets",
				"Json",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd"
			});
		}

		// Modules our Publics require
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"OnlineSubsystem",
				"GRDK",
				"GDKRuntime",
				"XSAPI",
				"XCurl",
			}
			);

		if (GRDK.IsValid(Target) )
		{
			ExtraRootPath = ("GSDK", GRDK.GetGDKRoot());

			if (Target.bCompileAgainstEngine)
			{
				PublicDependencyModuleNames.Add("PlayFabParty");
			}

			ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this, Target.CustomConfig);


			if (GRDK.IsLegacyFolderStructure())
			{
				// Add GameChat2 extension
				GRDK.AddLegacyExtensionDependency(Target, this, "Xbox.Game.Chat.2.Cpp.API", "GameChat2");

				// Add OSS definitions
				if (EnablePlayfabMatchmaking)
				{
					// Add PlayFab extensions
					GRDK.AddLegacyExtensionDependency(Target, this, "PlayFab.Party.Cpp", "Party");
					GRDK.AddLegacyExtensionDependency(Target, this, "PlayFab.Multiplayer.Cpp", "PlayFabMultiplayerGDK");
					PublicDefinitions.Add("UE_PLAYFAB_MATCHMAKING=1");
				}
			}
			else
			{
				GRDK.AddDependency(Target, this, "GameChat2");

				if (EnablePlayfabMatchmaking)
				{
					GRDK.AddDependency(Target, this, "Party", IncludeSubPath:"playfab/party");
					GRDK.AddDependency(Target, this, "PlayFabMultiplayer", IncludeSubPath:"playfab/multiplayer");
					PublicDefinitions.Add("UE_PLAYFAB_MATCHMAKING=1");
				}
			}

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDefinitions.Add("_GAMING_DESKTOP");
			}

			// Add OSS configuration files
			RuntimeDependencies.Add("$(ProjectDir)/Config/Xbl/*", StagedFileType.UFS);
			RuntimeDependencies.Add("$(ProjectDir)/Platforms/GDK/Config/OSS/*", StagedFileType.UFS); // legacy path
		}
	}

}