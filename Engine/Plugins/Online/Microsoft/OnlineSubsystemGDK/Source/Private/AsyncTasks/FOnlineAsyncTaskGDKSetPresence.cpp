// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSetPresence.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKSetPresence::FOnlineAsyncTaskGDKSetPresence(
	FOnlineSubsystemGDK* InGDKInterface,
	FGDKContextHandle InGDKContext,
	const FOnlineUserPresenceStatus& InPresenceStatus,
	const FString& InPresenceIdString,
	const FOnSetGDKPresenceCompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKSetPresence"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, PresenceStatus(InPresenceStatus)
	, PresenceIdString(InPresenceIdString)
{
}

void FOnlineAsyncTaskGDKSetPresence::Initialize()
{
	const ANSICHAR* Scid = nullptr;
	HRESULT Result = XblGetScid(&Scid);
	if (Result != S_OK || Scid == nullptr)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("FOnlineAsyncTaskGDKSetPresence: Failed to get Scid 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	XblPresenceRichPresenceIds Data;
	FMemory::Memcpy(Data.scid, Scid, XBL_SCID_LENGTH);
	TArray<ANSICHAR*> PresenceTokens;
	TArray<TArray<ANSICHAR>> PresenceStringStorage;
	if (PresenceStatus.Properties.Num() > 0)
	{
		PresenceStringStorage.AddDefaulted(PresenceStatus.Properties.Num());

		uint32 Idx = 0;
		for (const TPair<FPresenceKey, FVariantData>& Properties : PresenceStatus.Properties)
		{
			if (Properties.Key == DefaultPresenceKey)
			{
				++Idx;
				continue;
			}	

			FString ValueString = Properties.Value.ToString();
			const FTCHARToUTF8 BufferString(*ValueString);
			int32 BufferStringLen = BufferString.Length();

			TArray<ANSICHAR>& CharBuffer = PresenceStringStorage[Idx];
			CharBuffer.Append(BufferString.Get(), BufferStringLen + 1);
			PresenceTokens.Add(CharBuffer.GetData());
			++Idx;
		}
	}

	if (PresenceTokens.Num() > 0)
	{
		Data.presenceTokenIds = const_cast<const ANSICHAR**>(PresenceTokens.GetData());
		Data.presenceTokenIdsCount = PresenceTokens.Num();
	}
	else
	{
		Data.presenceTokenIds = nullptr;
		Data.presenceTokenIdsCount = 0;
	}

	const FTCHARToUTF8 BufferString(*PresenceIdString);
	Data.presenceId = BufferString.Get();

	Result = XblPresenceSetPresenceAsync(GDKContext, true, &Data, *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("FOnlineAsyncTaskGDKSetPresence: Failed to set presence with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSetPresence::ProcessResults()
{
	bWasSuccessful = true;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKSetPresence::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKSetPresence_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, PresenceStatus);
}

#endif //WITH_GRDK
