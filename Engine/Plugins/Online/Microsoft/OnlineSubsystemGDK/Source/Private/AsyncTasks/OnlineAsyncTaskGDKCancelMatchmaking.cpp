// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKCancelMatchmaking.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineAsyncTaskGDKCancelMatchmaking.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/matchmaking_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKCancelMatchmaking::FOnlineAsyncTaskGDKCancelMatchmaking(
	class FOnlineSubsystemGDK* InSubsystem,
	FGDKContextHandle InUserContext,
	FName InSessionName,
	FOnlineMatchTicketInfoPtr InTicketInfo)
	: FOnlineAsyncTaskGDK(InSubsystem, TEXT("FOnlineAsyncTaskGDKCancelMatchmaking"), 0)
	, SessionName(InSessionName)
	, GDKContext(InUserContext)
	, TicketInfo(InTicketInfo)
{
}

void FOnlineAsyncTaskGDKCancelMatchmaking::Initialize()
{
	if (TicketInfo.IsValid() == false)
	{
		bWasSuccessful = true;
		bIsComplete = true;
		return;
	}

	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);

	HRESULT Result = XblMatchmakingDeleteMatchTicketAsync(GDKContext, Scid, TCHAR_TO_UTF8(*TicketInfo->HopperName), TCHAR_TO_UTF8(*TicketInfo->TicketId), *AsyncBlock);
	if (FAILED(Result))
	{
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCancelMatchmaking::ProcessResults()
{
	bWasSuccessful = true;
	bIsComplete = true;
};

void FOnlineAsyncTaskGDKCancelMatchmaking::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKCancelMatchmaking_TriggerDelegates);
	Subsystem->GetMatchmakingInterfaceGDK()->TriggerOnCancelMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
}

#endif //WITH_GRDK