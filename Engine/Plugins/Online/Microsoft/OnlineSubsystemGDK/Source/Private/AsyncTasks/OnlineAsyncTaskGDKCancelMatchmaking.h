// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#pragma push_macro("CPP")
#undef CPP

#include <xsapi-c/xbox_live_context_c.h>

#pragma pop_macro("CPP")
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

class FOnlineAsyncTaskGDKCancelMatchmaking : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKCancelMatchmaking(
	class FOnlineSubsystemGDK* InGDKSubsystem,
		FGDKContextHandle InUserContext,
		FName InSessionName,
		FOnlineMatchTicketInfoPtr InTicketInfo);

	virtual ~FOnlineAsyncTaskGDKCancelMatchmaking() = default;

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKCancelMatchmaking");}
	virtual void TriggerDelegates() override;

private:
	FName SessionName;
	FGDKContextHandle GDKContext;

	FOnlineMatchTicketInfoPtr TicketInfo;

};

//------------------------------- End of file ---------------------------------
