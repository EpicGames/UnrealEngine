// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKCheckForPackageUpdate.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"
#include "Misc/ConfigCacheIni.h"

THIRD_PARTY_INCLUDES_START
#define _UITHREADCTXT_SUPPORT   0
#include <ppltasks.h>
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END

#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.packagecheck"


FOnlineAsyncTaskGDKCheckForPackageUpdate::FOnlineAsyncTaskGDKCheckForPackageUpdate(
	FOnlineSubsystemGDK* const InGDKSubsystem,
	FGDKUserHandle InLocalUser,
	const FOnCheckForPackageUpdateCompleteDelegate& CompletionDelegate
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKCheckForPackageUpdate"))
	, TaskCompletionDelegate(CompletionDelegate)
	, LocalUser(InLocalUser)
	, ErrorResult(false)
{
}

void FOnlineAsyncTaskGDKCheckForPackageUpdate::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;
	if (Subsystem->GetStoreGDK()->BlockMismatchedStoreUser(LocalUser))
	{
		bWasSuccessful = false;
		bIsComplete = true;
		ErrorResult= ONLINE_ERROR(EOnlineErrorResult::MismatchedUser);
		UE_LOG_ONLINE(Error, TEXT("Title / Store ID Mismatch Attempting to query packages"));
		return;
	}

	const XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(LocalUser);
	if (!StoreContextHandle)
	{
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XStoreQueryGameAndDlcPackageUpdatesAsync(StoreContextHandle, *AsyncBlock);
	if(Result != S_OK)
	{
		bWasSuccessful = false;
		bIsComplete = true;
		UE_LOG_ONLINE(Error, TEXT("Attempting to query packages failed: 0x%08X"), Result);
	}
	return;
}

void FOnlineAsyncTaskGDKCheckForPackageUpdate::ProcessResults()
{
	uint32 ResultCount = 0;
	HRESULT Result = XStoreQueryGameAndDlcPackageUpdatesResultCount(*AsyncBlock, &ResultCount);
	if (Result == S_OK)
	{
		TArray<XStorePackageUpdate> PackageArray;
		if (ResultCount > 0)
		{
			PackageArray.Reserve(ResultCount);
			Result = XStoreQueryGameAndDlcPackageUpdatesResult(*AsyncBlock, ResultCount, PackageArray.GetData());
			PackageArray.SetNum(ResultCount);
			if (PackageArray[0].isMandatory)
			{
				PackageUpdateResult = ECheckForPackageUpdateResult::MandatoryUpdateAvailable;
			}
			else
			{
				PackageUpdateResult = ECheckForPackageUpdateResult::OptionalUpdateAvailable;
			}
		}
		else
		{
			PackageUpdateResult = ECheckForPackageUpdateResult::NoUpdateAvailable;
		}

		ErrorResult.bSucceeded = true;
	}
	else
	{
		bool bIgnoreErrorCode = GConfig->GetBoolOrDefault(TEXT("OnlineSubsystemGDK.OnlineAsyncTaskGDKCheckForPackageUpdate"), TEXT("IgnoreAllErrorCodes"), true, GEngineIni);

		if (!bIgnoreErrorCode)
		{
			TArray<FString> IgnoredErrorCodes;
			GConfig->GetArray(TEXT("OnlineSubsystemGDK.OnlineAsyncTaskGDKCheckForPackageUpdate"), TEXT("IgnoredErrorCodes"), IgnoredErrorCodes, GEngineIni);
			if (IgnoredErrorCodes.Num() > 0)
			{
				bIgnoreErrorCode = IgnoredErrorCodes.Contains(FString::Printf(TEXT("0x%0.8X"), Result));
			}
		}

		UE_LOG_ONLINE(Error, TEXT("XStoreQueryGameAndDlcPackageUpdatesResultCount failed with 0x%0.8X%s"), Result, bIgnoreErrorCode ? TEXT(" (ignored)") : TEXT(""));
		if (bIgnoreErrorCode)
		{
			PackageUpdateResult = ECheckForPackageUpdateResult::NoUpdateAvailable;
			ErrorResult.bSucceeded = true;
		}
		bWasSuccessful = bIgnoreErrorCode;
		bIsComplete = true;
		return;
	}

	bWasSuccessful = true;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKCheckForPackageUpdate::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKCheckForPackageUpdate_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(ErrorResult, PackageUpdateResult);
}
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK