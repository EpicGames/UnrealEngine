// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

enum class ECheckForPackageUpdateResult : uint8
{
	NoUpdateAvailable,
	OptionalUpdateAvailable,
	MandatoryUpdateAvailable
};

inline const TCHAR* const LexToString(const TOptional<ECheckForPackageUpdateResult> Value)
{
	if (!Value.IsSet())
	{
		return TEXT("Unknown");
	}

	switch (Value.GetValue())
	{
	case ECheckForPackageUpdateResult::NoUpdateAvailable:
		return TEXT("NoUpdateAvailable");
	case ECheckForPackageUpdateResult::OptionalUpdateAvailable:
		return TEXT("OptionalUpdateAvailable");
	case ECheckForPackageUpdateResult::MandatoryUpdateAvailable:
		return TEXT("MandatoryUpdateAvailable");
	}

	checkNoEntry();
	return TEXT("Unknown");
}

/**
 * Delegate for after determining whether the package needs an update.
 */
DECLARE_DELEGATE_TwoParams(FOnCheckForPackageUpdateCompleteDelegate, const FOnlineError& /*ErrorResult*/, const TOptional<ECheckForPackageUpdateResult> /*PatchResult*/);

class FOnlineAsyncTaskGDKCheckForPackageUpdate
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKCheckForPackageUpdate(
		class FOnlineSubsystemGDK* const InGDKSubsystem,
		FGDKUserHandle InLocalUser,
		const FOnCheckForPackageUpdateCompleteDelegate& CompletionDelegate);

	virtual ~FOnlineAsyncTaskGDKCheckForPackageUpdate() = default;

	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKCheckForPackageUpdate bWasSuccessful: %d Result: %s"), ErrorResult.bSucceeded, LexToString(PackageUpdateResult)); }

	virtual void Tick() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

private:
	FOnCheckForPackageUpdateCompleteDelegate TaskCompletionDelegate;
	FGDKUserHandle LocalUser;
	FOnlineError ErrorResult;
	TOptional<ECheckForPackageUpdateResult> PackageUpdateResult;
	bool bTaskStarted = false;
};
