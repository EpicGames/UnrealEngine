// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKClearSessionActivity.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"

FOnlineAsyncTaskGDKClearSessionActivity::FOnlineAsyncTaskGDKClearSessionActivity(FOnlineSubsystemGDK* InGDKSubsystem,
																				   FGDKContextHandle InGDKContext,
																				   XblMultiplayerSessionReference* SessionReference)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKClearSessionActivity"))
	, GDKContext(InGDKContext)
{
	if (SessionReference)
	{
		memcpy(ServiceConfigurationId, SessionReference->Scid, XBL_SCID_LENGTH);
	}
	else
	{
		const ANSICHAR* Scid = nullptr;
		XblGetScid(&Scid);
		memcpy(ServiceConfigurationId, Scid, XBL_SCID_LENGTH);
	}

	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());
}

void FOnlineAsyncTaskGDKClearSessionActivity::Initialize()
{
	HRESULT Result = XblMultiplayerClearActivityAsync(GDKContext, ServiceConfigurationId, *AsyncBlock);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error starting ClearActivityAsync, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKClearSessionActivity::ProcessResults()
{

	size_t BufferSize = 0;
	XAsyncGetResultSize(*AsyncBlock, &BufferSize);
	
	TArray<char> ResultData;
	ResultData.Reserve(BufferSize);
	
	HRESULT DeleteResult = XAsyncGetResult(*AsyncBlock, nullptr, BufferSize, ResultData.GetData(), nullptr);
	if (DeleteResult == S_OK)
	{
		// We succeeded if CompletedTask didn't throw
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Successfully cleared Multiplayer Session Activity for player on ServiceConfigurationId %hs"),
			ServiceConfigurationId);
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Failed to clear Multiplayer Session Activity for player, error: (0x%0.8X)."),
			DeleteResult);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKClearSessionActivity::TriggerDelegates()
{
	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	check(SessionInt.IsValid());

	SessionInt->GetMpsdImpl()->ClearSessionActivityInProgress();

	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());

	FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(GDKContext);
	SessionInt->GetMpsdImpl()->OnSetUserActiveSessionActivityComplete(*UserNetId, false);
}

#endif //WITH_GRDK
