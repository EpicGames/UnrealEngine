// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to clear the current session activity for a user
 */
class FOnlineAsyncTaskGDKClearSessionActivity
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKClearSessionActivity(FOnlineSubsystemGDK* InGDKSubsystem,
										   FGDKContextHandle InLiveContext,
										   XblMultiplayerSessionReference* InSessionReference);
	virtual ~FOnlineAsyncTaskGDKClearSessionActivity() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKClearSessionActivity"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void TriggerDelegates() override;

protected:
	/** Service Configuration ID to clear for this user */
	ANSICHAR ServiceConfigurationId[XBL_SCID_LENGTH];
	FGDKContextHandle GDKContext;
};
