// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKCreateMatchSession.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineAsyncTaskGDKSubmitMatchTicket.h"
#include "OnlineAsyncTaskGDKRegisterLocalUser.h"
#include "OnlineAsyncTaskGDKDestroySession.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"

THIRD_PARTY_INCLUDES_START
#include <Ws2tcpip.h>
THIRD_PARTY_INCLUDES_END

//-----------------------------------------------------------------------------
// Constructor
//-----------------------------------------------------------------------------

FOnlineAsyncTaskGDKCreateMatchSession::FOnlineAsyncTaskGDKCreateMatchSession(FOnlineSubsystemGDK* InGDKSubsystem,
																			   const TArray<FUniqueNetIdRef>& InSearchingUserIds,
																			   const FName InSessionName,
																			   const FOnlineSessionSettings& InSessionSettings,
																			   const TSharedRef<FOnlineSessionSearch>& InSearchSettings)
	: FOnlineAsyncTaskGDKSessionBase(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKCreateMatchSession"), INDEX_NONE, InSessionName, InSessionSettings)
	, SearchingUserIds(InSearchingUserIds)
	, CurrentMatchSessionRef(nullptr)
	, bSessionCreated(false)
	, NumOtherLocalPlayersToAdd(0)
{
	check(SearchingUserIds.Num() > 0);

	FUniqueNetIdGDKRef FirstUserNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(SearchingUserIds[0]);
	UserIndex = Subsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(*FirstUserNetId);
	SearchingUser = Subsystem->GetIdentityGDK()->GetUserForUniqueNetId(*FirstUserNetId);

	ParseSearchSettings(&InSearchSettings.Get());
}

//-----------------------------------------------------------------------------
// Destructor
//-----------------------------------------------------------------------------

FOnlineAsyncTaskGDKCreateMatchSession::~FOnlineAsyncTaskGDKCreateMatchSession()
{
}

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

void FOnlineAsyncTaskGDKCreateMatchSession::Initialize()
{
	auto CreateSessionCompleteDelegate = FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete::CreateRaw(this, &FOnlineAsyncTaskGDKCreateMatchSession::ProcessResult);
	FString Keyword;
	SessionSettings.Get(SETTING_GAMEMODE, Keyword);
	Subsystem->GetSessionInterfaceGDK()->GetMpsdImpl()->StartCreateSession(SearchingUserIds[0].Get(), SessionSettings, Keyword, SessionTemplateName, SessionName, CreateSessionCompleteDelegate);

}

void FOnlineAsyncTaskGDKCreateMatchSession::ProcessResult(bool bSuccess, FName ResultSessionName)
{
	if(bSuccess)
	{
		FNamedOnlineSessionPtr NamedSession = Subsystem->GetSessionInterfaceGDK()->GetNamedSessionPtr(ResultSessionName);
		if (NamedSession.IsValid())
		{
			FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
			GDKSession = GDKInfo->GetGDKMultiplayerSession();
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKStartMatchmaking::Start: matching session creation failed"));
			bSessionCreated = false;
			bWasSuccessful = false;
			bIsComplete = true;
		}

		bSessionCreated = true;
		if (SearchingUserIds.Num() > 1)
		{
			// Finish updating our information on the game thread
			Subsystem->ExecuteNextTick([this, ResultSessionName]()
			{
				// There are other local users that need to be added to the match session before submitting it.
				FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
				check(MatchmakingInterface.IsValid());

				FOnlineMatchTicketInfoPtr MatchmakingTicket;
				MatchmakingInterface->GetMatchmakingTicket(ResultSessionName, MatchmakingTicket);

				check(MatchmakingTicket.IsValid());

				MatchmakingTicket->RefreshGDKInfo(GDKSession);	// need to do this before running tasks on the session

				NumOtherLocalPlayersToAdd.Set(SearchingUserIds.Num() - 1);
				bWasSuccessful = true;	// assume success unless a RegisterLocalPlayer delegate changes this

				XblMultiplayerSessionChangeTypes SessionSubscriptionType = FOnlineSessionMpsdGDK::GetSubscriptionType(SessionSettings);

				// Start from index 1, since user 0 was already added when the session was created.
				for (int32 i = 1; i < SearchingUserIds.Num(); ++i)
				{
					FUniqueNetIdGDKRef UserNetId(StaticCastSharedRef<const FUniqueNetIdGDK>(SearchingUserIds[i]));
					auto GDKContext = Subsystem->GetGDKContext(*UserNetId);
					if (GDKContext)
					{
						Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKRegisterLocalUser>(
							Subsystem,
							GDKContext,
							ResultSessionName,
							UserNetId,
							GDKSession,
							SessionSubscriptionType,
							FOnRegisterLocalPlayerCompleteDelegate::CreateRaw(this, &FOnlineAsyncTaskGDKCreateMatchSession::OnAddLocalPlayerComplete),
							TArray<const XblMultiplayerSessionMember*>());
					}
					else
					{
						UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKCreateMatchSession::ProcessResult - no GDK context for user %s"), *UserNetId->ToDebugString());
						OnAddLocalPlayerComplete( *UserNetId, EOnJoinSessionCompleteResult::UnknownError);
					}
				}
			});
		}
		else
		{
			// Complete, Finalize() will kick off the process of registering the match session and submitting
			// the match ticket.
			bWasSuccessful = true;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKStartMatchmaking::Start: matching session creation failed"));
		bSessionCreated = false;
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateMatchSession::OnAddLocalPlayerComplete(const FUniqueNetId& PlayerId, EOnJoinSessionCompleteResult::Type Result)
{
	if (Result != EOnJoinSessionCompleteResult::Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKStartMatchmaking::OnAddLocalPlayerComplete: failed to add local player to match session with result %u"), Result);
		bWasSuccessful = false;
	}

	const int32 PlayersLeftToAdd = NumOtherLocalPlayersToAdd.Decrement();
	if (PlayersLeftToAdd <= 0)
	{
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateMatchSession::Finalize()
{
	if (!bWasSuccessful)
	{
		// Accounts for failure to add a local user to the session
		if (bSessionCreated)
		{
			// The session was created, but another member failed to join. The session needs to be destroyed
			// to clean up.
			CreateDestroyMatchmakingCompleteTask(SessionName, GDKSession, Subsystem, false);
		}
		else
		{
			// this indicates a failure to create the Live match session. Remove the OSS named session
			// and trigger the delegate.
			FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
			check(MatchmakingInterface.IsValid());
			MatchmakingInterface->RemoveMatchmakingTicket(SessionName);
		}

		return;
	}

	// Live match session creation succeeded. Update the SessionInfo and register & submit the matching ticket.
	check(GDKSession.IsValid());

	FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
	check(MatchmakingInterface.IsValid());

	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	MatchmakingInterface->GetMatchmakingTicket(SessionName, MatchmakingTicket);
	// Abort if matchmaking was canceled
	if (MatchmakingTicket.IsValid() == false || MatchmakingTicket->MatchmakingState == EOnlineGDKMatchmakingState::UserCancelled)
	{
		bWasSuccessful = false;
		return;
	}

	MatchmakingTicket->SessionSettings = SessionSettings;
	// Store actual match session
	MatchmakingTicket->RefreshGDKInfo(GDKSession);	// need to do this before running tasks on the session
	MatchmakingTicket->MatchmakingState = EOnlineGDKMatchmakingState::SubmittingInitialTicket;
	MatchmakingTicket->SetHostUser(SearchingUser);

	// we don't want to keep named session for match sessions, they are transitory and the ticket has all the info we need. (tests for this not existing influence join logic)
	Subsystem->GetSessionInterfaceGDK()->RemoveNamedSession(SessionName);

	const XblMultiplayerSessionReference* GDKSessionRef = XblMultiplayerSessionSessionReference(GDKSession);
	Subsystem->GetMatchmakingInterfaceGDK()->SubmitMatchingTicket(GDKSession, SessionName, false);

	// Initialize session state after create/join (uses ticket if session is forgotten as above)
	Subsystem->GetSessionMessageRouter()->SyncInitialSessionState(SessionName, GDKSession);
}

void FOnlineAsyncTaskGDKCreateMatchSession::TriggerDelegates()
{
	// Account for failure to add a user to a created session
	if (!bWasSuccessful && !bSessionCreated)
	{
		// Only fire delegates on failure to create a session. It's unfortunate to break up
		// the logic like this, but on success the delegates will be fired in the GameSessionReady
		// task. If the session was created but a member join failed, the delegates will be fired
		// by the DestroySession task.
		FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
		check(MatchmakingInterface.IsValid());

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKCreateMatchSession_TriggerDelegates);
		MatchmakingInterface->TriggerOnMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
	}
}

#endif //WITH_GRDK