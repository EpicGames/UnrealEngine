// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSessionBase.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "HAL/ThreadSafeCounter.h"

class FOnlineSubsystemGDK;

//-----------------------------------------------------------------------------
// Task to encapsulate Start Matchmaking
//-----------------------------------------------------------------------------

class FOnlineAsyncTaskGDKCreateMatchSession : public FOnlineAsyncTaskGDKSessionBase 
{
public:
	//. Regular Matchmaking should use this constructor
	FOnlineAsyncTaskGDKCreateMatchSession(
		FOnlineSubsystemGDK* InLiveSubsystem,
		const TArray<FUniqueNetIdRef>& InSearchingUserIds,
		const FName InSessionName,
		const FOnlineSessionSettings& InSessionSettings,
		const TSharedRef<FOnlineSessionSearch>& InSearchSettings
		);

	virtual ~FOnlineAsyncTaskGDKCreateMatchSession();

	virtual void Initialize() override;

	virtual FString ToString() const override { return TEXT("CreateMatchSession"); }
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual void ProcessResult(bool bWasSuccessful, FName ResultSessionName);

private:
	// Store array of users
	TArray<FUniqueNetIdRef> SearchingUserIds;

	XblMultiplayerSessionReference* CurrentMatchSessionRef;

	FGDKUserHandle SearchingUser;
	TSharedPtr<FInternetAddr> HostAddr;

	// Used to track completion of tasks that add local users to match session
	bool bSessionCreated;
	FThreadSafeCounter NumOtherLocalPlayersToAdd;

	void OnAddLocalPlayerComplete(const FUniqueNetId& PlayerId, EOnJoinSessionCompleteResult::Type Result);
};

//------------------------------- End of file ---------------------------------
