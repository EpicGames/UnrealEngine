// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKCreateSearchHandle.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"

FOnlineAsyncTaskGDKCreateSearchHandle::FOnlineAsyncTaskGDKCreateSearchHandle(FOnlineSubsystemGDK* InGDKSubsystem,
																				FGDKContextHandle InGDKContext,
																				const XblMultiplayerSessionReference& InSessionReference,
																				const FOnlineSessionSettings& InSessionSettings,
																				const FOnCreateSearchHandleCompleteDelegate& InTaskCompletionDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKCreateSearchHandle"))
	, GDKContext(InGDKContext)
	, SessionReference(InSessionReference)
	, SessionSettings(InSessionSettings)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

template <typename T>
void SetNumAttrib(const FName& SettingName, const FOnlineSessionSetting& InSetting, TArray<XblMultiplayerSessionNumberAttribute>& Attribs)
{
	T tempValue;
	InSetting.Data.GetValue(tempValue);
	XblMultiplayerSessionNumberAttribute& Attrib = Attribs.AddZeroed_GetRef();
	FCStringAnsi::Strncpy(Attrib.name, TCHAR_TO_UTF8(*SettingName.ToString().ToLower()), XBL_MULTIPLAYER_SEARCH_HANDLE_MAX_FIELD_LENGTH);
	Attrib.value = static_cast<double>(tempValue);
}

void SetStringAttrib(const FName& SettingName, const FOnlineSessionSetting& InSetting, TArray<XblMultiplayerSessionStringAttribute>& Attribs)
{
	XblMultiplayerSessionStringAttribute& Attrib = Attribs.AddZeroed_GetRef();
	FCStringAnsi::Strncpy(Attrib.name, TCHAR_TO_UTF8(*SettingName.ToString().ToLower()), XBL_MULTIPLAYER_SEARCH_HANDLE_MAX_FIELD_LENGTH);
	FCStringAnsi::Strncpy(Attrib.value, TCHAR_TO_UTF8(*FString(InSetting.Data.ToString()).ToLower()), XBL_MULTIPLAYER_SEARCH_HANDLE_MAX_FIELD_LENGTH);
}

void FOnlineAsyncTaskGDKCreateSearchHandle::Initialize()
{
	TSharedRef<TArray<XblMultiplayerSessionTag>> SearchTags = MakeShared<TArray<XblMultiplayerSessionTag>>();
	TSharedRef<TArray<XblMultiplayerSessionStringAttribute>> SearchStringAttribs = MakeShared<TArray<XblMultiplayerSessionStringAttribute>>();
	TSharedRef<TArray<XblMultiplayerSessionNumberAttribute>> SearchNumAttribs = MakeShared<TArray<XblMultiplayerSessionNumberAttribute>>();

	for (FSessionSettings::TConstIterator It(SessionSettings.Settings); It; ++It)
	{
		const FName& SettingName = It.Key();
		const FOnlineSessionSetting& SettingValue = It.Value();

		// Only upload values that are marked for service use
		if (SettingValue.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			switch (SettingValue.Data.GetType())
			{
				case EOnlineKeyValuePairDataType::Int32:
				{
					SetNumAttrib<int32>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::UInt32:
				{
					SetNumAttrib<uint32>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::Int64:
				{
					SetNumAttrib<int64>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::UInt64:
				{
					SetNumAttrib<uint64>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::Double:
				{
					SetNumAttrib<double>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::Float:
				{
					SetNumAttrib<float>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}
				case EOnlineKeyValuePairDataType::Bool:
				{
					SetNumAttrib<bool>(SettingName, SettingValue.Data, *SearchNumAttribs);
					break;
				}

				case EOnlineKeyValuePairDataType::String:
				case EOnlineKeyValuePairDataType::Json:
				case EOnlineKeyValuePairDataType::Blob:
				{
					SetStringAttrib(SettingName, SettingValue.Data, *SearchStringAttribs);
					break;
				}

				default:
				{
				}
			}
		}
	}
	XblMultiplayerSessionTag* SearchTagsPtr = (SearchTags->Num() > 0) ? SearchTags->GetData() : nullptr;

	XblMultiplayerSessionStringAttribute* SearchStringAttribsPtr = (SearchStringAttribs->Num() > 0) ? SearchStringAttribs->GetData() : nullptr;
	XblMultiplayerSessionNumberAttribute* SearchNumAttribsPtr = (SearchNumAttribs->Num() > 0) ? SearchNumAttribs->GetData() : nullptr;

	HRESULT Result = XblMultiplayerCreateSearchHandleAsync(GDKContext, 
															&SessionReference, 
															SearchTagsPtr, 
															SearchTags->Num(), 
															SearchNumAttribsPtr, 
															SearchNumAttribs->Num(), 
															SearchStringAttribsPtr,
															SearchStringAttribs->Num(), 
															*AsyncBlock);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error starting XblMultiplayerCreateSearchHandleAsync, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateSearchHandle::ProcessResults()
{
	//CDATODO No implementation of XblMultiplayerDeleteSearchHandleAsync so session wait on server till they timeout at present.
	HRESULT Result = XblMultiplayerCreateSearchHandleResult(*AsyncBlock, nullptr);
	if (Result == S_OK)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Created search handle for session."));
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Failed to create search handle, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateSearchHandle::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKCreateSearchHandle_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK