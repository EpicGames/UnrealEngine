// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
* Delegate fired when search handle has been created
*/
DECLARE_DELEGATE_OneParam(FOnCreateSearchHandleCompleteDelegate, bool);

/**
 * Async Task to clear the current session activity for a user
 */
class FOnlineAsyncTaskGDKCreateSearchHandle
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKCreateSearchHandle(FOnlineSubsystemGDK* InGDKSubsystem,
										   FGDKContextHandle InLiveContext,
										   const XblMultiplayerSessionReference& InSessionReference,
										   const FOnlineSessionSettings& SessionSettings,
										   const FOnCreateSearchHandleCompleteDelegate& InTaskCompletionDelegate);
	virtual ~FOnlineAsyncTaskGDKCreateSearchHandle() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKCreateSearchHandle"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	XblMultiplayerSessionReference SessionReference;
	FOnlineSessionSettings SessionSettings;
	FOnCreateSearchHandleCompleteDelegate TaskCompletionDelegate;
};
