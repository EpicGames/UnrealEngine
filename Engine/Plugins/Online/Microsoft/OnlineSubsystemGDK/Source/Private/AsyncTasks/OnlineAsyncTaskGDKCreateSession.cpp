// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKCreateSession.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineSubsystemGDKPackage.h"

FOnlineAsyncTaskGDKCreateSession::FOnlineAsyncTaskGDKCreateSession(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InGDKContext,
	const FUniqueNetIdRef InUserId,
	const FName InSessionName,
	FGDKMultiplayerSessionHandle InGDKSession,
	const bool bInWriteSession,
	const bool bInSetActivity,
	FOnGDKCreateSessionComplete InTaskCompletionDelegate)
	: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, TEXT("FOnlineAsyncTaskGDKCreateSession"), InGDKContext, InSessionName, InGDKSession)
	, UserId(InUserId)
	, bSetActivity(bInSetActivity)
	, bWriteSession(bInWriteSession)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
	check(Subsystem);
	SessionWriteMode = XblMultiplayerSessionWriteMode::CreateNew;
}

void FOnlineAsyncTaskGDKCreateSession::Initialize()
{
	// Only initialize the base class if we want to write to GDK.
	if(bWriteSession)
	{
		FOnlineAsyncTaskGDKSafeWriteSession::Initialize();
	}
	else
	{
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateSession::ProcessWriteSessionResult()
{
	//FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult();

	FGDKMultiplayerSessionHandle NewGDKSession;
	HRESULT Result = XblMultiplayerWriteSessionByHandleResult(*WriteSessionAsyncBlock, NewGDKSession.GetInitReference());
	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineAsyncTaskGDKCreateSession::ProcessWriteSessionResult:  XblMultiplayerWriteSessionByHandleResult, Result: (0x%0.8X)."), Result);

	if (Result == S_OK)
	{
		if (XAsyncGetStatus(*WriteSessionAsyncBlock, false) == HTTP_E_STATUS_PRECOND_FAILED) // Records out of sync
		{
			Retry();
			return;
		}
		GDKSession = NewGDKSession;
		GDKSessionReference = XblMultiplayerSessionSessionReference(GDKSession);
		bWasSuccessful = true;
	}
	else if (Result == HTTP_E_STATUS_PRECOND_FAILED) //Records out of sync
	{
		Retry();
		return;
	}
	else
	{
		OnFailed();
		return;
	}

	if (bWasSuccessful && SessionWriteMode == XblMultiplayerSessionWriteMode::CreateNew)
	{
		// Creation was successful. Don't finalize just yet. We need to set the host and write it again.
		SessionWriteMode = XblMultiplayerSessionWriteMode::UpdateExisting;

		FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
		check(SessionInterface);

		FUniqueNetIdGDKRef GDKUserId = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId);
		SessionInterface->GetMpsdImpl()->SetHostOnCreatedSession(GDKSession, GDKUserId->ToUint64());
			
		// This will cause another write on the session, but since the SessionWriteMode was changed to Update, it will update with the new host.
		Retry();
	}
	else
	{
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKCreateSession::Finalize()
{
	FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
	check(SessionInterface);
	const TSharedPtr<FOnlineSessionMpsdGDK>& SessionInterfaceMpsd = SessionInterface->GetMpsdImpl();

	if (bWasSuccessful)
	{
		FUniqueNetIdGDKRef GDKUserId = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId);
		bool bSetHostSuccessfully = SessionInterfaceMpsd->SetHostOnCreatedSession(GDKSession, GDKUserId->ToUint64());

		// Find the named session and link GDK platform data to it
		FNamedOnlineSessionPtr NamedSession = SessionInterface->GetNamedSessionPtr(SessionName);
		if (!NamedSession.IsValid())
		{
			bWasSuccessful = false;
			return;
		}

		if (bSetHostSuccessfully)
		{
			NamedSession->bHosting = true;
			NamedSession->HostingPlayerNum = SessionInterfaceMpsd->GetHostingPlayerNum(*UserId);;
			SessionInterfaceMpsd->DetermineSessionHost(SessionName, GDKSession);
		}
		else
		{
			NamedSession->HostingPlayerNum = INDEX_NONE;
			NamedSession->OwningUserId = UserId;
		}

		FOnlineSessionInfoMpsdGDKPtr NewSessionInfo = MakeShared<FOnlineSessionInfoMpsdGDK>(GDKSession);
		NewSessionInfo->SetSessionReady();
		NamedSession->SessionInfo  = NewSessionInfo;
		NamedSession->SessionState = EOnlineSessionState::Pending;

		SessionInterfaceMpsd->ReadSettingsFromGDKSessionJson(GDKSession, *NamedSession);

		if (bSetActivity && GDKSessionReference)
		{
			SessionInterfaceMpsd->SetUserActiveSessionActivity(*StaticCastSharedRef<const FUniqueNetIdGDK>(UserId), GDKSession);
		}
	}

	if (bWriteSession)
	{
		FOnlineAsyncTaskGDKSafeWriteSession::Finalize();

		if (!bWasSuccessful)
		{
			// If our write failed, then cleanup
			FName LambdaSessionName = SessionName;
			FGDKMultiplayerSessionHandle LambdaSessionHandle = GDKSession;

			Subsystem->ExecuteNextTick([LambdaSessionName, LambdaSessionHandle, SessionInterface]()
			{
				if (LambdaSessionHandle.IsValid())
				{
					XblMultiplayerSessionLeave(LambdaSessionHandle);
				}
				SessionInterface->RemoveNamedSession(LambdaSessionName);
			});
		}
	}

	if (bWasSuccessful)
	{
		// Initialize session state after create/join
		FSessionMessageRouterPtr MessageRouter = Subsystem->GetSessionMessageRouter();
		if (MessageRouter.IsValid())
		{
			MessageRouter->SyncInitialSessionState(SessionName, GDKSession);
		}
	}
}

void FOnlineAsyncTaskGDKCreateSession::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKCreateSession_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, SessionName);
}

#endif //WITH_GRDK