// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSafeWriteSession.h"
class FOnlineSessionGDK;
class FOnlineSessionSettings;
class FOnlineSubsystemGDK;

/** 
 *  Async task for creating an GDK session
 */
class FOnlineAsyncTaskGDKCreateSession : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	DECLARE_DELEGATE_TwoParams(FOnGDKCreateSessionComplete, bool /*bWasSuccessful*/, FName /*SessionName*/);

private:
	/** The id of the user hosting the session */
	FUniqueNetIdRef UserId;

	/** Whether to set this session as our activity upon completion or not */
	bool bSetActivity;

	/** Whether to write the GDK session to the backend or not*/
	bool bWriteSession;

	FOnGDKCreateSessionComplete TaskCompletionDelegate;

public:

	/**
	 * Constructor
	 *
	 * @param InSessionInterface The session that created this task
	 * @param InUserIndex Index of the user who created this task
	 * @param InCreator User who created this task
	 * @param InSessionName Name of the session to create
	 */
	FOnlineAsyncTaskGDKCreateSession(
			FOnlineSubsystemGDK* const InSubsystem,
			FGDKContextHandle InGDKContext,
			const FUniqueNetIdRef InUserId,
			const FName InSessionName,
			FGDKMultiplayerSessionHandle InGDKSession,
			const bool bInWriteSession,
			const bool bInSetActivity,
			FOnGDKCreateSessionComplete InTaskCompletionDelegate);

	// FOnlineAsyncItem
	virtual void Initialize();
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKCreateSession SessionName: %s bWasSuccessful: %d"), *SessionName.ToString(), bWasSuccessful.operator bool()); }
	virtual void Finalize() override;
	virtual void ProcessWriteSessionResult() override;
	virtual void TriggerDelegates() override;
	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session) override { return true; }
};
