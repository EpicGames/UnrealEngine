// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKDestroySession.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "Interfaces/VoiceInterface.h"

THIRD_PARTY_INCLUDES_START
#define _UITHREADCTXT_SUPPORT   0
#include <ppltasks.h>
THIRD_PARTY_INCLUDES_END

//@todo: Ideally, this should be able to destroy both the XBL game session and match session associated
// with a NamedSession at the same time.

void CreateDestroyMatchmakingCompleteTask(FName SessionName,
								   FGDKMultiplayerSessionHandle Session,
								   FOnlineSubsystemGDK* Subsystem,
								   bool bWasSuccessful)
{
	// This function should only be called on the game thread.
	check(IsInGameThread());
	check(Subsystem);

	FOnlineMatchmakingInterfaceGDKPtr MatchmakingInt = Subsystem->GetMatchmakingInterfaceGDK();
	check(MatchmakingInt.IsValid());

	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to leave session %s, session gone. bWasSuccessful: 0"), *SessionName.ToString());

		// Session timed out?
		FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(Subsystem, SessionName);
		// This case only occurs if something failed during matchmaking, so always pass false for bWasSuccessful.
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyMatchmakingCompleteTask_Delegate);
		MatchmakingInt->TriggerOnMatchmakingCompleteDelegates(SessionName, false);
		return;
	}

	const FOnlineIdentityGDKPtr IdentityInt = Subsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	uint64 NumberOfMembers = 0;
	const XblMultiplayerSessionMember* Members;
	
	XblMultiplayerSessionMembers(Session, &Members, &NumberOfMembers);

	// Find the local player in the session and make them Leave() it.
	const XblMultiplayerSessionMember* Member = Members;
	for (int i = 0; i < NumberOfMembers; ++i, ++Member)
	{
		FUniqueNetIdGDKRef MemberId = FUniqueNetIdGDK::Create(Member->Xuid);
		FGDKUserHandle MemberUser = IdentityInt->GetUserForUniqueNetId(*MemberId);
		if (MemberUser.IsValid())
		{
			// Found a member to Leave() the session.
			FGDKContextHandle UserContext = Subsystem->GetGDKContext(MemberUser);
			check(UserContext);

			UE_LOG_ONLINE_SESSION(Log, TEXT("Leaving session %s for user %lld"), *SessionName.ToString(), Member->Xuid);
			Subsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKDestroyMatchmakingSession>(Subsystem, UserContext, SessionName, Session);
			return;
		}
	}

	UE_LOG_ONLINE_SESSION(Log, TEXT("Finished leaving Session %s for all local-members bWasSuccessful: 0"), *SessionName.ToString());
	// If we get here, we couldn't find a local user in the session, so remove it and trigger the success delegate.
	FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(Subsystem, SessionName);
	// This case only occurs if something failed during matchmaking, so always pass false for bWasSuccessful.
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyMatchmakingCompleteTask_Delegate);
	MatchmakingInt->TriggerOnMatchmakingCompleteDelegates(SessionName, false);
}

void CreateDestroyTask(FName SessionName,
					   FGDKMultiplayerSessionHandle Session,
					   FOnlineSubsystemGDK* Subsystem,
					   bool bWasSuccessful,
					   const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	// This function should only be called on the game thread.
	check(IsInGameThread());
	check(Subsystem);

	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	check(SessionInt.IsValid());

	if (!Session.IsValid())
	{
		// Session timed out?
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to leave session %s, session gone? bWasSuccessful: %d"), *SessionName.ToString(), bWasSuccessful);
		FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(Subsystem, SessionName);
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyTask_SessionInvalid_CompletionDelegate);
			CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
		}
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyTask_SessionInvalid_OnDestroySessionComplete);
			SessionInt->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
		}
		return;
	}

	const FOnlineIdentityGDKPtr IdentityInt = Subsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	// Find the local player in the session and make them Leave() it.
	uint64 NumberOfMembers = 0;
	const XblMultiplayerSessionMember* Members;

	XblMultiplayerSessionMembers(Session, &Members, &NumberOfMembers);

	// Find the local player in the session and make them Leave() it.
	const XblMultiplayerSessionMember* Member = Members;
	for (int i = 0; i < NumberOfMembers; ++i, ++Member)
	{
		FUniqueNetIdGDKRef MemberId = FUniqueNetIdGDK::Create(Member->Xuid);
		FGDKUserHandle MemberUser = IdentityInt->GetUserForUniqueNetId(*MemberId);
		if (MemberUser.IsValid())
		{
			// Found a member to Leave() the session.
			FGDKContextHandle UserContext = Subsystem->GetGDKContext(MemberUser);
			if(UserContext.IsValid())
			{
				UE_LOG_ONLINE_SESSION(Log, TEXT("Leaving session %s for user %llu"), *SessionName.ToString(), Member->Xuid);
				Subsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKDestroySession>(Subsystem, UserContext, SessionName, Session, CompletionDelegate);
				return;
			}
		}
	}

	UE_LOG_ONLINE_SESSION(Log, TEXT("Finished leaving Session %s for all local-members bWasSuccessful: %d"), *SessionName.ToString(), bWasSuccessful);
	// If we get here, we couldn't find a local user in the session, so remove it and trigger the success delegate.
	FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(Subsystem, SessionName);
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyTask_NoLocalUser_CompletionDelegate);
		CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
	}
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OnlineAsyncTaskGDKDestroySession_CreateDestroyTask_NoLocalUser_OnDestroySessionComplete);
		SessionInt->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
	}
}

bool FOnlineAsyncTaskGDKDestroySessionBase::UpdateSession(FGDKMultiplayerSessionHandle Session)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKDestroySessionBase::UpdateSession begin"));
	HRESULT Result = XblMultiplayerSessionLeave(Session);
	bLeaveSuccessful = (Result == S_OK);

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKDestroySessionBase::UpdateSession end Result: (0x%0.8X)."), Result);
	return true;
}

void FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(FOnlineSubsystemGDK* const Subsystem, const FName SessionName)
{
	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	FOnlineMatchmakingInterfaceGDKPtr MatchmakingInt = Subsystem->GetMatchmakingInterface();

	SessionInt->RemoveNamedSession(SessionName);
	MatchmakingInt->RemoveMatchmakingTicket(SessionName);

	// TODO: remove this block? The game should be deciding when to talk or not talk?
	if (SessionInt->GetNumSessions() == 0)
	{
		IOnlineVoicePtr VoiceInt = Subsystem->GetVoiceInterface();
		if (VoiceInt.IsValid())
		{
			if (!Subsystem->IsDedicated())
			{
				// Stop local talkers
				VoiceInt->UnregisterLocalTalkers();
			}

			// Stop remote voice 
			VoiceInt->RemoveAllRemoteTalkers();
		}
	}
}

void FOnlineAsyncTaskGDKDestroyMatchmakingSession::Finalize()
{
	FOnlineAsyncTaskGDKSafeWriteSession::Finalize();

	// Attempt to find another local user to Leave() the session.
	CreateDestroyMatchmakingCompleteTask(GetSessionName(), GetLatestGDKSession(), Subsystem, bWasSuccessful);
}

void FOnlineAsyncTaskGDKDestroySession::Finalize()
{
	FOnlineAsyncTaskGDKSafeWriteSession::Finalize();

	if (bLeaveSuccessful)
	{
		// Attempt to find another local user to Leave() the session.
		CreateDestroyTask(GetSessionName(), GetLatestGDKSession(), Subsystem, bWasSuccessful, CompletionDelegate);
	}
	else
	{
		// If the user failed to leave, calling CreateDestroyTask again will cause a loop, so we'll destroy the session instead
		FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
		check(SessionInt.IsValid());

		UE_LOG_ONLINE_SESSION(Log, TEXT("User failed to leave Session %s. Destroying session."), *SessionName.ToString());
		
		FOnlineAsyncTaskGDKDestroySessionBase::RemoveAndCleanupSession(Subsystem, SessionName);
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKDestroySession_Finalize_CompletionDelegate);
			CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
		}
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKDestroySession_Finalize_OnDestroySessionComplete);
			SessionInt->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}
}

#endif //WITH_GRDK