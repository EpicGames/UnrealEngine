// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSafeWriteSession.h"
#include "Interfaces/OnlineSessionInterface.h"

class FOnlineSubsystemGDK;

/** 
 * Async task used to have every local player Leave() the Live session.
 */
class FOnlineAsyncTaskGDKDestroySessionBase : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning Live online subsystem
	 * @param InContext the Live context of the user who's updating the session
	 * @param InSessionName the name of the session to destroy
	 * @param InGDKSession the GDK session we're writing to
	 */
	FOnlineAsyncTaskGDKDestroySessionBase(
		FOnlineSubsystemGDK* const InSubsystem,
		const FString& AsyncTaskName,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession
	)
		: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, AsyncTaskName, InContext, InSessionName, InGDKSession)
	{
	}

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKDestroySessionBase %s"), *GetAsyncSafeWriteTaskInfoString()); }

	/** Triggers the appropriate delegate based on DelegateType */
	static void RemoveAndCleanupSession(
		FOnlineSubsystemGDK* const Subsystem,
		const FName SessionName);

private:

	// FOnlineAsyncTaskLiveSafeWriteSession
	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session);

protected:

	// Whether the session leave command was processed successfully or not in UpdateSession.
	bool bLeaveSuccessful = false;
};

class FOnlineAsyncTaskGDKDestroyMatchmakingSession : public FOnlineAsyncTaskGDKDestroySessionBase
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the Live context of the user who's updating the session
	 * @param InSessionName the name of the session to destroy
	 * @param InGDKSession the GDK session we're writing to
	 */
	FOnlineAsyncTaskGDKDestroyMatchmakingSession(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession
	)
		: FOnlineAsyncTaskGDKDestroySessionBase(InSubsystem, TEXT("FOnlineAsyncTaskGDKDestroyMatchmakingSession"), InContext, InSessionName, InGDKSession)
	{
	}

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKDestroyMatchmakingSession %s"), *GetAsyncSafeWriteTaskInfoString()); }
	virtual void Finalize() override;

};

class FOnlineAsyncTaskGDKDestroySession : public FOnlineAsyncTaskGDKDestroySessionBase
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the GDK context of the user who's updating the session
	 * @param InSessionName the name of the session to update
	 * @param InGDKSession the GDK session we're writing to
	 * @param InCompletionDelegate the delegate to call once we're finished
	 */
	FOnlineAsyncTaskGDKDestroySession(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession,
		const FOnDestroySessionCompleteDelegate& InCompletionDelegate
	)
		: FOnlineAsyncTaskGDKDestroySessionBase(InSubsystem, TEXT("FOnlineAsyncTaskGDKDestroySession"), InContext, InSessionName, InGDKSession)
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKDestroySession %s"), *GetAsyncSafeWriteTaskInfoString()); }
	virtual void Finalize() override;

private:
	FOnDestroySessionCompleteDelegate CompletionDelegate;
};

void CreateDestroyMatchmakingCompleteTask(FName SessionName,
	FGDKMultiplayerSessionHandle Session,
	FOnlineSubsystemGDK* Subsystem,
	bool bWasSuccessful);

void CreateDestroyTask(FName SessionName,
	FGDKMultiplayerSessionHandle Session,
	FOnlineSubsystemGDK* Subsystem,
	bool bWasSuccessful,
	const FOnDestroySessionCompleteDelegate& CompletionDelegate);
