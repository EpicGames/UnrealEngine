// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKFindSessionById.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"

FOnlineAsyncTaskGDKFindSessionById::FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
																		 FGDKContextHandle InGDKContext,
																		 const int32 InLocalUserNum,
																		 FString&& InSessionIdString,
																		 const FOnSingleSessionResultCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKFindSessionById"))
	, LocalUserNum(InLocalUserNum)
	, SessionIdString(MoveTemp(InSessionIdString))
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
	, SessionReference()
	, bFindByHandle(false)
{
	FMemory::Memzero(&InviteHandle, sizeof(InviteHandle));
}

FOnlineAsyncTaskGDKFindSessionById::FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	const int32 InLocalUserNum,
	const XblMultiplayerSessionReference* InSessionReference,
	const FOnSingleSessionResultCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKFindSessionById"))
	, LocalUserNum(InLocalUserNum)
	, SessionIdString(FString())
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
	, SessionReference(*InSessionReference)
	, bFindByHandle(false)
{
	check(TCString<ANSICHAR>::Strlen(SessionReference.SessionName) > 0);
	FMemory::Memzero(&InviteHandle, sizeof(InviteHandle));
}

FOnlineAsyncTaskGDKFindSessionById::FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	const int32 InLocalUserNum,
	XblMultiplayerInviteHandle InInviteHandle,
	const FOnSingleSessionResultCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKFindSessionById"))
	, LocalUserNum(InLocalUserNum)
	, SessionIdString(FString())
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
	, SessionReference()
	, InviteHandle(InInviteHandle)
	, bFindByHandle(true)
{
}

FOnlineAsyncTaskGDKFindSessionById::FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	const int32 InLocalUserNum,
	const char* InSessionHandleId,
	const FOnSingleSessionResultCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKFindSessionById"))
	, LocalUserNum(InLocalUserNum)
	, SessionIdString(FString())
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
	, SessionReference()
	, bFindByHandle(true)
{
	FMemory::Memcpy(InviteHandle.Data, InSessionHandleId, XBL_GUID_LENGTH);
}

void FOnlineAsyncTaskGDKFindSessionById::Initialize()
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("Attempting to find Session by Id %s"), *SessionIdString);

	HRESULT Result = S_OK;
	// With no session reference, we assume that we need to look this up by URI
	
	if (!bFindByHandle && !TCString<ANSICHAR>::Strlen(SessionReference.SessionName))
	{
		Result = XblMultiplayerSessionReferenceParseFromUriPath(TCHAR_TO_UTF8(*SessionIdString), &SessionReference);
	}

	if(Result == S_OK)
	{
		if (!bFindByHandle)
		{
			Result = XblMultiplayerGetSessionAsync(GDKContext, &SessionReference, *AsyncBlock);
		}
		else
		{
			Result = XblMultiplayerGetSessionByHandleAsync(GDKContext, InviteHandle.Data, *AsyncBlock);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error starting FindSessionById query, error: (0x%08X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}

	return;
}

void FOnlineAsyncTaskGDKFindSessionById::ProcessResults()
{
	FGDKMultiplayerSessionHandle GDKSession;
	HRESULT Result = S_OK;
	
	if (bFindByHandle)
	{
		Result = XblMultiplayerGetSessionByHandleResult(*AsyncBlock, GDKSession.GetInitReference());
	}
	else
	{
		Result = XblMultiplayerGetSessionResult(*AsyncBlock, GDKSession.GetInitReference());
	}

	if(Result == S_OK && GDKSession.IsValid())
	{
		FString HostDisplayName;
		FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
		check(SessionInterface.IsValid())
		SearchResult = SessionInterface->GetMpsdImpl()->CreateSearchResultFromSession(GDKSession, HostDisplayName, GDKContext);

		bWasSuccessful = true;
		OnlineError.bSucceeded = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error during FindSessionById, error: (0x%08X). Session isValid= %s "), Result, GDKSession.IsValid()? TEXT("true"): TEXT("false"));
		bWasSuccessful = false;
	}
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKFindSessionById::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKFindSessionById_TriggerDelegates);
	Delegate.ExecuteIfBound(LocalUserNum, OnlineError.bSucceeded, SearchResult);
}

#endif //WITH_GRDK