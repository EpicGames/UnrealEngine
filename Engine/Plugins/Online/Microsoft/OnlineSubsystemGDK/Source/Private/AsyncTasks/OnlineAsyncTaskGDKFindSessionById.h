// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSessionGDK.h"
#include "OnlineError.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to find the session for a session id
 */
class FOnlineAsyncTaskGDKFindSessionById
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
										FGDKContextHandle InGDKContext,
										const int32 InLocalUserNum,
										FString&& InSessionIdString,
										const FOnSingleSessionResultCompleteDelegate& InDelegate);

	FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
										FGDKContextHandle InGDKContext,
										const int32 InLocalUserNum,
										const XblMultiplayerSessionReference* InSessionReference,
										const FOnSingleSessionResultCompleteDelegate& InDelegate);

	FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
										FGDKContextHandle InGDKContext,
										const int32 InLocalUserNum,
										XblMultiplayerInviteHandle InInviteHandle,
										const FOnSingleSessionResultCompleteDelegate& InDelegate);

	FOnlineAsyncTaskGDKFindSessionById(FOnlineSubsystemGDK* InGDKSubsystem,
										FGDKContextHandle InGDKContext,
										const int32 InLocalUserNum,
										const char* InSessionHandleId,
										const FOnSingleSessionResultCompleteDelegate& InDelegate);

	virtual ~FOnlineAsyncTaskGDKFindSessionById() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKFindSessionById bSuccessful: %d LocalUserNum: %d SessionId: %s Error: %s"), OnlineError.bSucceeded, LocalUserNum, *SessionIdString, *OnlineError.ErrorCode); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void TriggerDelegates();

protected:
	TArray<FUniqueNetIdRef> UsersToQuery;
	int32 LocalUserNum;
	FString SessionIdString;
	FOnSingleSessionResultCompleteDelegate Delegate;
	FOnlineError OnlineError;
	FOnlineSessionSearchResult SearchResult;
	FGDKContextHandle GDKContext;
	XblMultiplayerSessionReference SessionReference;
	XblMultiplayerInviteHandle InviteHandle;
	bool bFindByHandle;
};
