// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKFindSessionsBySearchHandle.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemSessionSettings.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineAsyncTaskGDKFindSessionById.h"
#include "Misc/ScopeLock.h"

FOnlineAsyncTaskGDKFindSessionsBySearchHandle::FOnlineAsyncTaskGDKFindSessionsBySearchHandle(
	FOnlineSubsystemGDK* InSubsystemGDK,
	FOnlineSessionGDK* InSessionInterface,
	FGDKContextHandle InGDKContext,
	TSharedPtr<FOnlineSessionSearch> InSearchSettings)
	: FOnlineAsyncTaskGDK(InSubsystemGDK, TEXT("FOnlineAsyncTaskGDKFindSessionsBySearchHandle"))
	, SearchSettings(InSearchSettings)
	, SessionInterface(InSessionInterface)
	, GDKContext(InGDKContext)
	, NumExpectedResults(0)
{
	check(SessionInterface);
}

void FOnlineAsyncTaskGDKFindSessionsBySearchHandle::Initialize()
{
	// Free up previous results
	SearchSettings->SearchResults.Empty();

	//TODO: support searching for tags

	FString QueryString;
	for (auto SearchParam : SearchSettings->QuerySettings.SearchParams)
	{
		double NumericValue = 0.0f;
		const FString& SettingName = SearchParam.Key.ToString();
		const FVariantData& SettingValue = SearchParam.Value.Data;
		if (SettingValue.ToString().IsEmpty() || !SettingName.Equals(SEARCH_KEYWORDS.ToString()))
		{
			continue;
		}

		if (!QueryString.IsEmpty())
		{
			QueryString.Append(TEXT(" and "));
		}

		EOnlineComparisonOp::Type ComparisonOp = SearchParam.Value.ComparisonOp;
		FString ComparisonString = TEXT("eq");
		switch (ComparisonOp)
		{
			case EOnlineComparisonOp::Equals:
			{
				// Already defaulted to EQ
				break;
			}

			case EOnlineComparisonOp::NotEquals:
			{
				ComparisonString = TEXT("ne");
				break;
			}

			case EOnlineComparisonOp::GreaterThanEquals:
			{
				ComparisonString = TEXT("ge");
				break;
			}

			case EOnlineComparisonOp::GreaterThan:
			{
				ComparisonString = TEXT("gt");
				break;
			}

			case EOnlineComparisonOp::LessThanEquals:
			{
				ComparisonString = TEXT("le");
				break;
			}

			case EOnlineComparisonOp::LessThan:
			{
				ComparisonString = TEXT("lt");
				break;
			}

			default:
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Comparison Operation %s not supported in GDK session queries. Defaulting to EQ"), EOnlineComparisonOp::ToString(ComparisonOp));
			}
		}

		if (SettingValue.IsNumeric())
		{
			QueryString.Append(FString::Printf(TEXT("numbers/%s %s %s"), *SettingName.ToLower(), *ComparisonString, *SettingValue.ToString()));
		}
		else
		{
			QueryString.Append(FString::Printf(TEXT("tolower(strings/%s) %s '%s'"), *SettingName.ToLower(), *ComparisonString, *SettingValue.ToString().ToLower()));
		}
	}

	UE_LOG_ONLINE_SESSION(Log, TEXT("Searching for Search Handles with query string: %s"), *QueryString);

	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);

	FString SessionTemplateName;
	SearchSettings->QuerySettings.Get(SETTING_SESSION_TEMPLATE_NAME, SessionTemplateName);
	if (SessionTemplateName.IsEmpty())
	{
		SessionTemplateName = FName(NAME_GameSession).ToString();
	}

	HRESULT Result = XblMultiplayerGetSearchHandlesAsync(GDKContext, Scid, TCHAR_TO_UTF8(*SessionTemplateName), "", true, TCHAR_TO_UTF8(*QueryString), nullptr, *AsyncBlock);
	if (Result != S_OK)
	{
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
}

void FOnlineAsyncTaskGDKFindSessionsBySearchHandle::ProcessResults()
{
	HRESULT Result = XblMultiplayerGetSearchHandlesResultCount(*AsyncBlock, &NumExpectedResults);
	if(Result == S_OK)
	{
		if (NumExpectedResults == 0)
		{
			SearchSettings->SearchState = EOnlineAsyncTaskState::Done;
			SessionInterface->GetMpsdImpl()->CurrentSessionSearch = nullptr;
				
			bWasSuccessful = true;
			bIsComplete = true;
			return;
		}
		else
		{
			TArray<XblMultiplayerSearchHandle> QueryResults;
			QueryResults.Reserve(NumExpectedResults);
			Result = XblMultiplayerGetSearchHandlesResult(*AsyncBlock, QueryResults.GetData(), NumExpectedResults);
			if (Result == S_OK)
			{
				QueryResults.SetNumUninitialized(NumExpectedResults);
				for (const XblMultiplayerSearchHandle& QueryResult : QueryResults)
				{
					const char* SessionId = nullptr;
					Result = XblMultiplayerSearchHandleGetId(QueryResult, &SessionId);
					if (Result == S_OK)
					{
						FString HostDisplayName;
						FOnlineSessionGDKPtr SessionInterfaceLocal = Subsystem->GetSessionInterfaceGDK();
						check(SessionInterfaceLocal.IsValid())
						FGDKMultiplayerSearchHandle SearchHandle(QueryResult);
						FOnlineSessionSearchResult SearchResult = SessionInterfaceLocal->GetMpsdImpl()->CreateSearchResultFromSearchHandle(SearchHandle, HostDisplayName, GDKContext);
						OnGetSingleSessionComplete(0, true, SearchResult);						
					}
					else
					{
						--NumExpectedResults;
						
						if (NumExpectedResults == 0)
						{
							bWasSuccessful = false;
							bIsComplete = true;
						}
					}
				}
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("MultiplayerService::GetSessionsAsync with 0x%0.8X"), Result);

		SessionInterface->GetMpsdImpl()->CurrentSessionSearch = nullptr;
		SearchSettings->SearchState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;		
	}
}

void FOnlineAsyncTaskGDKFindSessionsBySearchHandle::OnGetSingleSessionComplete(int32 LocalUserNum, bool bSucceeded, const FOnlineSessionSearchResult& SearchResult)
{
	// Lock for the entirety of this scope to protect safe access to SearchSettings' SearchResults and ExpectedResults
	FScopeLock Lock(&SessionInterface->GetMpsdImpl()->SessionResultLock);

	if(bSucceeded)
	{
		FString HostDisplayName;
		TSharedPtr<FOnlineSessionInfoMpsdGDK> GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SearchResult.Session.SessionInfo);
		const XblMultiplayerSessionMember* HostMember = SessionInterface->GetMpsdImpl()->GetGDKSessionHost(GDKSessionInfo->GetGDKMultiplayerSession());
		if (HostMember)
		{
			// XR-46 permits the use of Gamertag here.
			HostDisplayName = UTF8_TO_TCHAR(HostMember->Gamertag);
		}
		else
		{
			HostDisplayName = TEXT("Unknown Host");
		}

		SearchSettings->SearchResults.Add(SearchResult);
			
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("A MultiplayerService::GetCurrentSessionAsync call failed"));
	}

	--NumExpectedResults;

	if (NumExpectedResults == 0)
	{
		bWasSuccessful = true;
		bIsComplete = true;
		//PingResultsAndTriggerDelegates(SearchSettings);
	}
}


void FOnlineAsyncTaskGDKFindSessionsBySearchHandle::Finalize()
{
	if (bWasSuccessful != false)
	{
		/*
		// Free up previous results
		SearchSettings->SearchResults.Empty();

		// Copy results from SearchResults
		for (int i = 0; i < SearchResults.Num(); ++i)
		{
			FString HostDisplayName;
			if (Profiles.Num() > i && Profiles[i])
			{
				HostDisplayName = UTF8_TO_TCHAR(Profiles[i]->gameDisplayName);
			}
			SearchSettings->SearchResults.Add(SessionInterface->CreateSearchResultFromSession(SearchResults[i], HostDisplayName));
		}
		*/
		SearchSettings->SearchState = EOnlineAsyncTaskState::Done;
	}
	else
	{
		SearchSettings->SearchState = EOnlineAsyncTaskState::Failed;
	}

	SessionInterface->GetMpsdImpl()->CurrentSessionSearch = nullptr;
}

void FOnlineAsyncTaskGDKFindSessionsBySearchHandle::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKFindSessionsBySearchHandle_TriggerDelegates);
	SessionInterface->TriggerOnFindSessionsCompleteDelegates(bWasSuccessful);
}

#endif //WITH_GRDK