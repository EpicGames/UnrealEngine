// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

class FOnlineSessionSearch;
class FOnlineSessionGDK;

/** 
 * Async item used to marshal search results from the system callback thread to the game thread.
 */
class FOnlineAsyncTaskGDKFindSessionsBySearchHandle : public FOnlineAsyncTaskGDK
{
private:
	/** Pointer to the settings used for this search. */
	TSharedPtr<FOnlineSessionSearch> SearchSettings;

	/** All the results returned from the MPSD. */
	TArray<XblMultiplayerSessionHandle> SearchResults;

	/** Parallel vector of profiles of the host of each session, used to get the GameDisplayName */
	TArray<XblUserProfile*> Profiles;

	/** The session that created this task */
	FOnlineSessionGDK* SessionInterface;

	FGDKContextHandle GDKContext;

	uint64 NumExpectedResults;

public:
	FOnlineAsyncTaskGDKFindSessionsBySearchHandle(
		FOnlineSubsystemGDK* InSubsystemGDK,
		FOnlineSessionGDK* InSessionInterface,
		FGDKContextHandle GDKContext,
		TSharedPtr<FOnlineSessionSearch> InSearchSettings);

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKFindSessionsBySearchHandle"); }
	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	void OnGetSingleSessionComplete(int32 LocalUserNum, bool bSucceeded, const FOnlineSessionSearchResult& SearchResult);
};
