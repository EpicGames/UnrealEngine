// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGameSessionReady.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "OnlineAsyncTaskGDKSetSessionActivity.h"

THIRD_PARTY_INCLUDES_START
#include <Ws2tcpip.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKGameSessionReady::FOnlineAsyncTaskGDKGameSessionReady(
	FOnlineSubsystemGDK* InSubsystem,
	FGDKContextHandle InContext,
	FName InSessionName,
	const XblMultiplayerSessionReference* InGameSessionRef)
	: FOnlineAsyncTaskGDK(InSubsystem, TEXT("FOnlineAsyncTaskGDKGameSessionReady"), 0)
	, SessionName(InSessionName)
	, GameSessionRef(InGameSessionRef)
	, GDKContext(InContext)
{
}

void FOnlineAsyncTaskGDKGameSessionReady::Initialize()
{

	check(GDKContext.IsValid());

	HRESULT Result = S_OK;
	FNamedOnlineSession* const NamedSession = Subsystem->GetSessionInterfaceGDK()->GetNamedSession(SessionName);
	if (NamedSession->SessionInfo.IsValid())
	{
		if (StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo)->GetGDKMultiplayerSearchHandle().IsValid())
		{
			const char* HandleID = nullptr;
			Result = XblMultiplayerSearchHandleGetId(StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo)->GetGDKMultiplayerSearchHandle(), &HandleID);
			if (Result == S_OK)
			{
				Result = XblMultiplayerGetSessionByHandleAsync(GDKContext, HandleID, *AsyncBlock);
				if (Result != S_OK)
				{
					UE_LOG_ONLINE_SESSION(Warning, L"Failed to find session with result 0x%0.8X", Result);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, L"Failed get search handleID with result 0x%0.8X", Result);
			}
		}
		else
		{
			Result = XblMultiplayerGetSessionAsync(GDKContext, StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo)->GetGDKMultiplayerSessionRef(), *AsyncBlock);
		}
	}
	else
	{
		Result = XblMultiplayerGetSessionAsync(GDKContext, GameSessionRef, *AsyncBlock);
	}
}

void FOnlineAsyncTaskGDKGameSessionReady::ProcessResults()
{
	HRESULT Result = XblMultiplayerGetSessionResult(*AsyncBlock, GDKSession.GetInitReference());

	if (Result == S_OK)
	{
		if (GDKSession.IsValid())
		{
			UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - Session ready!");

			auto SessionInterface = StaticCastSharedPtr<FOnlineSessionGDK>(Subsystem->GetSessionInterface());

			//. Mark the current user as active, or they will time out and be removed from the session
			SessionInterface->GetMpsdImpl()->SetCurrentUserActive(GDKSession, true); // @todo: remove user index param

			FNamedOnlineSession* const NamedSession = SessionInterface->GetNamedSession(SessionName);			

			if (NamedSession->SessionSettings.bIsDedicated)
			{
				const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
				if (SessionProperties && SessionProperties->ServerConnectionStringCandidatesCount)
				{
					const  FUTF8ToTCHAR HostAddress(SessionProperties->ServerConnectionStringCandidates[0]);
					FString HostAddressString(HostAddress.Get());
					HostAddr = FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64(HostAddressString);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - No server candidate for dedicated session");
					HostAddr = TSharedPtr<FInternetAddr>();
				}
				bWasSuccessful = true;
				bIsComplete = true;
			}
			else
			{
				const XblMultiplayerSessionMember* CurrentHost = FOnlineSessionMpsdGDK::GetGDKSessionHost(GDKSession);
					// Host selection - if there isn't a host yet, try and get the prefered host candidate.
					// if there are no candidates proceed under the assumption the player at index 0 will host. 
					// If we are the player at index 0 try to become the host.
					// If we are not player 0, and player 0 does not have address information wait 10 seconds before failing and requing the task.

					if (!CurrentHost || (TCString<ANSICHAR>::Strlen(CurrentHost->DeviceToken.Value) <= 0))
					{
						const XblDeviceToken* deviceTokens = nullptr;
						size_t deviceTokensCount = 0;
						Result = XblMultiplayerSessionHostCandidates(
							GDKSession,
							&deviceTokens,
							&deviceTokensCount
						);
						UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - No Current host");
						if (FAILED(Result) || deviceTokensCount == 0)
						{
							const XblMultiplayerSessionMember* AssumedHost = nullptr;
							const XblMultiplayerSessionMember* Members = nullptr;
							uint64 NumMembers = 0;
							Result = XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);
							if (SUCCEEDED(Result) && NumMembers > 0)
							{
								AssumedHost = &Members[0];
							}
							if (AssumedHost != nullptr)
							{
								UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - Player 0 id = %hs", AssumedHost->DeviceToken.Value);

								if (AssumedHost->IsCurrentUser)
								{
									XblMultiplayerSessionSetHostDeviceToken(GDKSession, AssumedHost->DeviceToken);
									UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - I am first player become host %hs", AssumedHost->DeviceToken.Value);
								}
								else
								{
									
									UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - I wasn't first player, assuming someone else is going to host");
																		
									FSessionSettings* HostMemberSettings = NamedSession->SessionSettings.MemberSettings.Find(FUniqueNetIdGDK::Create(AssumedHost->Xuid));									
									
									if (!HostMemberSettings || !HostMemberSettings->Find(FName("Registered")))
									{
										UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - I wasn't first player, waiting to see if someone else becomes host");
										Sleep(2500);
										bWaitAndTryAgain = true;
										bWasSuccessful = false;
										bIsComplete = true;
										return;
									}
									else
									{
										UE_LOG_ONLINE_SESSION(Warning, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - I wasn't first player, but the first player is ready to host");
										XblMultiplayerSessionSetHostDeviceToken(GDKSession, AssumedHost->DeviceToken);
										CurrentHost = AssumedHost;
									}
								}
							}
							else
							{
								UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - No host candidates. We will not try and host");
							}
						}
						else if (const XblMultiplayerSessionMember* CurrentUser = XblMultiplayerSessionCurrentUser(GDKSession))
						{
							for (size_t i = 0; i < deviceTokensCount; ++i)
							{
								UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - Host candidate %lld = %hs", i, deviceTokens[i].Value);
								if (strcmp(deviceTokens[i].Value, CurrentUser->DeviceToken.Value) == 0)
								{
									UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - This candidate is us");
									if (i == 0)
									{
										UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - We are the first candidate, become host");
										XblMultiplayerSessionSetHostDeviceToken(GDKSession, CurrentUser->DeviceToken);
									}
									else
									{
										UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - We are not the first candidate, we will not host");
									}
								}
							}
						}
						else
						{
							UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - Couldn't get local session member, won't try and host");

						}

					}
					else
					{
						UE_LOG_ONLINE_SESSION(Log, L"FOnlineAsyncTaskGDKGameSessionReady::ProcessResults - Session already has host!");
					}
				const XblMultiplayerSessionMember* NewHostMember = FOnlineSessionMpsdGDK::GetGDKSessionHost(GDKSession);

				// Find the local player in the session and make them Leave() it.
				uint64 NumberOfMembers = 0;
				const XblMultiplayerSessionMember* Members;

				XblMultiplayerSessionMembers(GDKSession, &Members, &NumberOfMembers);

				// Find the local player in the session and make them Leave() it.
				const XblMultiplayerSessionMember* Member = Members;
				for (int i = 0; i < NumberOfMembers; ++i, ++Member)
				{
					const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);

					if (Member->IsCurrentUser && (FCStringAnsi::Strncmp(Member->DeviceToken.Value, SessionProperties->HostDeviceToken.Value, XBL_MULTIPLAYER_DEVICE_TOKEN_MAX_LENGTH) == 0))
					{
						// If this console is the host, it will wait for SecureDeviceAssociatons from the clients,
						// so we're done here.
						UE_LOG_ONLINE_SESSION(Log, TEXT("This console is the session host."));

						bWasSuccessful = GDKSession.IsValid();
						bIsComplete = true;
						return;
					}
				}

				// Store the host address, will be set in Finalize() since Finalize() runs on the game thread.
				if (NewHostMember && NewHostMember->SecureDeviceBaseAddress64)
				{
					const  FUTF8ToTCHAR HostAddress(NewHostMember->SecureDeviceBaseAddress64);
					FString HostAddressString(HostAddress.Get());
					HostAddr = FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64(HostAddressString);
				}
				else
				{
					HostAddr = TSharedPtr<FInternetAddr>();
				}
				bWasSuccessful = true;
				bIsComplete = true;
			}
		}
		else
		{
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Log, L"OnGameSessionReady: GetCurrentSessionAsync failed with 0x%0.8X", Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}


void FOnlineAsyncTaskGDKGameSessionReady::Finalize()
{
	if (bWaitAndTryAgain)
	{
		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGameSessionReady>(Subsystem, GDKContext, SessionName, GameSessionRef);
		return;
	}

	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	check(SessionInt.IsValid());
	const TSharedPtr<FOnlineSessionMpsdGDK>& SessionIntMpsd = SessionInt->GetMpsdImpl();

	if (!bWasSuccessful)
	{
		// @v2live Should this use CreateDestroyTask to ensure session is actually cleared out?
		SessionInt->RemoveNamedSession(SessionName);
		return;
	}

	// Get the GDK session info
	FNamedOnlineSession* const NamedSession = SessionInt->GetNamedSession(SessionName);
	check(NamedSession != nullptr);

	// Set activity to new session. This will be the session used for invites/join in progress if supported.
	bool bUpdateUserActiveSessionActivity = true;
	if (const FOnlineSessionSetting* ActivitySessionSetting = NamedSession->SessionSettings.Settings.Find(SETTING_ACTIVITY_SESSION))
	{
		ActivitySessionSetting->Data.GetValue(bUpdateUserActiveSessionActivity);
	}
	if (bUpdateUserActiveSessionActivity)
	{
		FGDKUserHandle GDKUser;
		XblContextGetUser(GDKContext, GDKUser.GetInitReference());
		if (GDKUser)
		{
			uint64 GDKUserId;
			if (SUCCEEDED(XUserGetId(GDKUser, &GDKUserId)))
			{
				FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(GDKUserId);
				SessionIntMpsd->SetUserActiveSessionActivity(*UserNetId, GDKSession);
			}
		}
	}

	SessionIntMpsd->ReadSettingsFromGDKSessionJson(GDKSession, *NamedSession);
	NamedSession->SessionState = EOnlineSessionState::Pending;

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	check(GDKInfo.IsValid());

	Subsystem->CacheGDKSession(SessionName, GDKSession);
	GDKInfo->SetHostAddr(HostAddr);

	SessionIntMpsd->DetermineSessionHost(SessionName, GDKSession);

	// Host will re-advertise the match
	FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
	MatchmakingInterface->SetTicketState(SessionName, EOnlineGDKMatchmakingState::Active);
	GDKInfo->SetSessionReady();

	Subsystem->GetMatchmakingInterfaceGDK()->SubmitMatchingTicket(GDKInfo->GetGDKMultiplayerSession(), SessionName, false);
}

void FOnlineAsyncTaskGDKGameSessionReady::TriggerDelegates()
{
	if (bWaitAndTryAgain)
	{
		return;
	}
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGameSessionReady_TriggerDelegates);
		Subsystem->GetMatchmakingInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
	}
	if (GDKSession.IsValid())
	{
		Subsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKSafeWriteSession>(Subsystem, TEXT("FOnlineAsyncTaskGDKSafeWriteSession"), GDKContext, SessionName, GDKSession);
	}
}

#endif //WITH_GRDK
