// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "IPAddress.h"


class FOnlineSubsystemGDK;

/**
 * Async task to connect to users in a session after matchmaking and initialization/QoS is complete.
 */
class FOnlineAsyncTaskGDKGameSessionReady : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKGameSessionReady(
		FOnlineSubsystemGDK* InSubsystem,
		FGDKContextHandle InContext,
		FName InSessionName,
		const XblMultiplayerSessionReference* InGameSessionRef);

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual FString ToString() const override { return TEXT("GameSessionReady"); }
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

private:
	FGDKContextHandle LiveContext;
	FName SessionName;
	const XblMultiplayerSessionReference* GameSessionRef;
	FGDKMultiplayerSessionHandle GDKSession;
	FGDKContextHandle GDKContext;

	TSharedPtr<FInternetAddr> HostAddr;
	bool bWaitAndTryAgain = false;
};

//------------------------------- End of file ---------------------------------
