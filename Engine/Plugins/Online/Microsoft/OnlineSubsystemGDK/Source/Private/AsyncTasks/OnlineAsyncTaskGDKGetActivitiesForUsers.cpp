// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetActivitiesForUsers.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKGetActivitiesForUsers::FOnlineAsyncTaskGDKGetActivitiesForUsers(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const TArray<uint64>& InUserArray,
	const FOnGetActivitiesForUsersCompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKGetActivitiesForUsers"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, UserArray(InUserArray)
{
}

void FOnlineAsyncTaskGDKGetActivitiesForUsers::Initialize()
{
	// Need to find the user's current session to include in the invite call.
	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);

	HRESULT Result = XblMultiplayerGetActivitiesForUsersAsync(GDKContext,
		Scid,
		UserArray.GetData(),
		UserArray.Num(),
		*AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting activities for users, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetActivitiesForUsers::ProcessResults()
{
	uint64 NumActivityResults = 0;
	HRESULT Result = XblMultiplayerGetActivitiesForUsersResultCount(*AsyncBlock, &NumActivityResults);
	if (Result == S_OK)
	{
		//TArray<XblMultiplayerActivityDetails> ActivityDetails;
		ActivityDetails.Reserve(NumActivityResults);
		Result = XblMultiplayerGetActivitiesForUsersResult(*AsyncBlock, NumActivityResults, ActivityDetails.GetData());
		if (Result == S_OK)
		{
			ActivityDetails.SetNum(NumActivityResults);
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("Error getting activities for users, error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("Error getting number of activities for users, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetActivitiesForUsers::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetActivitiesForUsers_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(ActivityDetails, bWasSuccessful);
}

#endif //WITH_GRDK