// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKGetActivitiesForUsers
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKGetActivitiesForUsers(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64>& InUserArray,
		const FOnGetActivitiesForUsersCompleteDelegate& InTaskCompletionDelegate
);
	
	virtual ~FOnlineAsyncTaskGDKGetActivitiesForUsers() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKGetActivitiesForUsers"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnGetActivitiesForUsersCompleteDelegate TaskCompletionDelegate;
	const FString AchievementIdString;
	const TArray<uint64> UserArray;
	TArray<XblMultiplayerActivityDetails> ActivityDetails;
};
