// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetLeaderboard.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineLeaderboardInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

FOnlineAsyncTaskGDKGetLeaderboard::FOnlineAsyncTaskGDKGetLeaderboard(
	FOnlineSubsystemGDK* InGDKSubsystem,
	//XblLeaderboardResult* InResults,
	const FGDKContextHandle InGDKContext,
	const FString& InLeaderboardName,
	const uint64 InPlayerId,
	XblSocialGroupType InSocialGroupType,
	const FOnlineLeaderboardReadRef& InReadObject,
	bool InSkipToUser,
	FString InStatName,
	bool InFireDelegates,
	const FOnGetLeaderboardCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKGetLeaderboard")),
	GDKContext(InGDKContext),
	//Results(InResults),
	ReadObject(InReadObject),
	LeaderboardName(InLeaderboardName),
	PlayerId(InPlayerId),
	SocialGroupType(InSocialGroupType),
	bFireDelegates(InFireDelegates),
	bSkipToUser(InSkipToUser),
	StatName(InStatName),
	Delegate(InDelegate)
{
}

void FOnlineAsyncTaskGDKGetLeaderboard::Initialize()
{
	FTCHARToUTF8 StatNameUTF8(*StatName);
	FTCHARToUTF8 LeaderboardNameUTF8(*LeaderboardName);

	XblLeaderboardQuery Query = {0};
	Query.xboxUserId = PlayerId;
	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);
	FMemory::Memcpy(Query.scid, Scid, XBL_SCID_LENGTH);
	Query.leaderboardName = LeaderboardNameUTF8.Get();
	Query.statName = StatName.IsEmpty() ? nullptr : StatNameUTF8.Get();
	Query.socialGroup = SocialGroupType;

	TArray<FTCHARToUTF8> ConverterArr;
	TArray<const ANSICHAR*> CStrPointerArr;
	ConverterArr.Reserve(ReadObject->ColumnMetadata.Num());
	CStrPointerArr.Reserve(ReadObject->ColumnMetadata.Num());
	for (const FColumnMetaData& Column : ReadObject->ColumnMetadata)
	{
		// The query will return a column with the sorted Stat already, so it shouldn't be included in the additional columns
		if (Column.ColumnName != Query.statName)
		{
			// Converter Array acts as buffer to make sure every different string gets copied
			FTCHARToUTF8& Converter = ConverterArr.Emplace_GetRef(*Column.ColumnName);
			CStrPointerArr.Emplace(Converter.Get());
		}
	}

	Query.additionalColumnleaderboardNames = CStrPointerArr.GetData();
	Query.additionalColumnleaderboardNamesCount = CStrPointerArr.Num();

	Query.order = XblLeaderboardSortOrder::Descending;
	Query.maxItems = 0;
	if (bSkipToUser)
	{
		Query.skipToXboxUserId = PlayerId;
	}
	else
	{
		Query.skipToXboxUserId = 0;
	}
	Query.skipResultToRank = 0;
	Query.continuationToken = nullptr;

	HRESULT Result = XblLeaderboardGetLeaderboardAsync(GDKContext, Query, *AsyncBlock);
	
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to create leaderboard request operation"));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
		//QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetLeaderboard_Initialize_OnLeaderboardReadComplete);
		//TriggerOnLeaderboardReadCompleteDelegates(false);
	}
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;
}

void FOnlineAsyncTaskGDKGetLeaderboard::ProcessResults()
{
	uint64 ResultSize = 0;
	HRESULT Result = XblLeaderboardGetLeaderboardResultSize(*AsyncBlock, &ResultSize);
	if (Result == S_OK)
	{
		BufferArray.Reserve(ResultSize);
		Result = XblLeaderboardGetLeaderboardResult(*AsyncBlock, ResultSize, BufferArray.GetData(), &LeaderboardResult, nullptr);

		if (Result == S_OK)
		{
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Leaderboard serivce request failed. Error: 0x%0.8X."), Result);
			ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Leaderboard serivce request failed. Error: 0x%0.8X."), Result);
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetLeaderboard::Finalize()
{
	//Copy row data from the LeaderboardResult to the FOnlineLeaderboardRead
	if (bWasSuccessful)
	{
		for (uint32 i = 0; i < LeaderboardResult->rowsCount; ++i)
		{
			const XblLeaderboardRow& GDKRow = LeaderboardResult->rows[i];

			FOnlineStatsRow NewStatsRow = FOnlineStatsRow(GDKRow.gamertag, FUniqueNetIdGDK::Create(GDKRow.xboxUserId));
			NewStatsRow.Rank = GDKRow.rank;

			//Copy each column name and value into the new stats row
			for (int32 j = 0; j < LeaderboardResult->columnsCount; ++j)
			{
				const XblLeaderboardColumn& GDKColumn = LeaderboardResult->columns[j];

				FVariantData StatValue = ConvertLeaderboardRowDataToRequestedType(UTF8_TO_TCHAR(GDKRow.columnValues[j]), GDKColumn.statType, ReadObject->ColumnMetadata[j].DataType);

				UE_LOG_ONLINE_LEADERBOARD(Log, TEXT("Stat %s for user %s has value %s"), UTF8_TO_TCHAR(GDKColumn.statName), UTF8_TO_TCHAR(GDKRow.gamertag), *StatValue.ToString());

				NewStatsRow.Columns.Add(FString(LeaderboardResult->columns[j].statName), StatValue);
			}

			ReadObject->Rows.Add(NewStatsRow);
		}

		ReadObject->ReadState = EOnlineAsyncTaskState::Done;
	}
	else
	{
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
	}
}

void FOnlineAsyncTaskGDKGetLeaderboard::TriggerDelegates() 
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetLeaderboard_TriggerDelegates);
	Delegate.ExecuteIfBound(bWasSuccessful);
}

FVariantData FOnlineAsyncTaskGDKGetLeaderboard::ConvertLeaderboardRowDataToRequestedType(const TCHAR* RowData, XblLeaderboardStatType FromType, EOnlineKeyValuePairDataType::Type ToType)
{
	FString DataString(RowData);

	switch (ToType)
	{
	case EOnlineKeyValuePairDataType::Int32:
		{
			int32 Value = FCString::Strtoi(*DataString, NULL, 10);
			return FVariantData(Value);
		}
	case EOnlineKeyValuePairDataType::Int64:
		{
			int64 Value = FCString::Strtoi64(*DataString, NULL, 10);
			return FVariantData(Value);
		}
	case EOnlineKeyValuePairDataType::UInt64:
		{
			uint64 Value = FCString::Strtoui64(*DataString, NULL, 10);
			return FVariantData(Value);
		}
	case EOnlineKeyValuePairDataType::Float:
		{
			UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("GDK leaderboard data only supports Double floats, processing as Double."));
		}
	case EOnlineKeyValuePairDataType::Double:
		{
			if(FromType != XblLeaderboardStatType::Double)
			{
				ReportTypeMismatchWarning(FromType, ToType);
			}
			double Value = FCString::Atod(*DataString);
			return FVariantData(Value);
		}
	case EOnlineKeyValuePairDataType::String:
		{
			return FVariantData(DataString);
		}
	case EOnlineKeyValuePairDataType::Bool:
		{
			if(FromType != XblLeaderboardStatType::Boolean)
			{
				ReportTypeMismatchWarning(FromType, ToType);
			}
			bool Value = FCString::ToBool(*DataString);
			return FVariantData(Value);
		}
	default:
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Unsupported leaderboard result conversion."));
		return FVariantData();
	}
}

void FOnlineAsyncTaskGDKGetLeaderboard::ReportTypeMismatchWarning(XblLeaderboardStatType FromType, EOnlineKeyValuePairDataType::Type ToType)
{
	FString FromTypeString;
	FString ToTypeString;
	//These are the currently supported PropertyTypes for data in a leaderboard.
	switch (FromType)
	{
	case XblLeaderboardStatType::Uint64:
		FromTypeString = "uint64";
		break;
	case XblLeaderboardStatType::Double:
		FromTypeString = "double";
		break;
	case XblLeaderboardStatType::String:
		FromTypeString = "string";
		break;
	case XblLeaderboardStatType::Boolean:
		FromTypeString = "bool";
		break;
	case XblLeaderboardStatType::Other:
		FromTypeString = "other";
		break;
	default:
		FromTypeString = "Unknown";
	}
	
	switch (ToType)
	{
	case EOnlineKeyValuePairDataType::Int32:
		ToTypeString = "int32";
		break;
	case EOnlineKeyValuePairDataType::Int64:
		ToTypeString = "int64";
		break;
	case EOnlineKeyValuePairDataType::Double:
		ToTypeString = "double";
		break;
	case EOnlineKeyValuePairDataType::String:
		ToTypeString = "string";
		break;
	case EOnlineKeyValuePairDataType::Float:
		ToTypeString = "float";
		break;
	case EOnlineKeyValuePairDataType::Bool:
		ToTypeString = "bool";
		break;
	default:
		ToTypeString = "Unknown";
	}
	
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Converting leaderboard result data from '%s' to '%s'"), *FromTypeString, *ToTypeString);
}

#endif //WITH_GRDK