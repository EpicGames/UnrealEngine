// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStats.h"
#include "OnlineLeaderboardInterfaceGDK.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/leaderboard_c.h>
THIRD_PARTY_INCLUDES_END

/**
 *	Async task to retrieve a single user's leaderboard from Live
 */
class FOnlineAsyncTaskGDKGetLeaderboard : public FOnlineAsyncTaskGDK
{
public:
	/**
	 * Constructor
	 *
	 * @param InGDKSubsystem - The GDKOnlineSubsystem used to retrieve the leaderboards interface
	 * @param InResults - The leaderboard data returned from Live
	 * @param InReadObject - The storage location for the returned leaderboard data
	 * @param InFireDelegates - Whether or not to fire the OnLeaderboardReadComplete delegate
	 */
	FOnlineAsyncTaskGDKGetLeaderboard(
		FOnlineSubsystemGDK* InGDKSubsystem,
		//XblLeaderboardResult* InResults,
		const FGDKContextHandle InGDKContext,
		const FString& InLeaderboardName,
		uint64 InPlayerId,
		XblSocialGroupType InSocialGroupType,
		const FOnlineLeaderboardReadRef& InReadObject,
		bool InSkipToUser,
		FString InStatName,
		bool InFireDelegates,
		const FOnGetLeaderboardCompleteDelegate& InDelegate);

	/**
	 *	Get a human readable description of task
	 */
	void Initialize();
	virtual void ProcessResults() override;
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKGetLeaderboard");}
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

private:
	
	/** Pointer to the leaderboard returned by the async request */
	XblLeaderboardResult* Results;

	const FGDKContextHandle GDKContext;

	/** Handle to the read object where the data will be stored */
	FOnlineLeaderboardReadPtr ReadObject;
	
	const FString LeaderboardName;
	uint64 PlayerId;
	XblSocialGroupType SocialGroupType;

	/** Whether or not to fire the OnLeaderboardReadCompleteDelegates on completion */
	bool bFireDelegates;

	bool bSkipToUser;
	FString StatName;

	/* Holds raw result info from leaderboard query */
	TArray<uint8> BufferArray;
	/* Points to BufferArray's memory after successfully completing leaderboard query */
	XblLeaderboardResult* LeaderboardResult = nullptr;

	FOnGetLeaderboardCompleteDelegate Delegate;
	/**
	* Converts the LeaderboardResult string data to the type requested in the LeaderboardRead metadata
	*
	* @param RowData - The string containing the value to be parsed
	* @param FromType - The value type of the column in the leaderboard result. This is compared to the ToType to verify the conversion is appropriate.
	* @param ToType - The type of data to parse from the string and return
	* @return An FVariantData containing the requested datatype parsed from the RowData string
	*/
	FVariantData ConvertLeaderboardRowDataToRequestedType(const TCHAR* RowData, XblLeaderboardStatType FromType, EOnlineKeyValuePairDataType::Type ToType);

	/**
	* Reports a warning that the FromType and the ToType are mismatched and the conversion from one to the other may not be accurate
	*
	* @param FromType - The GDK Leaderboard PropertyType that is being converted from
	* @param ToType - The EOnlineKeyValuePairDataType that is being converted to
	*/
	void ReportTypeMismatchWarning(XblLeaderboardStatType FromType, EOnlineKeyValuePairDataType::Type ToType);
};
