// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetLeaderboardForUsers.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineLeaderboardInterfaceGDK.h"

FOnlineAsyncTaskGDKGetLeaderboardForUsers::FOnlineAsyncTaskGDKGetLeaderboardForUsers(
	FOnlineSubsystemGDK* InGDKSubsystem,
	TArray<FOnlineLeaderboardReadRef> InLeaderboardReads,
	const FOnlineLeaderboardReadRef& InReadObject)
	: FOnlineAsyncTaskBasic(InGDKSubsystem),
	LeaderboardReads(InLeaderboardReads),
	ReadObject(InReadObject)
{
}

void FOnlineAsyncTaskGDKGetLeaderboardForUsers::Tick()
{
	//wait for all LeaderboardReads to be populated with data
	bool bComplete = true;
	for(int i = 0; i < LeaderboardReads.Num(); i++)
	{
		if(LeaderboardReads[i]->ReadState == EOnlineAsyncTaskState::NotStarted || LeaderboardReads[i]->ReadState == EOnlineAsyncTaskState::InProgress)
		{
			bComplete = false;
			break;
		}
	}

	//Once all requests are completed, if all LeaderboardReads were successful then this async task was successful
	if(bComplete)
	{
		bool bSuccessful = true;
		for(int i = 0; i < LeaderboardReads.Num(); i++)
		{
			if(LeaderboardReads[i]->ReadState != EOnlineAsyncTaskState::Done)
			{
				bSuccessful = false;
				break;
			}
		}
		bWasSuccessful = bSuccessful;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetLeaderboardForUsers::Finalize()
{
	FOnlineAsyncTaskBasic::Finalize();
	if (bWasSuccessful && LeaderboardReads.Num() > 0)
	{		
		//For each leaderboard result returned by an async request
		for(int ResultIndex = 0; ResultIndex < LeaderboardReads.Num(); ResultIndex++)
		{
			//take every row in the result and copy it into the single FOnlineLeaderboardRead passed into this request
			for(int ResultRow = 0; ResultRow < LeaderboardReads[ResultIndex]->Rows.Num(); ResultRow++)
			{
				//add the leaderboard row into the single FOnlineLeaderboardRead in sorted order by rank
				int InsertIndex = 0;
				for(; InsertIndex < ReadObject->Rows.Num(); InsertIndex++)
				{
					if(ReadObject->Rows[InsertIndex].Rank >= LeaderboardReads[ResultIndex]->Rows[ResultRow].Rank)
					{
						break;
					}
				}
				ReadObject->Rows.Insert(LeaderboardReads[ResultIndex]->Rows[ResultRow], InsertIndex);
			}
		}
	}

	ReadObject->ReadState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
}

void FOnlineAsyncTaskGDKGetLeaderboardForUsers::TriggerDelegates() 
{
	FOnlineAsyncTaskBasic::TriggerDelegates();
	FOnlineLeaderboardsGDKPtr Leaderboards = Subsystem->GetLeaderboardsInterfaceGDK();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetLeaderboardForUsers_TriggerDelegates);
	Leaderboards->TriggerOnLeaderboardReadCompleteDelegates(bWasSuccessful);
}

#endif //WITH_GRDK