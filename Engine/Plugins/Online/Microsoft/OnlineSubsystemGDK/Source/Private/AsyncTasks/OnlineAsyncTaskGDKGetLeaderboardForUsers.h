// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "OnlineAsyncTaskGDKGetLeaderboard.h"
#include "OnlineStats.h"

/**
 *	Async task to retrieve a leaderboard containing a list of users from Live
 */
class FOnlineAsyncTaskGDKGetLeaderboardForUsers : public FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>
{
public:
	/**
	 * Constructor
	 *
	 * @param InGDKSubsystem - The GDKOnlineSubsystem used to retrieve the leaderboards interface
	 * @param InLeaderboardReads - The array of LeaderboardReads to consolidate into the ReadObject
	 * @param InReadObject - The storage location for the returned leaderboard data
	 */
	FOnlineAsyncTaskGDKGetLeaderboardForUsers(
			FOnlineSubsystemGDK* InGDKSubsystem,
			TArray<FOnlineLeaderboardReadRef> InLeaderboardReads,
			const FOnlineLeaderboardReadRef& InReadObject);

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const { return FString::Printf(TEXT("FOnlineAsyncTaskGDKGetLeaderboardForUsers bWasSuccessful: %d"), WasSuccessful());}
	virtual void Tick();
	virtual void Finalize();
	virtual void TriggerDelegates();

private:
	/** List of ReadObjects to be populated by single user leaderboard requests */
	TArray<FOnlineLeaderboardReadRef> LeaderboardReads;
	/** Handle to the read object where the data will be stored */
	FOnlineLeaderboardReadPtr ReadObject;
};
