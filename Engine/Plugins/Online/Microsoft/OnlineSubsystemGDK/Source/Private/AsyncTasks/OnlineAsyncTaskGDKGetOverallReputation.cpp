// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetOverallReputation.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKGetOverallReputation::FOnlineAsyncTaskGDKGetOverallReputation(
	FOnlineSubsystemGDK* const InGDKSubsystem
	, FGDKContextHandle InGDKContext
	, TArray<FUniqueNetIdGDKRef >&& InUserIDs
	, const FOnGetOverallReputationCompleteDelegate& InCompletionDelegate
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKGetOverallReputation"))
	, GDKContext(InGDKContext)
	, CompletionDelegate(InCompletionDelegate)
	, UserIds(MoveTemp(InUserIDs))
{

}

FOnlineAsyncTaskGDKGetOverallReputation::FOnlineAsyncTaskGDKGetOverallReputation(
	FOnlineSubsystemGDK* const InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	const FUniqueNetIdGDKRef& InUserID,
	const FOnGetOverallReputationCompleteDelegate& InCompletionDelegate
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKGetOverallReputation"))
	, GDKContext(InGDKContext)
	, CompletionDelegate(InCompletionDelegate)
{
	UserIds.Add(InUserID);
}

FString FOnlineAsyncTaskGDKGetOverallReputation::ToString() const
{
	return TEXT("GetOverallReputation task");
}

void FOnlineAsyncTaskGDKGetOverallReputation::Initialize()
{
	TArray<uint64> UserIDsToSend;

	for (const FUniqueNetIdGDKRef& UserId : UserIds)
	{
		UserIDsToSend.Add(UserId->ToUint64());
	}

	TArray<const char*> StatisticsToSearchFor;
	StatisticsToSearchFor.Add("OverallReputationIsBad");

	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsAsync(
		GDKContext,
		UserIDsToSend.GetData(),
		UserIDsToSend.Num(),
		TCHAR_TO_UTF8(GDK_GLOBAL_SCID),
		const_cast<const char**>(StatisticsToSearchFor.GetData()),
		StatisticsToSearchFor.Num(),
		*AsyncBlock);

	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskLiveGetOverallReputation: GetMultipleUserStatisticsAsync: Failed to get statistics. HResult = 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetOverallReputation::ProcessResults()
{
	uint64 resultSize = 0;
	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsResultSize(*AsyncBlock, &resultSize);
	if(Result == S_OK)
	{
		TArray<char> CharBuffer;
		CharBuffer.Reserve(static_cast<int32>(resultSize));

		XblUserStatisticsResult* StatisticsResult = nullptr;
		uint64 NumberOfResults = 0;
		Result = XblUserStatisticsGetMultipleUserStatisticsResult(
			*AsyncBlock,
			resultSize,
			CharBuffer.GetData(),
			&StatisticsResult,
			&NumberOfResults,
			nullptr);

		if (Result == S_OK)
		{
			for (uint64 UserStatIndex = 0; UserStatIndex < NumberOfResults; ++UserStatIndex)
			{
				XblUserStatisticsResult& Stats = StatisticsResult[UserStatIndex];
				XblServiceConfigurationStatistic* ServiceStats = Stats.serviceConfigStatistics;

				UE_LOG_ONLINE(Verbose, TEXT("Reputation: Found %d statistics for user %" UINT64_FMT), Stats.serviceConfigStatisticsCount, Stats.xboxUserId);

				bool bOverallReputationIsBad = false;

				for (uint32 ConfigStatIndex = 0; ConfigStatIndex < Stats.serviceConfigStatisticsCount; ++ConfigStatIndex)
				{
					XblServiceConfigurationStatistic& ConfigStat = ServiceStats[ConfigStatIndex];

					//UE_LOG_ONLINE(Verbose, TEXT("Reputation: stat %d: %s"), ConfigStatIndex, ConfigurationStatToString(ConfigStat)); /WMM TODO make a toString func
					UE_LOG_ONLINE(Verbose, TEXT("Reputation: Found %d config statistics in stat %d"), ConfigStat.statisticsCount, ConfigStatIndex);

					for (uint32 StatIndex = 0; StatIndex < ConfigStat.statisticsCount; ++StatIndex)
					{
						XblStatistic& Stat = ConfigStat.statistics[StatIndex];

						FString StatName(UTF8_TO_TCHAR(Stat.statisticName));
						if (StatName.Equals(TEXT("OverallReputationIsBad")))
						{
							// the Value returned by the query should be either "0" or "1" for false or true, respectively
							FString StatValue(UTF8_TO_TCHAR(Stat.value));
							bOverallReputationIsBad = StatValue.ToBool();
						}
					}
				}
				FUniqueNetIdGDKRef UserId = FUniqueNetIdGDK::Create(Stats.xboxUserId);
				UsersWithBadReputations.Add(UserId, bOverallReputationIsBad);
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("GetMultipleUserStatisticsAsync failed with 0x%0.8X"), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("GetMultipleUserStatisticsAsyncSize failed with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetOverallReputation::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetOverallReputation_TriggerDelegates);
	CompletionDelegate.ExecuteIfBound(UsersWithBadReputations);
}

#endif //WITH_GRDK