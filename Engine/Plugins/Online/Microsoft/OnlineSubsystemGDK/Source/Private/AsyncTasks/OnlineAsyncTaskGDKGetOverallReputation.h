// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/user_statistics_c.h>
THIRD_PARTY_INCLUDES_END

typedef TUniqueNetIdMap<bool> UserReputationMap;
DECLARE_DELEGATE_OneParam(FOnGetOverallReputationCompleteDelegate, const UserReputationMap& /* UsersWithBadReputations */);

class FOnlineAsyncTaskGDKGetOverallReputation
	: public FOnlineAsyncTaskGDK
{
public:

	FOnlineAsyncTaskGDKGetOverallReputation(FOnlineSubsystemGDK* const InGDKSubsystem
		, FGDKContextHandle InGDKContext
		, TArray<FUniqueNetIdGDKRef>&& InUserIDs
		, const FOnGetOverallReputationCompleteDelegate& InCompletionDelegate
	);

	FOnlineAsyncTaskGDKGetOverallReputation(FOnlineSubsystemGDK* const InGDKSubsystem
		, FGDKContextHandle InLiveContext
		, const FUniqueNetIdGDKRef& InUserID
		, const FOnGetOverallReputationCompleteDelegate& InCompletionDelegate
	);

	virtual FString ToString() const override;

private:
	/**
	* Handles creating an AsyncOperation.  The task should be created and returned in this method
	*
	* @Return The AsyncOperation to run, or nullptr if there was a failure
	*/
	virtual void Initialize() override;

	/**
	* Handles processing of an async result.  Any processed data should be stored on the task until Finalize is called on the Game thread.
	*
	* @Param CompletedTask The finished task to call get() on
	* @Return Whether or not our task was successful and is stored in bWasSuccessful
	*/
	virtual void ProcessResults() override;

	virtual void TriggerDelegates() override;

private:
	FGDKContextHandle GDKContext;
	FOnGetOverallReputationCompleteDelegate CompletionDelegate;

	TArray<FUniqueNetIdGDKRef > UserIds;
	UserReputationMap UsersWithBadReputations;
};
