// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetUserCollectionsId.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"

FOnlineAsyncTaskGetUserCollectionsId::FOnlineAsyncTaskGetUserCollectionsId(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const FString& InServiceTicket, const FString& InPublisherUserId, const FOnCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGetUserCollectionsId"))
	, ServiceTicket(InServiceTicket)
	, PublisherUserId(InPublisherUserId)
	, Delegate(InDelegate)
	, GDKContext(InGDKContext)
{
	check(Subsystem);
}

FOnlineAsyncTaskGetUserCollectionsId::~FOnlineAsyncTaskGetUserCollectionsId()
{
}

void FOnlineAsyncTaskGetUserCollectionsId::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	FGDKUserHandle GDKUserHandle;
	HRESULT Result = XblContextGetUser(GDKContext, GDKUserHandle.GetInitReference());
	if (Result == S_OK)
	{
		XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUserHandle);
		if (StoreContextHandle != nullptr)
		{
			Result = XStoreGetUserCollectionsIdAsync(StoreContextHandle, TCHAR_TO_UTF8(*ServiceTicket), TCHAR_TO_UTF8(*PublisherUserId), *AsyncBlock);
			if (Result != S_OK)
			{
				ErrorResponse.bSucceeded = false;
				ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user collections id, error: (0x%0.8X)."), Result)));
				bWasSuccessful = false;
				bIsComplete = true;
			}
		}
		else
		{
			ErrorResponse.bSucceeded = false;
			ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user collections id (store context), error: (0x%0.8X)."), Result)));
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		ErrorResponse.bSucceeded = false;
		ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user collections id (user handle), error: (0x%0.8X)."), Result)));
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGetUserCollectionsId::ProcessResults()
{
	size_t Size;
	HRESULT Result = XStoreGetUserCollectionsIdResultSize(*AsyncBlock, &Size);
	if (Result == S_OK)
	{
		TArray<char> CollectionsIdUtf8;
		CollectionsIdUtf8.SetNum(Size);
		Result = XStoreGetUserCollectionsIdResult(*AsyncBlock, Size, CollectionsIdUtf8.GetData());
		if (Result == S_OK)
		{
			ErrorResponse.bSucceeded = true;
			FUTF8ToTCHAR Converter(CollectionsIdUtf8.GetData(), CollectionsIdUtf8.Num());
			CollectionsId = FString::ConstructFromPtrSize(Converter.Get(), Converter.Length());
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			FString ErrorMessage = FString::Printf(TEXT("Error getting user collections id (result), error: (0x%0.8X)."), Result);
			ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
			ErrorResponse.bSucceeded = false;
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		FString ErrorMessage = FString::Printf(TEXT("Error getting user collections id (result size), error: (0x%0.8X)."), Result);
		ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
		ErrorResponse.bSucceeded = false;
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGetUserCollectionsId::Finalize()
{
}

void FOnlineAsyncTaskGetUserCollectionsId::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGetUserCollectionsId_TriggerDelegates);
	Delegate.ExecuteIfBound(CollectionsId, ErrorResponse);
}

#undef DEBUG_LOG_GDK_STORE_ITEMS

#endif //WITH_GRDK