// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetUserPrivilege.h"
#include "OnlineAsyncTaskGDKCheckForPackageUpdate.h"
#include "OnlineAsyncTaskGDKResolvePrivilegeWithUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "Misc/CommandLine.h"

typedef IOnlineIdentity::EPrivilegeResults EPrivilegeResults;

TAutoConsoleVariable<bool> CVarXboxForceMandatoryUpdateAvailable(
	TEXT("xb.ForceMandatoryUpdateAvailable"),
	false,
	TEXT("Force a MandatoryUpdateAvailable result when checking for package update")
);

FOnlineAsyncTaskGDKGetUserPrivilege::FOnlineAsyncTaskGDKGetUserPrivilege(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const FUniqueNetIdGDKRef& InUserId, 
	EUserPrivileges::Type InPrivilege, 
	const IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate& InTaskCompletionDelegate,
	EShowPrivilegeResolveUI InShowResolveUI
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKGetUserPrivilege"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKUserId(InUserId)
	, Privilege(InPrivilege)
	, ShowResolveUI(InShowResolveUI)
{
}

void FOnlineAsyncTaskGDKGetUserPrivilege::Initialize()
{
	const FOnlineIdentityGDKPtr IdentityInterface = Subsystem->GetIdentityGDK();
	check(IdentityInterface.IsValid());

	if(IdentityInterface->GetPrivilegeCheckDelay() > 0)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[this](float) {InitializeInternal(); return false; }), IdentityInterface->GetPrivilegeCheckDelay());
		return;
	}
	InitializeInternal();	

}
void FOnlineAsyncTaskGDKGetUserPrivilege::InitializeInternal()
{
	const FOnlineIdentityGDKPtr IdentityInterface = Subsystem->GetIdentityGDK();
	check(IdentityInterface.IsValid());

	GDKUser = IdentityInterface->GetUserForUniqueNetId(*GDKUserId);

	if (!GDKUser.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineAsyncTaskGDKGetUserPrivilege::Initialize couldn't find GDK user for unique id %s."), *GDKUserId->ToString());
		
		PrivilegeResult = EPrivilegeResults::UserNotFound;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	XUserState UserState = XUserState::SignedOut;
	XUserGetState(GDKUser, &UserState);

	// Docs warn to not call CheckPrivilegeAsync if the user isn't signed in.
	if (UserState != XUserState::SignedIn)
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineIdentityGDK::GetUserPrivilege GDK user %s is not signed in."), *IdentityInterface->GetPlayerNickname(GDKUser));
		PrivilegeResult = EPrivilegeResults::UserNotLoggedIn;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	bool bShowResolveUI = (ShowResolveUI == EShowPrivilegeResolveUI::Show);

	KnownPrivilege = XUserPrivilege::Multiplayer;
	switch (Privilege)
	{
		case EUserPrivileges::CanPlayOnline:
		{
			KnownPrivilege = XUserPrivilege::Multiplayer;
			bShowResolveUI |= (ShowResolveUI == EShowPrivilegeResolveUI::Default);
			break;
		}

		case EUserPrivileges::CanCommunicateOnline:
		{
			KnownPrivilege = XUserPrivilege::Communications;
			break;
		}

		case EUserPrivileges::CanUseUserGeneratedContent:
		{
			KnownPrivilege = XUserPrivilege::UserGeneratedContent;
			break;
		}

		case EUserPrivileges::CanUserCrossPlay:
		{
			KnownPrivilege = XUserPrivilege::CrossPlay;
			break;
		}

		default:
		{
			// @todo: Add other privilege types
			PrivilegeResult = EPrivilegeResults::NoFailures;
			bWasSuccessful = false;
			bIsComplete = true;
			return;
		}
	}

	bHasPrivilege = false;
	DenyReason = XUserPrivilegeDenyReason::None;
	HRESULT Result = XUserCheckPrivilege(GDKUser, XUserPrivilegeOptions::None, KnownPrivilege, &bHasPrivilege, &DenyReason);

	if (!bHasPrivilege && bShowResolveUI)
	{
		auto CompletionDelegate = FOnQueryGDKResolvePrivilegeWithUICompleteDelegate::CreateRaw(this, &FOnlineAsyncTaskGDKGetUserPrivilege::ResolvePrivilegeWithUIComplete);
		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKResolvePrivilegeWithUI>(Subsystem, GDKContext, GDKUser, XUserPrivilegeOptions::None, KnownPrivilege, CompletionDelegate);
		return;
	}
	else if (Result != S_OK)
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineIdentityGDK::GetUserPrivilege failed code 0x%0.8X."), Result);
		PrivilegeResult = EPrivilegeResults::GenericFailure;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
	CheckForGDKPackageUpdate();
}

void FOnlineAsyncTaskGDKGetUserPrivilege::ResolvePrivilegeWithUIComplete(bool bSuccess)
{
	bHasPrivilege = bSuccess;

	if (bHasPrivilege)
	{
		CheckForGDKPackageUpdate();
	}
	else
	{
		PrivilegeResult = EPrivilegeResults::GenericFailure;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
}

void FOnlineAsyncTaskGDKGetUserPrivilege::CheckForGDKPackageUpdate()
{
	// If console variable xb.ForceMandatoryUpdateAvailable is set we force return that a mandatory update is available
	if (CVarXboxForceMandatoryUpdateAvailable.GetValueOnAnyThread())
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("Forcing mandatory update available"));
		PatchCheckCompletion(FOnlineError::Success(), ECheckForPackageUpdateResult::MandatoryUpdateAvailable);
	}
	// The patch check is unreliable in non store builds at present. This is to unblock QA until a propper solution is found.
	else if (FParse::Param(FCommandLine::Get(), TEXT("NoUpdateCheckGDK")) || Subsystem->GetOnlineEnvironment() == EOnlineEnvironment::Development)
	{
		// Skip patch check if in development (will always fail if non-packaged build)
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("Skipping patch check in development environment"));
		PatchCheckCompletion(FOnlineError::Success(), ECheckForPackageUpdateResult::NoUpdateAvailable);
	}
	else
	{
		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKCheckForPackageUpdate>(Subsystem, GDKUser,
			FOnCheckForPackageUpdateCompleteDelegate::CreateRaw(this, &FOnlineAsyncTaskGDKGetUserPrivilege::PatchCheckCompletion));
	}
}

void FOnlineAsyncTaskGDKGetUserPrivilege::PatchCheckCompletion(const FOnlineError& ErrorResult, const TOptional<ECheckForPackageUpdateResult> OptionalPatchCheckResult)
{
	if (!ErrorResult.bSucceeded)
	{
		// Ensure our patch check succeeds in non-development environments.  In development, unpackaged builds won't have a package
		PrivilegeResult = EPrivilegeResults::GenericFailure;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	if (OptionalPatchCheckResult.IsSet())
	{
		const ECheckForPackageUpdateResult PatchCheckResult = OptionalPatchCheckResult.GetValue();
		if (PatchCheckResult == ECheckForPackageUpdateResult::MandatoryUpdateAvailable)
		{
			PrivilegeResult = EPrivilegeResults::RequiredPatchAvailable;
			bWasSuccessful = false;
			bIsComplete = true;
			return;
		}
	}

	const uint32 ResultInt = static_cast<uint32>(DenyReason);
	UE_LOG_ONLINE_IDENTITY(Log, TEXT("CheckPrivilegeAsync User=[%s] Privilege=[%d] Result=[%d] AllowNonPremium=[%s]"),
		*GDKUserId->ToString(), static_cast<uint32>(Privilege), ResultInt, *LexToString(Subsystem && !Subsystem->IsXBLGoldRequired()));

	// Note: DenyReason is a hint, and is not guaranteed to hold a useful value in all use-cases.
	// Determining if a player account is Banned/Restricted is currently dependent on it in the logic
	// below, but may not be the correct process - it's not clear from the XBL docs what is correct.

	if (bHasPrivilege)
	{
		PrivilegeResult = EPrivilegeResults::NoFailures;
		bWasSuccessful = true;
	}
	else
	{
		const EPrivilegeResults DefaultFailure = (XUserPrivilegeDenyReason::PurchaseRequired == DenyReason) ?
			EPrivilegeResults::AccountTypeFailure : EPrivilegeResults::GenericFailure;

		PrivilegeResult = DefaultFailure;

		const bool bBannedAccount = (XUserPrivilegeDenyReason::Banned == DenyReason)
			|| (XUserPrivilegeDenyReason::Restricted == DenyReason);

		// Special case if we want to allow non-Gold XBL accounts to play.
		if (!bBannedAccount && (XUserPrivilege::Multiplayer == KnownPrivilege) && ensure(Subsystem))
		{
			// If we allow non-premium accounts to play online multiplayer, override the result
			// of certain failures with success instead.  Otherwise, indicate that the account
			// type was insufficient.
			switch (Privilege)
			{
			case EUserPrivileges::CanPlayOnline:
			case EUserPrivileges::CanCommunicateOnline:
			case EUserPrivileges::CanUserCrossPlay:
				if (Subsystem->IsXBLGoldRequired())
				{
					PrivilegeResult = EPrivilegeResults::AccountTypeFailure;
				}
				else
				{
					PrivilegeResult = EPrivilegeResults::NoFailures;
					bWasSuccessful = true;
				}
				break;

			case EUserPrivileges::CanUseUserGeneratedContent:
				PrivilegeResult = Subsystem->IsXBLGoldRequired() ?
					EPrivilegeResults::AccountTypeFailure : EPrivilegeResults::UGCRestriction;
				break;

			default:
				PrivilegeResult = DefaultFailure;
				break;
			}; // switch
		}
		else
		{
			// All other failure cases, generate the appropriate result based on specific restriction.
			switch (Privilege)
			{
			case EUserPrivileges::CanPlayOnline:
				PrivilegeResult = EPrivilegeResults::OnlinePlayRestricted;
				break;

			case EUserPrivileges::CanCommunicateOnline:
				PrivilegeResult = EPrivilegeResults::ChatRestriction;
				break;

			case EUserPrivileges::CanUseUserGeneratedContent:
				PrivilegeResult = EPrivilegeResults::UGCRestriction;
				break;

			case EUserPrivileges::CanUserCrossPlay:
				PrivilegeResult = EPrivilegeResults::OnlinePlayRestricted;
				break;

			default:
				PrivilegeResult = DefaultFailure;
				break;
			}; // switch
		} // if-else banned account
	} // if-else has privilege

	bIsComplete = true;
}

void FOnlineAsyncTaskGDKGetUserPrivilege::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetUserPrivilege_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(*GDKUserId, Privilege, static_cast<uint32>(PrivilegeResult));
}

#endif //WITH_GRDK