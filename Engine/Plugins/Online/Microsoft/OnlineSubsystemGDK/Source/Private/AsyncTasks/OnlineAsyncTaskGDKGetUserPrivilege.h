// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKGetUserPrivilege
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKGetUserPrivilege(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FUniqueNetIdGDKRef& InUserId,
		EUserPrivileges::Type InPrivilege,
		const IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate& InTaskCompleteionDelegate,
		EShowPrivilegeResolveUI InShowResolveUI);
	
	virtual ~FOnlineAsyncTaskGDKGetUserPrivilege() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKGetUserPrivilege"); }

	// Starts in Game Thread
	virtual void Initialize() override;

	void InitializeInternal();


	void ResolvePrivilegeWithUIComplete(bool bWasSuccessful);
	void CheckForGDKPackageUpdate();
	void PatchCheckCompletion(const FOnlineError& ErrorResult, const TOptional<ECheckForPackageUpdateResult> OptionalPatchCheckResult);

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate TaskCompletionDelegate;
	FGDKUserHandle GDKUser;
	const FUniqueNetIdGDKRef GDKUserId;
	EUserPrivileges::Type Privilege;
	IOnlineIdentity::EPrivilegeResults PrivilegeResult = IOnlineIdentity::EPrivilegeResults::GenericFailure;
	bool bHasPrivilege = false;
	XUserPrivilegeDenyReason DenyReason = XUserPrivilegeDenyReason::None;
	XUserPrivilege KnownPrivilege = XUserPrivilege::Sessions;
	EShowPrivilegeResolveUI ShowResolveUI;
};
