// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetUserProfile.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKGetUserProfile::FOnlineAsyncTaskGDKGetUserProfile(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	uint64 InUserId, 
	const FOnGetUserProfileCompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKGetUserProfile"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKUserId(InUserId)
{
	FMemory::Memzero(&Profile, sizeof(Profile));
	
}

void FOnlineAsyncTaskGDKGetUserProfile::Initialize()
{
	HRESULT Result = XblProfileGetUserProfileAsync(GDKContext, GDKUserId, *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineAsyncTaskGDKGetUserProfile::Initialize unable to resolve with code 0x%0.8X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	
}

void FOnlineAsyncTaskGDKGetUserProfile::ProcessResults()
{
	HRESULT Result = XblProfileGetUserProfileResult(*AsyncBlock, &Profile);
			
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineAsyncTaskGDKGetUserProfile::ProcessResults unable to resolve with code 0x%0.8X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	else
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetUserProfile::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetUserProfile_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, Profile);
}

#endif //WITH_GRDK