// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to get user profile
 */
class FOnlineAsyncTaskGDKGetUserProfile
	: public FOnlineAsyncTaskGDK
{
public:
	/**
	* Delegate fired when GetUserProfile Task is complete.
	*/
	DECLARE_DELEGATE_TwoParams(FOnGetUserProfileCompleteDelegate, bool, const XblUserProfile&);

	FOnlineAsyncTaskGDKGetUserProfile(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		uint64 InUserId,
		const FOnGetUserProfileCompleteDelegate& InTaskCompletionDelegate);
	
	virtual ~FOnlineAsyncTaskGDKGetUserProfile() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKGetUserProfile"); }

	// Starts in Game Thread
	virtual void Initialize() override;

	virtual void ProcessResults() override;
	
	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnGetUserProfileCompleteDelegate TaskCompletionDelegate;
	uint64 GDKUserId;
	XblUserProfile Profile;
};
