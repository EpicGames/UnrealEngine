// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetUserPurchaseId.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"

FOnlineAsyncTaskGetUserPurchaseId::FOnlineAsyncTaskGetUserPurchaseId(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const FString& InServiceTicket, const FString& InPublisherUserId, const FOnCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGetUserPurchaseId"))
	, ServiceTicket(InServiceTicket)
	, PublisherUserId(InPublisherUserId)
	, Delegate(InDelegate)
	, GDKContext(InGDKContext)
{
	check(Subsystem);
}

FOnlineAsyncTaskGetUserPurchaseId::~FOnlineAsyncTaskGetUserPurchaseId()
{
}

void FOnlineAsyncTaskGetUserPurchaseId::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	FGDKUserHandle GDKUserHandle;
	HRESULT Result = XblContextGetUser(GDKContext, GDKUserHandle.GetInitReference());
	if (Result == S_OK)
	{
		XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUserHandle);
		if (StoreContextHandle != nullptr)
		{
			Result = XStoreGetUserPurchaseIdAsync(StoreContextHandle, TCHAR_TO_UTF8(*ServiceTicket), TCHAR_TO_UTF8(*PublisherUserId), *AsyncBlock);
			if (Result != S_OK)
			{
				ErrorResponse.bSucceeded = false;
				ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user purchase id, error: (0x%0.8X)."), Result)));
				bWasSuccessful = false;
				bIsComplete = true;
			}
		}
		else
		{
			ErrorResponse.bSucceeded = false;
			ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user purchase id (store context), error: (0x%0.8X)."), Result)));
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		ErrorResponse.bSucceeded = false;
		ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error getting user purchase id (user handle), error: (0x%0.8X)."), Result)));
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGetUserPurchaseId::ProcessResults()
{
	size_t Size;
	HRESULT Result = XStoreGetUserPurchaseIdResultSize(*AsyncBlock, &Size);
	if (Result == S_OK)
	{
		TArray<char> PurchaseIdUtf8;
		PurchaseIdUtf8.SetNum(Size);
		Result = XStoreGetUserPurchaseIdResult(*AsyncBlock, Size, PurchaseIdUtf8.GetData());
		if (Result == S_OK)
		{
			ErrorResponse.bSucceeded = true;
			FUTF8ToTCHAR Converter(PurchaseIdUtf8.GetData(), PurchaseIdUtf8.Num());
			PurchaseId = FString::ConstructFromPtrSize(Converter.Get(), Converter.Length());
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			FString ErrorMessage = FString::Printf(TEXT("Error getting user purchase id (result), error: (0x%0.8X)."), Result);
			ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
			ErrorResponse.bSucceeded = false;
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		FString ErrorMessage = FString::Printf(TEXT("Error getting user purchase id (result size), error: (0x%0.8X)."), Result);
		ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
		ErrorResponse.bSucceeded = false;
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGetUserPurchaseId::Finalize()
{
}

void FOnlineAsyncTaskGetUserPurchaseId::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGetUserPurchaseId_TriggerDelegates);
	Delegate.ExecuteIfBound(PurchaseId, ErrorResponse);
}

#endif //WITH_GRDK