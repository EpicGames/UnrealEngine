// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineStoreInterfaceGDK.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END

class FOnlineSubsystemGDK;

/**
 * Async item used to marshal user purchase id results from the GDK system thread to the game thread.
 */
class FOnlineAsyncTaskGetUserPurchaseId : public FOnlineAsyncTaskGDK
{
public:
	DECLARE_DELEGATE_TwoParams(FOnCompleteDelegate, const FString& /*StoreId*/, const FOnlineError& /*Result*/);

	FOnlineAsyncTaskGetUserPurchaseId(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const FString& InServiceTicket, const FString& InPublisherUserId, const FOnCompleteDelegate& InDelegate);
	virtual ~FOnlineAsyncTaskGetUserPurchaseId();

	virtual void Tick() override;
	virtual void ProcessResults() override;
	
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGetUserPurchaseId");}
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
protected:
	FString ServiceTicket;
	FString PublisherUserId;
	FString PurchaseId;
	FOnlineError ErrorResponse;
	FOnCompleteDelegate Delegate;
	FGDKContextHandle GDKContext;
	bool bTaskStarted = false;
};
