// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKGetXSTSToken.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKGetXSTSToken::FOnlineAsyncTaskGDKGetXSTSToken(
	FOnlineSubsystemGDK* const InGDKSubsystem,
	FGDKUserHandle InLocalUser,
	int32 InLocalUserNum,
	const FString& InEndPointURL,
	const FOnXSTSTokenCompleteDelegate& CompletionDelegate
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKGetXSTSToken"))
	, LocalUserNum(InLocalUserNum)
	, LocalUser(InLocalUser)
	, RequestEndPointURL(InEndPointURL)
	, TaskCompletionDelegate(CompletionDelegate)
{
	GDKUserId = FUniqueNetIdGDK::Create(InLocalUser);
}

void FOnlineAsyncTaskGDKGetXSTSToken::Initialize()
{

	HRESULT Result = XUserGetTokenAndSignatureUtf16Async(
		LocalUser,
		XUserGetTokenAndSignatureOptions::None,
		TEXT("GET"),
		*RequestEndPointURL,
		0,
		nullptr,
		0,
		nullptr,
		*AsyncBlock);

	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKGetXSTSToken::GetTokenAndSignatureAsync: Failed to get token. Result = 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKGetXSTSToken::ProcessResults()
{
	uint64 ResultSize = 0;
	HRESULT Result = XUserGetTokenAndSignatureUtf16ResultSize(*AsyncBlock, &ResultSize);
	if(Result == S_OK)
	{
		BufferArray.Reserve(ResultSize);
		XUserGetTokenAndSignatureUtf16Data* ResultData = nullptr;

		Result = XUserGetTokenAndSignatureUtf16Result(
			*AsyncBlock,
			ResultSize,
			BufferArray.GetData(),
			&ResultData,
			nullptr);
		
		if(Result == S_OK)
		{
	
			ResultToken = ResultData->token;
			ResultSignature = ResultData->signature;
		
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("GetTokenAndSignatureResult failed with 0x%0.8X"), Result);
			bWasSuccessful = false;
			bIsComplete = true;
			return;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("GetTokenAndSignatureResultSize failed with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
}

void FOnlineAsyncTaskGDKGetXSTSToken::Finalize()
{
	if (bWasSuccessful)
	{
		FOnlineIdentityGDKPtr GDKIdentity = Subsystem->GetIdentityGDK();
		GDKIdentity->SetUserXSTSToken(LocalUser, ResultToken);
	}
}

void FOnlineAsyncTaskGDKGetXSTSToken::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKGetXSTSToken_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(FOnlineError(bWasSuccessful), LocalUserNum, GDKUserId.ToSharedRef(), ResultSignature, ResultToken);
}

#endif //WITH_GRDK