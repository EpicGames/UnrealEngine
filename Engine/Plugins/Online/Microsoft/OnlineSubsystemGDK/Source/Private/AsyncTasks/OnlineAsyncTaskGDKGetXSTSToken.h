// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

/**
* Delegate for after receiving an XSTS auth token.
*/
DECLARE_DELEGATE_FiveParams(FOnXSTSTokenCompleteDelegate, const FOnlineError& /*Result*/, int32 /*LocalUserNum*/, const FUniqueNetIdGDKRef& /*UserId*/, const FString& /*ResultSignature*/, const FString& /*ResultToken*/);

class FOnlineAsyncTaskGDKGetXSTSToken
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKGetXSTSToken(
		class FOnlineSubsystemGDK* const InGDKSubsystem,
		FGDKUserHandle InLocalUser,
		int32 InLocalUserNum,
		const FString& InEndPointURL,
		const FOnXSTSTokenCompleteDelegate& CompletionDelegate);

	virtual ~FOnlineAsyncTaskGDKGetXSTSToken() = default;

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKGetXSTSToken"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

private:
	int32 LocalUserNum;
	FGDKUserHandle LocalUser;
	FString RequestEndPointURL;
	FOnXSTSTokenCompleteDelegate TaskCompletionDelegate;

	FUniqueNetIdGDKPtr GDKUserId;
	FString ResultToken;
	FString ResultSignature;
	TArray<uint8> BufferArray;
};
