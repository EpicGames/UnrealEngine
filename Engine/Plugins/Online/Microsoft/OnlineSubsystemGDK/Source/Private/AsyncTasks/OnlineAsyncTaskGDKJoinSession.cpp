// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKJoinSession.h"
#include "OnlineSubsystemGDKPrivate.h"
#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#endif //WITH_ENGINE
#include "Misc/Base64.h"
#include "OnlineAsyncTaskGDKRegisterLocalUser.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskGDKRegisterLocalUser.h"
#include "Misc/Base64.h"
#include "Misc/OutputDeviceRedirector.h"

THIRD_PARTY_INCLUDES_START
#include <WinSock2.h>
#include <ws2tcpip.h>
#include <xsapi-c/multiplayer_c.h>
THIRD_PARTY_INCLUDES_END

#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "OnlineAsyncTaskGDKSetSessionActivity.h"

#include "OnlineAsyncTaskGDKJoinSession.h"
#include "OnlineSessionSettings.h"

FOnlineAsyncTaskGDKJoinSession::FOnlineAsyncTaskGDKJoinSession(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	FGDKMultiplayerSessionHandle InSession,
	FNamedOnlineSessionRef InNamedSession,
	const int32 InMaxRetryCount,
	const bool bInSessionIsMatchmakingResult,
	const bool bInSetActivity,
	const TOptional<FString>& InSessionInviteHandle)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, NamedSession(InNamedSession)
	, GDKSession(InSession)
	, GDKSessionReference(nullptr)
	, GDKContext(InContext)
	, JoinResult(EOnJoinSessionCompleteResult::Success)
	, MaxRetryCount(InMaxRetryCount)
	, TotalRetryAttempts(0)
	, TotalSessionWrites(0)
	, TotalSessionReads(0)
	, bSessionIsMatchmakingResult(bInSessionIsMatchmakingResult)
	, OtherLocalPlayersToAdd(0)
	, bSetActivity(bInSetActivity)
	, SessionInviteHandle(InSessionInviteHandle)
{
	check(Subsystem);
	check(GDKSession);
	GDKSessionReference = XblMultiplayerSessionSessionReference(GDKSession);
	check(GDKSessionReference);
	check(GDKContext);
	check(MaxRetryCount > 0);
}

FOnlineAsyncTaskGDKJoinSession::FOnlineAsyncTaskGDKJoinSession(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	const XblMultiplayerSessionReference* InReference,
	FNamedOnlineSessionRef InNamedSession,
	const int32 InMaxRetryCount,
	const bool bInSessionIsMatchmakingResult,
	const bool bInSetActivity,
	const TOptional<FString>& InSessionInviteHandle)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, NamedSession(InNamedSession)
	, GDKSession(nullptr)
	, GDKContext(InContext)
	, JoinResult(EOnJoinSessionCompleteResult::Success)
	, MaxRetryCount(InMaxRetryCount)
	, TotalRetryAttempts(0)
	, TotalSessionWrites(0)
	, TotalSessionReads(0)
	, bSessionIsMatchmakingResult(bInSessionIsMatchmakingResult)
	, OtherLocalPlayersToAdd(0)
	, bSetActivity(bInSetActivity)
	, SessionInviteHandle(InSessionInviteHandle)
{
	check(Subsystem);
	check(GDKSessionReference);
	check(GDKContext);
	check(MaxRetryCount > 0);
	FMemory::Memzero(&AsyncBlock, sizeof(AsyncBlock));

	GDKSessionReferenceInternal = *InReference;
	GDKSessionReference = &GDKSessionReferenceInternal;
}

void FOnlineAsyncTaskGDKJoinSession::Initialize()
{
	FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>::Initialize();

	if (!bIsComplete)
	{
		Retry();
	}
}

void FOnlineAsyncTaskGDKJoinSession::Finalize()
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineAsyncTaskGDKJoinSession::Finalize JoinSessionGDK Complete bWasSuccessful=[%d]"), WasSuccessful());

	FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
	check(SessionInterface.IsValid());

	if (!bWasSuccessful)
	{
		// Clean up partial create/join
		SessionInterface->RemoveNamedSession(NamedSession->SessionName);
		return;
	}

	// Update our cached session info
	Subsystem->CacheGDKSession(NamedSession->SessionName, GDKSession);

	NamedSession->SessionState = EOnlineSessionState::Pending;

	// Set activity to new session. This will be the session used for invites/join in progress if supported.
	if (bSetActivity)
	{
		FGDKUserHandle GDKUser;
		HRESULT Result = XblContextGetUser(GDKContext, GDKUser.GetInitReference());
		if (GDKUser)
		{
			uint64 GDKUserId;
			if (SUCCEEDED(XUserGetId(GDKUser, &GDKUserId)))
			{
				FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(GDKUserId);

				SessionInterface->GetMpsdImpl()->SetUserActiveSessionActivity(*UserNetId, GDKSession);
			}
		}
	}

	if (NamedSession->SessionInfo.IsValid() && !bSessionIsMatchmakingResult)
	{

		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		const XblMultiplayerSessionMember* NewHostMember = FOnlineSessionMpsdGDK::GetGDKSessionHost(GDKSession);
		if (NewHostMember != nullptr && NewHostMember->SecureDeviceBaseAddress64)
		{
			TSharedPtr<FInternetAddr> HostAddr = FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64(UTF8_TO_TCHAR(NewHostMember->SecureDeviceBaseAddress64));	
			GDKInfo->SetHostAddr(HostAddr);
			UE_CLOG_ONLINE_SESSION(HostAddr.IsValid(),Verbose, TEXT("FOnlineAsyncTaskGDKJoinSession::Finalize Getting host address as %s"), *HostAddr->ToString(true));			
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineAsyncTaskGDKJoinSession::Finalize No host address for session joined") );
		}
		
	}

	// Initialize session state after create/join
	Subsystem->GetSessionMessageRouter()->SyncInitialSessionState(NamedSession->SessionName, GDKSession);
}

void FOnlineAsyncTaskGDKJoinSession::TriggerDelegates()
{
	if (!bWasSuccessful && bSessionIsMatchmakingResult)
	{
		// This join was part of session initialization during matchmaking, and it failed. Matchmaking
		// needs to fail as well.
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKJoinSession_TriggerDelegates_Matchmaking);
		Subsystem->GetMatchmakingInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(NamedSession->SessionName, bWasSuccessful);
	}

	// In matchmaking, this isn't a final state, and we don't want to trigger any unrelated delegates.
	if (!bSessionIsMatchmakingResult)
	{
		FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
		check(SessionInterface.IsValid());

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKJoinSession_TriggerDelegates_JoinSession);
		SessionInterface->TriggerOnJoinSessionCompleteDelegates(NamedSession->SessionName, JoinResult);
	}
}

void FOnlineAsyncTaskGDKJoinSession::OnSuccess()
{
	JoinResult = EOnJoinSessionCompleteResult::Success;
	bWasSuccessful = true;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKJoinSession::OnFailed(EOnJoinSessionCompleteResult::Type Result)
{
	JoinResult = Result;
	bWasSuccessful = false;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKJoinSession::Retry()
{
	if (TotalRetryAttempts >= MaxRetryCount)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to start session join; ran out of join attempts"));
		if (TotalSessionReads > 0 && TotalSessionWrites == 0)
		{
			OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
		}
		else
		{
			OnFailed(EOnJoinSessionCompleteResult::UnknownError);
		}
		return;
	}

	++TotalRetryAttempts;

	const bool bGetSession = !GDKSession.IsValid();
	if (bGetSession)
	{
		++TotalSessionReads;

		FMemory::Memzero(&AsyncBlock, sizeof(AsyncBlock));
		AsyncBlock.context = this;		

		if (SessionInviteHandle.IsSet())
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Attempting to find session by invite handle %s"), *SessionInviteHandle.GetValue());

			AsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
			{
				FOnlineAsyncTaskGDKJoinSession* Owner = static_cast<FOnlineAsyncTaskGDKJoinSession*>(LambdaAsyncBlock->context);
				Owner->ProcessGetSessionByHandleResult();
			};

			HRESULT Result = XblMultiplayerGetSessionByHandleAsync(GDKContext, TCHAR_TO_UTF8(*SessionInviteHandle.GetValue()), &AsyncBlock);
			if (Result != S_OK)
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to find session."));
				OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
			}
		}
		else if(NamedSession->SessionInfo.IsValid() && StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo)->GetGDKMultiplayerSearchHandle().IsValid())
		{
			const char* HandleID = nullptr;
			HRESULT Result = XblMultiplayerSearchHandleGetId(StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo)->GetGDKMultiplayerSearchHandle(), &HandleID);
			if (Result == S_OK)
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Attempting to find session by search handle %s"), UTF8_TO_TCHAR(HandleID));
				AsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
				{
					FOnlineAsyncTaskGDKJoinSession* Owner = static_cast<FOnlineAsyncTaskGDKJoinSession*>(LambdaAsyncBlock->context);
					Owner->ProcessGetSessionByHandleResult();
				};

				Result = XblMultiplayerGetSessionByHandleAsync(GDKContext, HandleID, &AsyncBlock);
				if (Result != S_OK)
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to find session."));
					OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed get search handleID"));
				OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Attempting to find session by reference"));
			
			AsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
			{
				FOnlineAsyncTaskGDKJoinSession* Owner = static_cast<FOnlineAsyncTaskGDKJoinSession*>(LambdaAsyncBlock->context);
				Owner->ProcessGetSessionResult();
			};
			HRESULT Result = XblMultiplayerGetSessionAsync(GDKContext, GDKSessionReference, &AsyncBlock);
			if (Result != S_OK)
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to find session."));
				OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
			}
		}
	}
	else
	{
		TryJoinSession();
	}
}

void FOnlineAsyncTaskGDKJoinSession::ProcessGetSessionResultCommon(HRESULT Result)
{
	if (Result != S_OK)
	{
		if (Result == HTTP_E_STATUS_NOT_FOUND)
		{
			GDKSession.Clear();
			Retry();
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to find get session."));
			OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
		}
		return;
	}

	TryJoinSession();
}

void FOnlineAsyncTaskGDKJoinSession::ProcessGetSessionResult()
{
	ProcessGetSessionResultCommon(XblMultiplayerGetSessionResult(&AsyncBlock, GDKSession.GetInitReference()));
}

void FOnlineAsyncTaskGDKJoinSession::ProcessGetSessionByHandleResult()
{
	ProcessGetSessionResultCommon(XblMultiplayerGetSessionByHandleResult(&AsyncBlock, GDKSession.GetInitReference()));
}

void FOnlineAsyncTaskGDKJoinSession::TryJoinSession()
{
	// Session may be null if it timed out after the SessionReference was stored
	if (!GDKSession.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to start session join; GDKSession invalid"));
		OnFailed(EOnJoinSessionCompleteResult::SessionDoesNotExist);
		return;
	}

	// Don't join if the session is full.
	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());

	if (!FOnlineSessionMpsdGDK::CanUserJoinSession(GDKUser, GDKSession))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to start session join; CanUserJoinSession returned false"));
		OnFailed(EOnJoinSessionCompleteResult::SessionIsFull);
		return;
	}

	// For matchmaking, identify local users for proper join and QoS handling
	LocalMembers.Empty();
	TArray<uint32> LocalMemberIds;

	if (bSessionIsMatchmakingResult)
	{
		FOnlineIdentityGDKPtr IdentityInt(Subsystem->GetIdentityGDK());
		check(IdentityInt.IsValid());
		{
			TArray<FGDKUserHandle>& CachedUsers = IdentityInt->GetCachedUsers();
			
			const XblMultiplayerSessionMember* Members;
			uint64 MembersCount = 0;
			XblMultiplayerSessionMembers(GDKSession, &Members, &MembersCount);
				
			for (int i=0; i<MembersCount; ++i)
			{
				const XblMultiplayerSessionMember& Member = Members[i];
				for (FGDKUserHandle& User : CachedUsers)
				{
					uint64 UserId = 0;
					if (SUCCEEDED(XUserGetId(User, &UserId)) && Member.Xuid == UserId)
					{
						// Found a local member
						LocalMembers.Add(&Member);
						LocalMemberIds.Add(Member.MemberId);
						break;
					}
				}
			}
		}

		// Ensure all local members pass or fail QoS together by putting them in an initialization group.
		// Other local members will also need to set this in RegisterLocalUserAsync.
		XblMultiplayerSessionCurrentUserSetMembersInGroup(GDKSession, LocalMemberIds.GetData(), LocalMemberIds.Num());

		// Don't need to call join, matchmaking did that automatically
	}
	else
	{
		// Determine if we want to use managed initialization based on if it was configured in the session template.
		bool bInitialize = false;

		const XblMultiplayerSessionConstants* SesessionContants = XblMultiplayerSessionSessionConstants(GDKSession);

		if (SesessionContants !=nullptr && SesessionContants->MemberInitialization != nullptr)
		{
			bInitialize = !(SesessionContants->MemberInitialization->JoinTimeout == 0 &&
							SesessionContants->MemberInitialization->MeasurementTimeout == 0 &&
							SesessionContants->MemberInitialization->EvaluationTimeout == 0 &&
							SesessionContants->MemberInitialization->MembersNeededToStart == 0 &&
							SesessionContants->MemberInitialization->ExternalEvaluation == false);
		}

		HRESULT Result = XblMultiplayerSessionJoin(GDKSession, nullptr, bInitialize, true);
		if (FAILED(Result))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to join session"));
		}
		
	}
	HRESULT Result = XblMultiplayerSessionCurrentUserSetStatus(GDKSession, XblMultiplayerSessionMemberStatus::Active);
	if (FAILED(Result))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to activate user"));
	}
	// The client's external/WAN address should be set for the user so that other clients can initiate
	// connections.  Xbox platform doesn't currently have a way to retrieve this address so the local
	// address is used instead.  This means that clients will only be able to connect to each other if
	// they are on the same local network.
	bool BindAll = false;
	TSharedRef<FInternetAddr> LocalIp = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, BindAll);
#if WITH_ENGINE
	LocalIp->SetPort(FURL::UrlConfig.DefaultPort);
#else
	FString Port;
	// Allow the command line to override the default port
	if (FParse::Value(FCommandLine::Get(),TEXT("Port="),Port) == false)
	{
		Port = GConfig->GetStr( TEXT("URL"), TEXT("Port"), GEngineIni );
	}
	LocalIp->SetPort(FCString::Atoi( *Port ));
#endif //WITH_ENGINE
	if (LocalIp->IsValid())
	{
		XblMultiplayerSessionCurrentUserSetSecureDeviceAddressBase64(GDKSession, TCHAR_TO_UTF8(*FBase64::Encode(LocalIp->ToString(true))));
	}

	// Indicate what events to subscribe to. Also handle joining matchmaking target session prior to QoS.
	// Also use TryWriteSessionAsync when committing changes
	XblMultiplayerSessionChangeTypes SubscriptionType = FOnlineSessionMpsdGDK::GetSubscriptionType(NamedSession->SessionSettings);
	XblMultiplayerSessionSetSessionChangeSubscription(GDKSession, SubscriptionType);

	// If this is from a matchmaking result, our session info may already be created
	if (!NamedSession->SessionInfo.IsValid())
	{
		NamedSession->SessionInfo = MakeShared<FOnlineSessionInfoMpsdGDK>(GDKSession);
	}

	++TotalSessionWrites;

	FMemory::Memzero(&AsyncBlock, sizeof(AsyncBlock));
	AsyncBlock.context = this;

	if (SessionInviteHandle.IsSet())
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Attempting to join session by session handle %s"), *SessionInviteHandle.GetValue());
		AsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
		{
			FOnlineAsyncTaskGDKJoinSession* Owner = static_cast<FOnlineAsyncTaskGDKJoinSession*>(LambdaAsyncBlock->context);
			Owner->ProcessJoinSessionByHandleResult();
		};

		Result = XblMultiplayerWriteSessionByHandleAsync(GDKContext, GDKSession, XblMultiplayerSessionWriteMode::UpdateExisting, TCHAR_TO_UTF8(*SessionInviteHandle.GetValue()), &AsyncBlock);
		if (Result != S_OK)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to start session join write with error 0x%0.8X"), Result);
			OnFailed(EOnJoinSessionCompleteResult::UnknownError);
			return;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Attempting to join session by session object"));
		AsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
		{
			FOnlineAsyncTaskGDKJoinSession* Owner = static_cast<FOnlineAsyncTaskGDKJoinSession*>(LambdaAsyncBlock->context);
			Owner->ProcessJoinSessionByHandleResult();
		};

		const XblMultiplayerSessionInfo* SessionInfo = XblMultiplayerSessionGetInfo(GDKSession);

		Result = XblMultiplayerWriteSessionByHandleAsync(GDKContext, GDKSession, XblMultiplayerSessionWriteMode::UpdateExisting, SessionInfo->SearchHandleId, &AsyncBlock);
		if (Result != S_OK)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to start session join write with error 0x%0.8X"), Result);
			OnFailed(EOnJoinSessionCompleteResult::UnknownError);
			return;
		}	
	}
}

void FOnlineAsyncTaskGDKJoinSession::ProcessJoinSessionByHandleResult()
{
	FGDKMultiplayerSessionHandle NewSessionHandle;
	HRESULT Result = XblMultiplayerWriteSessionByHandleResult(&AsyncBlock, NewSessionHandle.GetInitReference());
	ProcessJoinSessionResultCommon(NewSessionHandle, Result);
	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineAsyncTaskGDKJoinSession::ProcessJoinSessionByHandleResult: XblMultiplayerWriteSessionResult, Result: (0x%0.8X)."), Result);
}

void FOnlineAsyncTaskGDKJoinSession::ProcessJoinSessionResultCommon(FGDKMultiplayerSessionHandle NewSessionHandle, HRESULT Result)
{
	if(Result == S_OK)
	{
		GDKSession = NewSessionHandle;

		check(GDKSessionReference);
		// Register for shouldertaps (push-notifications from GDK about session changes)
		Subsystem->GetSessionMessageRouter()->AddOnSessionChangedDelegate(Subsystem->GetSessionInterfaceGDK()->GetMpsdImpl()->OnSessionChangedDelegate, GDKSessionReference);

		if (bSessionIsMatchmakingResult)
		{
			TryJoinSessionFromMatchmaking(LocalMembers);
		}
		else if (NamedSession->SessionSettings.bIsDedicated)
		{
			TryJoinSessionFromDedicated();
		}
		else if (const XblMultiplayerSessionMember* Host = FOnlineSessionMpsdGDK::GetGDKSessionHost(GDKSession))
		{
			TryJoinSessionFromPeer(Host);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to join session: no host player and session not dedicated."));
			OnFailed(EOnJoinSessionCompleteResult::CouldNotRetrieveAddress);
			return;
		}		
	}
	//else if (Result->Status == WriteSessionStatus::OutOfSync) //TODO: WMM is there a result code that maps to this?
	//{
	//	Retry();
	//}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to write to session with error 0x%0.8X"), Result);
		OnFailed(EOnJoinSessionCompleteResult::UnknownError);
	}
}

void FOnlineAsyncTaskGDKJoinSession::TryJoinSessionFromMatchmaking(const TArray<const XblMultiplayerSessionMember*>& InLocalMembers)
{
	// There may have been other local users in the matchmaking session. Put them
	// in the game session as well.

	OtherLocalPlayersToAdd.Set(0);
	bWasSuccessful = true;	// assume success unless a RegisterLocalPlayer delegate changes this

	XblMultiplayerSessionChangeTypes SubscriptionType = FOnlineSessionMpsdGDK::GetSubscriptionType(NamedSession->SessionSettings);

	for (const XblMultiplayerSessionMember* Member : InLocalMembers)
	{
		const XblMultiplayerSessionMember* GDKSessionMember = XblMultiplayerSessionCurrentUser(GDKSession);
		if (GDKSessionMember != nullptr)
		{
			if (Member->Xuid != GDKSessionMember->Xuid)
			{
				OtherLocalPlayersToAdd.Increment();

				FUniqueNetIdGDKRef MemberId = FUniqueNetIdGDK::Create(Member->Xuid);
				FGDKContextHandle MemberContext = Subsystem->GetGDKContext(*MemberId);
				if (MemberContext)
				{
					Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKRegisterLocalUser>(
						Subsystem,
						MemberContext,
						NamedSession->SessionName,
						MemberId,
						GDKSession,
						SubscriptionType,
						FOnRegisterLocalPlayerCompleteDelegate::CreateRaw(this, &FOnlineAsyncTaskGDKJoinSession::OnAddLocalPlayerComplete),
						InLocalMembers);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKJoinSession::TryJoinSessionFromMatchmaking - no GDK context for user %s"), *MemberId->ToDebugString());
					OnAddLocalPlayerComplete( *MemberId, EOnJoinSessionCompleteResult::UnknownError );
				}
			}
		}
	}

	bIsComplete = (OtherLocalPlayersToAdd.GetValue() == 0);

	// Connecting to the host, etc., will happen in the GameSessionReady task after
	// QoS is complete.
}

void FOnlineAsyncTaskGDKJoinSession::TryJoinSessionFromDedicated()
{
	// Success, we're in the session, but it's still up to the game to get onto the server
	OnSuccess();
}

void FOnlineAsyncTaskGDKJoinSession::TryJoinSessionFromPeer(const XblMultiplayerSessionMember* Host)
{
	OnSuccess();
}

void FOnlineAsyncTaskGDKJoinSession::OnAddLocalPlayerComplete(const FUniqueNetId& PlayerId, EOnJoinSessionCompleteResult::Type Result)
{
	OtherLocalPlayersToAdd.Decrement();

	if (Result != EOnJoinSessionCompleteResult::Success)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("FOnlineAsyncTaskGDKJoinSession::OnAddLocalPlayerComplete: failed to add local player to game session with result %u"), Result);
		bWasSuccessful = false;
	}

	if (OtherLocalPlayersToAdd.GetValue() <= 0)
	{
		bIsComplete = true;
	}
}

#endif //WITH_GRDK