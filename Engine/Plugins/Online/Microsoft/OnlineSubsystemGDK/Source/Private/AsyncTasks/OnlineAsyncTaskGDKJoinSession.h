// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "HAL/ThreadSafeCounter.h"

class FOnlineSubsystemGDK;
class FNamedOnlineSession;

class FOnlineAsyncTaskGDKJoinSession : public FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the GDK context of the user who's updating the session
	 * @param InSession the session we're writing to
	 * @param InNamedSession The named session to join
	 * @param InMaxRetryCount number of times to retry the session write
	 * @param bInSessionIsMatchmakingResult Is this a matchmaking result?
	 * @param bInSetActivity Should we set this as our presence session?
	 * @param InSessionInviteHandle Secret session key in case of invites to private parties
	 */
	FOnlineAsyncTaskGDKJoinSession(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		FGDKMultiplayerSessionHandle InSession,
		FNamedOnlineSessionRef InNamedSession,
		const int32 InMaxRetryCount,
		const bool bInSessionIsMatchmakingResult,
		const bool bInSetActivity,
		const TOptional<FString>& InSessionInviteHandle);

	/**
	 * This constructor takes a GDK Session Reference and is safe to call
	 * in non-game threads.  We will download the current session object from
	 * the provided reference.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the GDK context of the user who's updating the session
	 * @param InSessionReference the session reference we need to read to get the session we need to join

	 * @param InNamedSession The named session to join
	 * @param InMaxRetryCount number of times to retry the session write
	 * @param bInSessionIsMatchmakingResult Is this a matchmaking result?
	 * @param bInSetActivity Should we set this as our presence session?
	 * @param InSessionInviteHandle Secret session key in case of invites to private parties
	 */
	FOnlineAsyncTaskGDKJoinSession(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const XblMultiplayerSessionReference* InGDKSessionReference,
		FNamedOnlineSessionRef InNamedSession,
		const int32 InMaxRetryCount,
		const bool bInSessionIsMatchmakingResult,
		const bool bInSetActivity,
		const TOptional<FString>& InSessionInviteHandle);

	// FOnlineAsyncItem
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskGDKJoinSession SessionName=[%s] bWasSuccessful=[%d] JoinResult=[%s] UpdateAttempts=[%d/%d] SessionWrites=[%d] SessionReads=[%d]"),
			*NamedSession->SessionName.ToString(), WasSuccessful(), LexToString(JoinResult), TotalRetryAttempts, MaxRetryCount, TotalSessionWrites, TotalSessionReads);
	}

	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

	void ProcessGetSessionResultCommon(HRESULT Result);
	void ProcessGetSessionResult();
	void ProcessGetSessionByHandleResult();
	void ProcessJoinSessionResultCommon(FGDKMultiplayerSessionHandle NewSessionHandle, HRESULT Result);
	void ProcessJoinSessionByHandleResult();
private:
	void OnSuccess();
	void OnFailed(EOnJoinSessionCompleteResult::Type Result);
	void Retry();
	void TryJoinSession();
	void TryJoinSessionFromMatchmaking(const TArray<const XblMultiplayerSessionMember*>& LocalMembers);
	void TryJoinSessionFromDedicated();
	void TryJoinSessionFromPeer(const XblMultiplayerSessionMember* Host);

	// Callback when other local players are added to session in matchmaking case
	void OnAddLocalPlayerComplete(const FUniqueNetId& PlayerId, EOnJoinSessionCompleteResult::Type Result);

	/** Named session backing this object */
	FNamedOnlineSessionRef NamedSession;

	/** GDK session object we're writing to to join (may be null) */
	FGDKMultiplayerSessionHandle GDKSession;
	/** GDK session reference object used to obtain an GDKSession (MUST not be null) */
	const XblMultiplayerSessionReference* GDKSessionReference;


	/** GDK session reference object used to obtain an GDKSession. Internal copy made when joining via transient reference */
	XblMultiplayerSessionReference GDKSessionReferenceInternal;

	/** GDK context for the user initiating the join of this session */
	FGDKContextHandle GDKContext;

	/** The result of this join */
	EOnJoinSessionCompleteResult::Type JoinResult;

	/** Maximum amount of times to try writing upon write-conflicts */
	const int32 MaxRetryCount;

	/** Maximum amount of times to attempt our read/write flow */
	int32 TotalRetryAttempts;

	/** Total amount of times we sent a write request for this session */
	int32 TotalSessionWrites;

	/** Total amount of times we downloaded a new copy of the session document */
	int32 TotalSessionReads;

	/** Are we joining an GDK matchmaking session? */
	const bool bSessionIsMatchmakingResult;

	/** Count of other local-players are still queued to be added when we join the session (splitscreen cases) */
	FThreadSafeCounter OtherLocalPlayersToAdd;

	/** Should we set this session as our presence-advertised session? */
	const bool bSetActivity;

	/** Session invite handle to be used if we are joining a private session by invite */
	const TOptional<FString> SessionInviteHandle;

	/** Struct that GDK uses to manage async tasks */
	XAsyncBlock AsyncBlock;

	/** Array of multiplayer session members associated with local users on this device */
	TArray<const XblMultiplayerSessionMember*> LocalMembers;
};
