// Copyright Epic Games, Inc. All Rights Reserved.

// Task to do title-measured QoS and upload it as part of matchmaking

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMeasureAndUploadQos.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "OnlineAsyncTaskGDKGameSessionReady.h"
#include "OnlineSubsystemGDKTypes.h"

FOnlineAsyncTaskGDKMeasureAndUploadQos::FOnlineAsyncTaskGDKMeasureAndUploadQos(
	FOnlineSubsystemGDK* InSubsystem,
	FGDKContextHandle InContext,
	FNamedOnlineSessionRef InNamedSession,
	FGDKMultiplayerSessionHandle InGDKSession,
	int RetryCount,
	int32 InQosTimeoutMs,
	int32 InQosProbeCount)
	: FOnlineAsyncTaskGDK(InSubsystem, TEXT("FOnlineAsyncTaskGDKMeasureAndUploadQos"), 0)
	, NamedSession(InNamedSession)
	, GDKSession(InGDKSession)
	, GDKContext(InContext)
{
}

void FOnlineAsyncTaskGDKMeasureAndUploadQos::Tick()
{
	FOnlineSessionSetting * QOSSetting = NamedSession->SessionSettings.Settings.Find(SETTING_QOS);
	FString QOSString;
	if (QOSSetting != nullptr)
	{
		bWasSuccessful = false;
		QOSString = QOSSetting->Data.ToString();
		if (QOSString.IsEmpty())
		{
			UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKMeasureAndUploadQos- failed retrieve QOS measurements"));
			bIsComplete = true;
			return;
		}
		HRESULT Result = XblMultiplayerSessionCurrentUserSetServerQosMeasurements(GDKSession, TCHAR_TO_UTF8(*QOSString));
		if (SUCCEEDED(Result))
		{
			bWasSuccessful = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKMeasureAndUploadQos- failed to write QOS to session: 0x%08X"), Result);
		}
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKMeasureAndUploadQos::Initialize()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMeasureAndUploadQos_Initialize_Delegate);
	Subsystem->GetSessionInterface()->TriggerOnQosDataRequestedDelegates(NamedSession->SessionName);
}

void FOnlineAsyncTaskGDKMeasureAndUploadQos::TriggerDelegates()
{
	if (!bWasSuccessful)
	{
		FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
		MatchmakingInterface->SetTicketState(NamedSession->SessionName, EOnlineGDKMatchmakingState::None);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMeasureAndUploadQos_TriggerDelegates);
		MatchmakingInterface->TriggerOnMatchmakingCompleteDelegates(NamedSession->SessionName, false);
	}
	// On success, don't trigger anything--the change handler in OnlineSessionInterfaceGDK will see
	// when QoS finishes for the session and process the results.
}

#endif //WITH_GRDK