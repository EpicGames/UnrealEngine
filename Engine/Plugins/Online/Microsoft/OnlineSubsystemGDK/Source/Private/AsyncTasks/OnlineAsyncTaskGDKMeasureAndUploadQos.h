// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManager.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManagerGDK.h"

class FOnlineSubsystemGDK;

/** 
 * Async task used to measure QoS to peers during session initialization, then upload the results to
 * the session document.
 */
class FOnlineAsyncTaskGDKMeasureAndUploadQos : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKMeasureAndUploadQos(
		FOnlineSubsystemGDK* InSubsystem,
		FGDKContextHandle InContext,
		FNamedOnlineSessionRef InNamedSession,
		FGDKMultiplayerSessionHandle InSession,
		int32 RetryCount,
		int32 InQosTimeoutMs,
		int32 InQosProbeCount);

	virtual void Initialize() override;
	virtual FString ToString() const override { return TEXT("MeasureAndUploadQosAsync");}
	virtual void TriggerDelegates() override;
	virtual void Tick() override;

private:

	FNamedOnlineSessionRef NamedSession;
	FGDKMultiplayerSessionHandle GDKSession;
	FGDKContextHandle GDKContext;
};
