// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaDeleteActivity.h"

#include "OnlineSessionInterfaceMpaGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDK.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKMpaDeleteActivity::FOnlineAsyncTaskGDKMpaDeleteActivity(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext ,
	const FUniqueNetIdGDKRef& InUserIdGDK,
	FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaDeleteActivity"))
	, GDKContext(InGDKContext)
	, UserIdGDK(InUserIdGDK)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

void FOnlineAsyncTaskGDKMpaDeleteActivity::Initialize()
{
	FOnlineSessionMpaGDKPtr SessionInterfaceMpa = Subsystem->GetSessionInterfaceGDK()->GetMpaImpl();
		HRESULT Result = XblMultiplayerActivityDeleteActivityAsync(GDKContext, *AsyncBlock);
		SessionInterfaceMpa->ClearMpaActivity(UserIdGDK);

		if (Result != S_OK)
		{
			UE_LOG_ONLINE(Error, TEXT("Error deleting MPA activity when start, error: (0x%0.8X)."), Result);

			bWasSuccessful = false;
			bIsComplete = true;
		}	
}

void FOnlineAsyncTaskGDKMpaDeleteActivity::ProcessResults()
{
	HRESULT Result = XAsyncGetStatus(*AsyncBlock, false);
	if (FAILED(Result))
	{
		UE_LOG_ONLINE(Error, TEXT("Error deleting MPA activity, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
	}
	else
	{
		bWasSuccessful = true;
	}
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKMpaDeleteActivity::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMpaDeleteActivity_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK