// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

/**
 * Async task to delete MPA activity
 */
class FOnlineAsyncTaskGDKMpaDeleteActivity
	: public FOnlineAsyncTaskGDK
{
public:
	DECLARE_DELEGATE_OneParam(FOnComplete, bool /*bWasSuccessful*/);

	FOnlineAsyncTaskGDKMpaDeleteActivity(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FUniqueNetIdGDKRef& InUserIdGDK,
		FOnComplete InTaskCompletionDelegate
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaDeleteActivity"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	FUniqueNetIdGDKPtr UserIdGDK;
	FOnComplete TaskCompletionDelegate;
};
