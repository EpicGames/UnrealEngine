// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaGetActivities.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpaGDK.h"
#include "OnlineSubsystemGDK.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKMpaGetActivities::FOnlineAsyncTaskGDKMpaGetActivities(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const TArray<uint64>& InGDKUserIds,
	FOnlineAsyncTaskGDKMpaGetActivities::FOnComplete InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaGetActivities"))
	, GDKContext(InGDKContext)
	, GDKUserIds(InGDKUserIds)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

void FOnlineAsyncTaskGDKMpaGetActivities::Initialize()
{
	if (GDKUserIds.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting MPA activities, GDKUserIds can't be empty."));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XblMultiplayerActivityGetActivityAsync(
		GDKContext,
		GDKUserIds.GetData(),
		GDKUserIds.Num(),
		*AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting MPA activities when start, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKMpaGetActivities::ProcessResults()
{
    size_t ResultSize = 0;
    HRESULT Result = XblMultiplayerActivityGetActivityResultSize(*AsyncBlock, &ResultSize);

	if (Result == S_OK)
	{
		TArray<uint8> Buffer;
		Buffer.Reserve(ResultSize);
        XblMultiplayerActivityInfo* ActivityInfoArray = nullptr;
        size_t ResultCount = 0;

        Result = XblMultiplayerActivityGetActivityResult(*AsyncBlock, ResultSize, Buffer.GetData(), &ActivityInfoArray, &ResultCount, nullptr/*bufferUsed*/);
        if (Result == S_OK)
        {
			bWasSuccessful = true;
			OnlineSessions.Reserve(ResultCount);
			for (size_t i = 0; i < ResultCount; ++i)
			{
				const XblMultiplayerActivityInfo& ActivityInfo = ActivityInfoArray[i];
				FOnlineSession OnlineSession;
				OnlineSession.OwningUserId = FUniqueNetIdGDK::Create(ActivityInfo.xuid);
				FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
				OnlineSession.SessionSettings.Set(SETTING_CUSTOM_JOIN_INFO, FString(ActivityInfo.connectionString));
				OnlineSession.SessionSettings.bAllowJoinViaPresence = ActivityInfo.joinRestriction == XblMultiplayerActivityJoinRestriction::Followed;
				
				OnlineSessions.Add(OnlineSession);
			}
        }
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting MPA activities, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
	}

	bIsComplete = true;
}

void FOnlineAsyncTaskGDKMpaGetActivities::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMpaGetActivities_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, MoveTemp(OnlineSessions));
}

#endif //WITH_GRDK