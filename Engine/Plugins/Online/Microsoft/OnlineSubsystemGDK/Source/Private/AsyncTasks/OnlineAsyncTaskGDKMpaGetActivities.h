// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

/**
 * Async task to get activities of users which was set by MPA
 */
class FOnlineAsyncTaskGDKMpaGetActivities
	: public FOnlineAsyncTaskGDK
{
public:
	DECLARE_DELEGATE_TwoParams(FOnComplete, bool /*bWasSuccessful*/, TArray<FOnlineSession> /*OnlineSessions*/);

	FOnlineAsyncTaskGDKMpaGetActivities(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64>& InGDKUserIds,
		FOnComplete InTaskCompletionDelegate
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaGetActivities"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	TArray<uint64> GDKUserIds; 
	TArray<FOnlineSession> OnlineSessions;
	FOnComplete TaskCompletionDelegate;
};
