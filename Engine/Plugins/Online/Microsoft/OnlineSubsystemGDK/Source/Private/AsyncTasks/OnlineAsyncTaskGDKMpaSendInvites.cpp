// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaSendInvites.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKMpaSendInvites::FOnlineAsyncTaskGDKMpaSendInvites(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const TArray<uint64>& InGDKUserIds, 
	bool InAllowCrossPlatformJoin,
	const FString& InConnectionString
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaSendInvites"))
	, GDKContext(InGDKContext)
	, GDKUserIds(InGDKUserIds)
	, AllowCrossPlatformJoin(InAllowCrossPlatformJoin)
	, ConnectionString(InConnectionString)
{
}

void FOnlineAsyncTaskGDKMpaSendInvites::Initialize()
{
	if (GDKUserIds.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("Error sending MPA invites, GDKUserIds can't be empty."));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XblMultiplayerActivitySendInvitesAsync(
		GDKContext,
		GDKUserIds.GetData(),
		GDKUserIds.Num(),
		AllowCrossPlatformJoin,
		TCHAR_TO_ANSI(*ConnectionString),
		*AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error sending MPA invites when start, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKMpaSendInvites::ProcessResults()
{
	HRESULT Result = XAsyncGetStatus(*AsyncBlock, false);
	if (FAILED(Result))
	{
		UE_LOG_ONLINE(Error, TEXT("Error sending MPA invites, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
	}
	else
	{
		bWasSuccessful = true;
	}
	bIsComplete = true;
}


#endif //WITH_GRDK