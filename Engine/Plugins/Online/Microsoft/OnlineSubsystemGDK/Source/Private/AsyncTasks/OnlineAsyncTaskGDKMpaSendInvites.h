// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

/**
 * Async task to send MPA invites
 */
class FOnlineAsyncTaskGDKMpaSendInvites
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKMpaSendInvites(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64>& InGDKUserIds,
		bool InAllowCrossPlatformJoin,
		const FString& InConnectionString
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaSendInvites"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

protected:
	FGDKContextHandle GDKContext;
	TArray<uint64> GDKUserIds; 
	bool AllowCrossPlatformJoin;
	FString ConnectionString;
};
