// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaSendInvitesWithUI.h"

THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKMpaSendInvitesWithUI::FOnlineAsyncTaskGDKMpaSendInvitesWithUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	FGDKUserHandle InGDKUser
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaSendInvitesWithUI"))
	, GDKContext(InGDKContext)
	, GDKUser(InGDKUser)
{
}

void FOnlineAsyncTaskGDKMpaSendInvitesWithUI::Initialize()
{
	HRESULT Result = XGameUiShowMultiplayerActivityGameInviteAsync(*AsyncBlock, GDKUser);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error sending MPA invites with UI when start, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKMpaSendInvitesWithUI::ProcessResults()
{
	HRESULT Result = XAsyncGetStatus(*AsyncBlock, false);
	if (FAILED(Result))
	{
		UE_LOG_ONLINE(Error, TEXT("Error sending MPA invites, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
	}
	else
	{
		bWasSuccessful = true;
	}
	bIsComplete = true;
}


#endif //WITH_GRDK