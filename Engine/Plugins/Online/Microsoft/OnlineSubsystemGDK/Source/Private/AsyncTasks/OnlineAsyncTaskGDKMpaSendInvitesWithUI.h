// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

/**
 * Async task to send MPA invites by showing xbox UI to select invitees
 */
class FOnlineAsyncTaskGDKMpaSendInvitesWithUI
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKMpaSendInvitesWithUI(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		FGDKUserHandle InGDKUser
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaSendInvitesWithUI"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

protected:
	FGDKContextHandle GDKContext;
	FGDKUserHandle GDKUser;
};
