// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaSetActivity.h"

#include "OnlineSessionInterfaceMpaGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDK.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

TAutoConsoleVariable<bool> CVarXboxSetMpaActivityInPrivateSession(
	TEXT("xb.setMpaActivityInPrivateSession"),
	true,
	TEXT("Set mpa activity even for private sessions")
);


FOnlineAsyncTaskGDKMpaSetActivity::FOnlineAsyncTaskGDKMpaSetActivity(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const FUniqueNetIdGDKRef& InUserIdGDK, 
	const FName& InSessionName,
	const FString& InConnectionString,
	const FString& InGroupId,
	uint32 InCurrentPlayers,
	bool InAllowCrossPlatformJoin,
	const FOnlineSessionSettings& InOnlineSessionSettings,
	FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaSetActivity"))
	, GDKContext(InGDKContext)
	, UserIdGDK(InUserIdGDK)
	, SessionName(InSessionName)
	, ConnectionString(TCHAR_TO_ANSI(*InConnectionString))
	, GroupId(TCHAR_TO_ANSI(*InGroupId))
	, CurrentPlayers(InCurrentPlayers)
	, AllowCrossPlatformJoin(InAllowCrossPlatformJoin)
	, OnlineSessionSettings(InOnlineSessionSettings)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

namespace UE
{
namespace Online
{
namespace Private
{

uint32 GetMaxPlayers(const FOnlineSessionSettings& OnlineSessionSettings)
{
	if (OnlineSessionSettings.bShouldAdvertise)
	{
		return OnlineSessionSettings.NumPublicConnections;
	}
	else if (OnlineSessionSettings.bAllowJoinViaPresence)
	{
		return OnlineSessionSettings.NumPrivateConnections;
	}
	else
	{
		return OnlineSessionSettings.NumPrivateConnections;
	}
}

XblMultiplayerActivityJoinRestriction GetJoinRestriction(const FOnlineSessionSettings& OnlineSessionSettings)
{
	// For in-game party, there is no setting corresponding to XblMultiplayerActivityJoinRestriction::Public, which allow any follower of a player to join through presence, who is not a friend

	if (OnlineSessionSettings.bAllowJoinViaPresence)
	{
		return XblMultiplayerActivityJoinRestriction::Followed;
	}

	return XblMultiplayerActivityJoinRestriction::InviteOnly;
}

FString GetJoinRestrictionAsString(const FOnlineSessionSettings& OnlineSessionSettings)
{
	switch (GetJoinRestriction(OnlineSessionSettings))
	{
	case XblMultiplayerActivityJoinRestriction::Followed: return TEXT("Followed");
	case XblMultiplayerActivityJoinRestriction::Public: return TEXT("Public");
	case XblMultiplayerActivityJoinRestriction::InviteOnly: return TEXT("InviteOnly");
	default: checkNoEntry(); return "Unknown";
	}
}

uint64 GetLocalUserId(const FGDKContextHandle& GDKContext)
{
	uint64 Xuid{0};
	XblContextGetXboxUserId(GDKContext, &Xuid);
	return Xuid;
}

}
}
}

void FOnlineAsyncTaskGDKMpaSetActivity::Initialize()
{
	FOnlineSessionMpaGDKPtr SessionInterfaceMpa = Subsystem->GetSessionInterfaceGDK()->GetMpaImpl();
	
	XblMultiplayerActivityJoinRestriction JoinRestriction = UE::Online::Private::GetJoinRestriction(OnlineSessionSettings);
	bool bShouldSetActivityOverride = CVarXboxSetMpaActivityInPrivateSession.GetValueOnAnyThread();

	if (JoinRestriction != XblMultiplayerActivityJoinRestriction::InviteOnly || bShouldSetActivityOverride)
	{
		uint32 MaxPlayers = UE::Online::Private::GetMaxPlayers(OnlineSessionSettings);

#ifdef UE_PLAYFAB_MATCHMAKING
		if (ConnectionString.empty())
		{	// Set activity will fail without a connection string. With our playfab net driver this is set later.
			ConnectionString = "PLACEHOLDER";
		}
#endif// UE_PLAYFAB_MATCHMAKING

		FMpaActivity MpaActivity; 
		MpaActivity.ConnectionString = ConnectionString;
		MpaActivity.GroupId = GroupId;
		MpaActivity.CurrentPlayers = CurrentPlayers;
		MpaActivity.MaxPlayers = MaxPlayers;
		MpaActivity.JoinRestriction = JoinRestriction;
		MpaActivity.AllowCrossPlatformJoin = AllowCrossPlatformJoin;

		if (const FMpaActivity* ExistingMpaActivity = SessionInterfaceMpa->GetMpaActivity(UserIdGDK))
		{
			if (MpaActivity == *ExistingMpaActivity)
			{
				bWasSuccessful = true;
				bIsComplete = true;
				return;
			}
		}

		XblMultiplayerActivityInfo ActivityInfo;
		// ActivityInfo.xuid don't need to be set here, it's only used when get activities. Players can only set activity for themselves.
		ActivityInfo.connectionString = ConnectionString.c_str();
		ActivityInfo.groupId = GroupId.c_str();
		ActivityInfo.currentPlayers = CurrentPlayers;
		ActivityInfo.maxPlayers  = MaxPlayers;
		ActivityInfo.joinRestriction = JoinRestriction;

		HRESULT Result = XblMultiplayerActivitySetActivityAsync(
			GDKContext,
			&ActivityInfo,
			AllowCrossPlatformJoin,
			*AsyncBlock);

		if (Result == S_OK)
		{
			SessionInterfaceMpa->SetMpaActivity(UserIdGDK, MpaActivity);
			bSetLocalActivty = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Error setting MPA activity when start, error: (0x%0.8X)."), Result);

			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		if (SessionInterfaceMpa->GetMpaActivity(UserIdGDK))
		{
			HRESULT Result = XblMultiplayerActivityDeleteActivityAsync(GDKContext, *AsyncBlock);

			if (Result == S_OK)
			{
				SessionInterfaceMpa->ClearMpaActivity(UserIdGDK);
			}
			else
			{
				UE_LOG_ONLINE(Error, TEXT("Error unsetting MPA activity when start, error: (0x%0.8X)."), Result);
				bWasSuccessful = false;
				bIsComplete = true;
			}
		}
		else
		{
			bWasSuccessful = true;
			bIsComplete = true;
		}
	}
}

void FOnlineAsyncTaskGDKMpaSetActivity::ProcessResults()
{
	HRESULT Result = XAsyncGetStatus(*AsyncBlock, false);
	if (FAILED(Result))
	{
		UE_LOG_ONLINE(Error, TEXT("Error setting MPA activity, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
	}
	else
	{
		bWasSuccessful = true;
	}

	UE_LOG_ONLINE(Verbose, TEXT("Setting MPA activity %s for user %lld, groupId: %s, currentPLayers: %d, maxPlayers: %d, joinRestriction: %s, allowCrossPlatformJoin: %s, connectionString: %s"), 
		bWasSuccessful ? TEXT("succeed") : TEXT("failed"),
		UE::Online::Private::GetLocalUserId(GDKContext),
		UTF8_TO_TCHAR(GroupId.c_str()),
		CurrentPlayers,
		UE::Online::Private::GetMaxPlayers(OnlineSessionSettings),
		*UE::Online::Private::GetJoinRestrictionAsString(OnlineSessionSettings),
		*LexToString(AllowCrossPlatformJoin),
		UTF8_TO_TCHAR(ConnectionString.c_str()));

	bIsComplete = true;
}

void FOnlineAsyncTaskGDKMpaSetActivity::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMpaSetActivity_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, SessionName);
}

void FOnlineAsyncTaskGDKMpaSetActivity::Finalize()
{		
	// If our write failed, then cleanup
	if (!bWasSuccessful)
	{
		FOnlineSessionMpaGDKPtr SessionInterfaceMpa = Subsystem->GetSessionInterfaceGDK()->GetMpaImpl();
		check(SessionInterfaceMpa);

		FName LambdaSessionName = SessionName;
		if (bSetLocalActivty)
		{
			SessionInterfaceMpa->ClearMpaActivity(UserIdGDK);
		}

		Subsystem->ExecuteNextTick([LambdaSessionName, SessionInterfaceMpa]()
			{
				SessionInterfaceMpa->RemoveNamedSession(LambdaSessionName);
			});
	}
}

#endif //WITH_GRDK