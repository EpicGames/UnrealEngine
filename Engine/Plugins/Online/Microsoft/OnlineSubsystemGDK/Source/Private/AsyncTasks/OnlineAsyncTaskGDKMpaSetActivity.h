// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include <string>

/**
 * Async task to set MPA activity
 */
class FOnlineAsyncTaskGDKMpaSetActivity
	: public FOnlineAsyncTaskGDK
{
public:
	DECLARE_DELEGATE_TwoParams(FOnComplete, bool /*bWasSuccessful*/, FName /*SessionName*/);

	FOnlineAsyncTaskGDKMpaSetActivity(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FUniqueNetIdGDKRef& InUserIdGDK, 
		const FName& InSessionName,
		const FString& InConnectionString,
		const FString& InGroupId,
		uint32 InCurrentPlayers,
		bool InAllowCrossPlatformJoin,
		const FOnlineSessionSettings& InOnlineSessionSettings,
		FOnComplete InTaskCompletionDelegate
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaSetActivity"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;
	virtual void Finalize() override;


protected:
	FGDKContextHandle GDKContext;
	FUniqueNetIdGDKPtr UserIdGDK;
	FName SessionName;
	std::string ConnectionString;
	std::string GroupId;
	uint32 CurrentPlayers;
	bool AllowCrossPlatformJoin;
	FOnlineSessionSettings OnlineSessionSettings;
	bool bSetLocalActivty = false;

	FOnComplete TaskCompletionDelegate;
};
