// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKMpaUpdateRecentPlayers.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::FOnlineAsyncTaskGDKMpaUpdateRecentPlayers(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const TArray<FReportPlayedWithUser>& InRecentPlayers,
	FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::FOnComplete InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKMpaUpdateRecentPlayers"))
	, GDKContext(InGDKContext)
	, RecentPlayers(InRecentPlayers)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

namespace UE
{
namespace Online
{
namespace Private
{
	XblMultiplayerActivityEncounterType ToXblEncounterType(ERecentPlayerEncounterType EncounterType)
	{
		switch (EncounterType)
		{
		case ERecentPlayerEncounterType::Teammate: return XblMultiplayerActivityEncounterType::Teammate;
		case ERecentPlayerEncounterType::Opponent: return XblMultiplayerActivityEncounterType::Opponent;
		case ERecentPlayerEncounterType::Default: return XblMultiplayerActivityEncounterType::Default;
		default: return XblMultiplayerActivityEncounterType::Default;
		}
	}
}
}
}

void FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::Initialize()
{
	if (RecentPlayers.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("Error updating MPA recent players, RecentPlayers can't be empty."));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	TArray<XblMultiplayerActivityRecentPlayerUpdate> RecentPlayerUpdates;
	RecentPlayerUpdates.Reserve(RecentPlayers.Num());
	for (const FReportPlayedWithUser& RecentPlayer : RecentPlayers)
	{
		XblMultiplayerActivityRecentPlayerUpdate RecentPlayerUpdate{};
		FUniqueNetIdGDKRef GDKPlayer = FUniqueNetIdGDK::Cast(*RecentPlayer.UserId);
		if (GDKPlayer->IsValid())
		{
			RecentPlayerUpdate.xuid = GDKPlayer->ToUint64();
			RecentPlayerUpdate.encounterType = UE::Online::Private::ToXblEncounterType(RecentPlayer.EncounterType);
			RecentPlayerUpdates.Add(RecentPlayerUpdate);
		}
	}

	if (RecentPlayerUpdates.IsEmpty())
	{
		bWasSuccessful = true;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XblMultiplayerActivityUpdateRecentPlayers(
		GDKContext,
		RecentPlayerUpdates.GetData(),
		RecentPlayerUpdates.Num());

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error updating MPA recent players when start, error: (0x%0.8X)."), Result);
	}

	bWasSuccessful = (Result == S_OK);
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKMpaUpdateRecentPlayers_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK