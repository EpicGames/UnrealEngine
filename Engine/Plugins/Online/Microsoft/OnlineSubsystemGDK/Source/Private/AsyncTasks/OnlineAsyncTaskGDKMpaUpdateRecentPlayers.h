// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "Interfaces/OnlineFriendsInterface.h"

/**
 * Async task to update MPA recent players
 */
class FOnlineAsyncTaskGDKMpaUpdateRecentPlayers
	: public FOnlineAsyncTaskGDK
{
public:
	DECLARE_DELEGATE_OneParam(FOnComplete, bool /*bWasSuccessful*/);

	FOnlineAsyncTaskGDKMpaUpdateRecentPlayers(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<FReportPlayedWithUser>& InRecentPlayers,
		FOnComplete InDelegate
	);

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKMpaUpdateRecentPlayers"); }

	virtual void Initialize() override;
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	TArray<FReportPlayedWithUser> RecentPlayers; 
	FOnComplete TaskCompletionDelegate;
};
