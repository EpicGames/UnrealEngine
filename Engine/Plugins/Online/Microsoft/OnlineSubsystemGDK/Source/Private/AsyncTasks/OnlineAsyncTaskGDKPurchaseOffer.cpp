// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKPurchaseOffer.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineAsyncTaskGDKQueryReceipts.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"

FOnlineAsyncTaskGDKPurchaseOfferBase::FOnlineAsyncTaskGDKPurchaseOfferBase(FOnlineSubsystemGDK* const InGDKInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdGDK, const FString& InSubClassName)
	: FOnlineAsyncTaskGDK(InGDKInterface, InSubClassName)
	, StoreId(InStoreId)
	, NetIdGDK(InNetIdGDK)
	, GDKContext(InGDKContext)
	, GDKUser(InGDKUser)
{
	check(!StoreId.IsEmpty());
}

void FOnlineAsyncTaskGDKPurchaseOfferBase::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	XStoreContextHandle GDKStoreContext = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUser);

	if (!GDKStoreContext)
	{
		ErrorResult = OnlinePurchaseGDK::Errors::PurchaseCanceled();
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XStoreShowPurchaseUIAsync(GDKStoreContext, TCHAR_TO_UTF8(*StoreId), nullptr, nullptr, *AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Warning, TEXT("Error starting GDK purchase, StoreId: %s error: (0x%0.8X)."), *StoreId, Result);
		ErrorResult = OnlinePurchaseGDK::Errors::PurchaseFailureStart(Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKPurchaseOfferBase::ProcessResults()
{
	HRESULT Result = XStoreShowPurchaseUIResult(*AsyncBlock);

	if (Result == S_OK)
	{
		ErrorResult.bSucceeded = true;
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		ErrorResult = Result == E_ABORT ? OnlinePurchaseGDK::Errors::PurchaseCanceled() : OnlinePurchaseGDK::Errors::PurchaseFailure(Result);
		UE_LOG_ONLINE(Warning, TEXT("FOnlinePurchaseGDK::Checkout failed with code 0x%0.8X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

FString FOnlineAsyncTaskGDKPurchaseOfferBase::ToString() const
{
	return FString::Printf(TEXT("Shouldn't have hit here! FOnlineAsyncTaskGDKPurchaseOfferBase: %s"), *ErrorResult.ToLogString());
}

FOnlineAsyncTaskGDKPurchaseOffer::FOnlineAsyncTaskGDKPurchaseOffer(FOnlineSubsystemGDK* const InGDKInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdGDK, const FOnPurchaseCheckoutComplete& InDelegate)
	: FOnlineAsyncTaskGDKPurchaseOfferBase(InGDKInterface, InStoreId, InGDKUser, InGDKContext, InNetIdGDK, TEXT("FOnlineAsyncTaskGDKPurchaseOffer"))
	, Delegate(InDelegate)
{
}

FString FOnlineAsyncTaskGDKPurchaseOffer::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskGDKPurchaseOffer: %s"), *ErrorResult.ToLogString());
}

void FOnlineAsyncTaskGDKPurchaseOffer::Finalize()
{
	FOnlinePurchaseGDKPtr PurchaseInt = Subsystem->GetPurchaseGDK();
	if (PurchaseInt.IsValid())
	{
		PurchaseInt->bIsCurrentlyInCheckout = false;
	}

	// Query our new entitlements if we succeeded
	if (ErrorResult.bSucceeded)
	{
		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryReceipts>(Subsystem, NetIdGDK, GDKContext, Delegate);
	}
}

void FOnlineAsyncTaskGDKPurchaseOffer::TriggerDelegates()
{
	// Only trigger delegates now if we failed, otherwise our QueryReceipts call will do it for us
	if (!ErrorResult.bSucceeded)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKPurchaseOffer_TriggerDelegates);
		Delegate.ExecuteIfBound(ErrorResult, MakeShared<FPurchaseReceipt>());
	}
}

FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements::FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements(FOnlineSubsystemGDK* const InGDKInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdGDK, const FOnPurchaseReceiptlessCheckoutComplete& InDelegate)
	: FOnlineAsyncTaskGDKPurchaseOfferBase(InGDKInterface, InStoreId, InGDKUser, InGDKContext, InNetIdGDK, TEXT("FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements"))
	, Delegate(InDelegate)
{
}

FString FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements: %s"), *ErrorResult.ToLogString());
}

void FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements::Finalize()
{
	FOnlinePurchaseGDKPtr PurchaseInt = Subsystem->GetPurchaseGDK();
	if (PurchaseInt.IsValid())
	{
		PurchaseInt->bIsCurrentlyInCheckout = false;
	}
}

void FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKPurchaseOffer_TriggerDelegates);
	Delegate.ExecuteIfBound(ErrorResult);
}

#endif //WITH_GRDK