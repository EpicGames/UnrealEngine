// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineError.h"

/**
 * Base Async task to purchase an offer
 */
class FOnlineAsyncTaskGDKPurchaseOfferBase
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKPurchaseOfferBase(FOnlineSubsystemGDK* const InLiveInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdLive, const FString& SubClassName);
	virtual ~FOnlineAsyncTaskGDKPurchaseOfferBase() = default;

	virtual void ProcessResults() override;

	// FOnlineAsyncItem Interface
	virtual FString ToString() const override;

	// FOnlineAsyncTask Interface
	virtual void Tick() override;

protected:
	FString StoreId;
	FUniqueNetIdGDKRef NetIdGDK;
	FOnlineError ErrorResult;

	FGDKContextHandle GDKContext;
	FGDKUserHandle GDKUser;
	bool bTaskStarted = false;
};

/**
 * Async task to purchase an offer and queries receipts to return via Delegate
 */
class FOnlineAsyncTaskGDKPurchaseOffer
	: public FOnlineAsyncTaskGDKPurchaseOfferBase
{
public:
	FOnlineAsyncTaskGDKPurchaseOffer(FOnlineSubsystemGDK* const InLiveInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdLive, const FOnPurchaseCheckoutComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKPurchaseOffer() = default;

	// FOnlineAsyncItem Interface
	virtual FString ToString() const override;

	// Move results and trigger delegates in Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

protected:
	FOnPurchaseCheckoutComplete Delegate;
};

/**
 * Async task to purchase an offer and triggers the delegate with only result, doesn't query receipts
 */
class FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements
	: public FOnlineAsyncTaskGDKPurchaseOfferBase
{
public:
	FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements(FOnlineSubsystemGDK* const InLiveInterface, const FString& InStoreId, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdLive, const FOnPurchaseReceiptlessCheckoutComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements() = default;

	// FOnlineAsyncItem Interface
	virtual FString ToString() const override;

	// Move results and trigger delegates in Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

protected:
	FOnPurchaseReceiptlessCheckoutComplete Delegate;
};
