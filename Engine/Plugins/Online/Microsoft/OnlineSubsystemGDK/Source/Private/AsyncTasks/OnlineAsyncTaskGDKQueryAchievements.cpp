// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryAchievements.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/achievements_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKQueryAchievements::FOnlineAsyncTaskGDKQueryAchievements(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const FUniqueNetIdGDKRef& InUserIdGDK, 
	const FOnQueryGDKAchievementCompleteDelegate& InTaskCompletionDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryAchievements"))
	, UserIdGDK(InUserIdGDK)
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{
}

void FOnlineAsyncTaskGDKQueryAchievements::Initialize()
{
	HRESULT Result = XblAchievementsGetAchievementsForTitleIdAsync(
		GDKContext, 
		UserIdGDK->ToUint64(),			// GDK user Id
		Subsystem->TitleId,					// Title Id to get achievement data for
		XblAchievementType::All,			// AchievementType filter: All mean to get Persistent and Challenge achievements
		false,								// All possible achievements including accurate unlocked data
		XblAchievementOrderBy::TitleId,		// AchievementOrderBy filter: Default means no particular order
		0,									// The number of achievement items to skip
		0,									// The maximum number of achievement items to return in the response
		*AsyncBlock);
	
	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting achievements, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAchievements::ProcessResults()
{
	XblAchievementsResultHandle ResultHandle = nullptr;
	HRESULT Result = XblAchievementsGetAchievementsForTitleIdResult(*AsyncBlock, &ResultHandle);
	if (Result == S_OK)
	{
		ProcessResultInternal(ResultHandle);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting achievement results, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAchievements::ProcessNextResult()
{
	XblAchievementsResultHandle ResultHandle = nullptr;
	HRESULT Result = XblAchievementsResultGetNextResult(*NextResultAsyncBlock, &ResultHandle);
	if (Result == S_OK)
	{
		ProcessResultInternal(ResultHandle);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error getting achievement results, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAchievements::ProcessResultInternal(XblAchievementsResultHandle ResultHandle)
{
	const XblAchievement* AchievementArrayRaw = nullptr;
	uint64 NumResults = 0;
	HRESULT Result = XblAchievementsResultGetAchievements(ResultHandle, &AchievementArrayRaw, &NumResults);
	if (SUCCEEDED(Result))
	{
		for (uint32 i = 0; i < NumResults; ++i)
		{
			const XblAchievement& Achievement = AchievementArrayRaw[i];
			AchievementArray.Add(Achievement);
		}
	}

	bool bHasMoreResults = false;
	XblAchievementsResultHasNext(ResultHandle, &bHasMoreResults);
	if (bHasMoreResults)
	{
		RemoveAsyncBlock(NextResultAsyncBlock);
		NextResultAsyncBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock*)
		{
			ProcessNextResult();
		});

		Result = XblAchievementsResultGetNextAsync(ResultHandle, 0, *NextResultAsyncBlock);
		if (Result != S_OK)
		{
			UE_LOG_ONLINE(Error, TEXT("Error getting achievement results, error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAchievements::TriggerDelegates()
{
	//WMM Todo: Don't like that we copy the array twice. Should pass back a shared ptr to the array.
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryAchievements_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(*UserIdGDK, bWasSuccessful, AchievementArray);
}

#endif //WITH_GRDK