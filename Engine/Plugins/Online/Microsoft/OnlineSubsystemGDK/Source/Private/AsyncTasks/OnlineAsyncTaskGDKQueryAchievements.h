// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineAchievementsInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKQueryAchievements
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryAchievements(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FUniqueNetIdGDKRef& InUserIdGDK,
		const FOnQueryGDKAchievementCompleteDelegate& InDelegate
		);
	
	virtual ~FOnlineAsyncTaskGDKQueryAchievements() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryAchievements"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	
	// Processes in Online Thread
	void ProcessResults();
	
	// Processes in Online Thread
	void ProcessNextResult();

	// Processes in Online Thread
	void ProcessResultInternal(XblAchievementsResultHandle ResultHandle);

	// Trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FUniqueNetIdGDKRef UserIdGDK;
	FGDKContextHandle GDKContext;
	const FOnQueryGDKAchievementCompleteDelegate TaskCompletionDelegate;

	TArray<XblAchievement> AchievementArray;
	FGDKAsyncBlockPtr NextResultAsyncBlock;
};
