// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryActivitiesForUsers.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"

FOnlineAsyncTaskGDKQueryActivitiesForUsers::FOnlineAsyncTaskGDKQueryActivitiesForUsers(FOnlineSubsystemGDK* const InGDKInterface, FGDKContextHandle InGDKContext, const TArray<uint64>& InXUIDs, const FOnGDKQueryActivitiesForUsersComplete& InCompletionDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryActivitiesForUsers"))
	, XUIDs(InXUIDs)
	, CompletionDelegate(InCompletionDelegate)
	, ErrorResponse(false)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryActivitiesForUsers::Initialize()
{
	const char* PrimaryServiceConfigId = nullptr;
	XblGetScid(&PrimaryServiceConfigId);

	HRESULT Result = XblMultiplayerGetActivitiesForUsersAsync(GDKContext, PrimaryServiceConfigId, XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
	
	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying activity session details for users, error: (0x%0.8X)."), Result);
		ErrorResponse = OnlinePresenceGDK::Errors::HResultError(Result);
		bWasSuccessful = ErrorResponse.bSucceeded;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryActivitiesForUsers::ProcessResults()
{
	uint64 NumActivityResults = 0;
	HRESULT Result = XblMultiplayerGetActivitiesForUsersResultCount(*AsyncBlock, &NumActivityResults);
	
	if(Result == S_OK)
	{
		TArray<XblMultiplayerActivityDetails> ActivityArray;
		ActivityArray.Reserve(NumActivityResults);
		Result = XblMultiplayerGetActivitiesForUsersResult(*AsyncBlock, NumActivityResults, ActivityArray.GetData());
		if (Result == S_OK)
		{
			ActivityArray.SetNum(NumActivityResults);
			// Allocate all our ids with an empty result to make sure we return a value for anybody without an activity

			Results.Empty(XUIDs.Num());
			for (uint64 GDKUserId : XUIDs)
			{
				Results.Emplace(FUniqueNetIdGDK::Create(GDKUserId), FUniqueNetIdPtr());
			}

			FOnlineSessionGDKPtr GDKSessionInt = Subsystem->GetSessionInterfaceGDK();
			const int32 FoundActivityCount = ActivityArray.Num();
			for (int32 Index = 0; Index < FoundActivityCount; ++Index)
			{
				const XblMultiplayerActivityDetails& UserActivity = ActivityArray[Index];

				UE_LOG_ONLINE(VeryVerbose, TEXT("Found User Activity:"));
				UE_LOG_ONLINE(VeryVerbose, TEXT("  OwnerXuid: %" UINT64_FMT), UserActivity.OwnerXuid);
				UE_LOG_ONLINE(VeryVerbose, TEXT("  bIsClosed: %d"), UserActivity.Closed);
				UE_LOG_ONLINE(VeryVerbose, TEXT("  HandleId: %ls"), UTF8_TO_TCHAR(UserActivity.HandleId));
				UE_LOG_ONLINE(VeryVerbose, TEXT("  JoinRestriction: %d"), EnumToUnderlyingType(UserActivity.JoinRestriction));
				UE_LOG_ONLINE(VeryVerbose, TEXT("  MembersCount: %d"), UserActivity.MembersCount);
				UE_LOG_ONLINE(VeryVerbose, TEXT("  MaxMembersCount: %d"), UserActivity.MaxMembersCount);
				UE_LOG_ONLINE(VeryVerbose, TEXT("  MultiplayerSessionReference: %ls"), *FOnlineSessionMpsdGDK::SessionReferenceToUri(UserActivity.SessionReference));
				UE_LOG_ONLINE(VeryVerbose, TEXT("  TitleId: %d"), UserActivity.TitleId);
				UE_LOG_ONLINE(VeryVerbose, TEXT("  Visibility: %d"), EnumToUnderlyingType(UserActivity.Visibility));

				const bool bHasEmptySlot = UserActivity.MembersCount < UserActivity.MaxMembersCount;
				const bool bSessionOpen = !UserActivity.Closed;

				const bool bIsJoinable = bHasEmptySlot && bSessionOpen;

				// Set the current activity if one was found and joinable
				if (bIsJoinable)
				{
					FUniqueNetIdPtr* SessionInfo = Results.Find(FUniqueNetIdGDK::Create(UserActivity.OwnerXuid));
					if (ensure(SessionInfo))
					{
						*SessionInfo = FUniqueNetIdString::Create(GDKSessionInt->GetMpsdImpl()->SessionReferenceToUri(UserActivity.SessionReference), GDK_SUBSYSTEM);
					}
				}
			}

			ErrorResponse.bSucceeded = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Error querying friend session details, error: (0x%0.8X)."), Result);
			ErrorResponse = OnlinePresenceGDK::Errors::HResultError(Result);
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend session details, error: (0x%0.8X)."), Result);
		ErrorResponse = OnlinePresenceGDK::Errors::HResultError(Result);
	}
	bWasSuccessful = ErrorResponse.bSucceeded;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKQueryActivitiesForUsers::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryActivitiesForUsers_TriggerDelegates);
	CompletionDelegate.ExecuteIfBound(ErrorResponse, Results);
}

#endif //WITH_GRDK