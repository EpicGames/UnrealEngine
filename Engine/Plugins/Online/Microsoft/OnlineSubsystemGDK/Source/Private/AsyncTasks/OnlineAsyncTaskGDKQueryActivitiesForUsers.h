// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

class FOnlineSubsystemGDK;

using FOnlineGDKActivitiesResultMap = TUniqueNetIdMap<FUniqueNetIdPtr /*JoinableSessionId*/>;
DECLARE_DELEGATE_TwoParams(FOnGDKQueryActivitiesForUsersComplete, const FOnlineError& /*ErrorResult*/, const FOnlineGDKActivitiesResultMap& /*ActivitiesResultMap*/);

/**
 * Async Task to query the session details of friends
 */
class FOnlineAsyncTaskGDKQueryActivitiesForUsers
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryActivitiesForUsers(FOnlineSubsystemGDK* const InGDKInterface,
												FGDKContextHandle InGDKContext,
												const TArray<uint64>& InXUIDs,
												const FOnGDKQueryActivitiesForUsersComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryActivitiesForUsers() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryActivitiesForUsers"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void TriggerDelegates() override;

protected:
	TArray<uint64> XUIDs;
	FOnGDKQueryActivitiesForUsersComplete CompletionDelegate;
	FOnlineError ErrorResponse;
	FOnlineGDKActivitiesResultMap Results;
	FGDKContextHandle GDKContext;
};
