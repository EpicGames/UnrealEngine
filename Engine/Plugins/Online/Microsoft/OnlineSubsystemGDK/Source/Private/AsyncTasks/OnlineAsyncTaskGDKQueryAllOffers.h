// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END

class FOnlineSubsystemGDK;

/**
 * Async item used to marshal store offer results from the GDK system thread to the game thread.
 */
class FOnlineAsyncTaskGDKQueryAllOffers : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryAllOffers(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const FOnQueryOnlineStoreOffersComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryAllOffers();

	virtual void Tick() override;
	virtual void ProcessResults() override;
	
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryAllOffers");}
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	void StartQuery();
	bool EnumerateProductResults(const XStoreProduct* Product);

protected:
	TArray<FOnlineStoreOfferRef> OffersData;
	FOnlineError ErrorResponse;
	FOnQueryOnlineStoreOffersComplete Delegate;
	FGDKContextHandle GDKContext;
	FGDKUserHandle GDKUserHandle;
	bool bHasMoreToQuery;
	bool bTaskStarted = false;
};
