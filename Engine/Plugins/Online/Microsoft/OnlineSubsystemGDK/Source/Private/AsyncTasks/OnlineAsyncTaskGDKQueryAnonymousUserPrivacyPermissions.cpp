// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions::FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions(FOnlineSubsystemGDK* InGDKSubsystem,
																						 FGDKContextHandle InGDKContext,
																						 const TArray<XblAnonymousUserType>& InUserTypesToQuery,
																						 const TArray<XblPermission>& InPermissionsToQuery,
																						 const FOnGDKAnonymousUserPrivacyPermissionsQueryComplete& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions"))
	, UserTypesToQuery(InUserTypesToQuery)
	, PermissionsToQuery(InPermissionsToQuery)
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions::Initialize()
{
	// Ensure we have permissions
	if (PermissionsToQuery.Num() < 1)
	{
		OnlineError.SetFromErrorCode(TEXT("Error starting anonymous user privacy permissions query, no permissions were requested"));
		UE_LOG_ONLINE(Warning, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	// Ensure we have some user types
	if (UserTypesToQuery.Num() < 1)
	{
		OnlineError.SetFromErrorCode(TEXT("Error starting anonymous user privacy permissions query, no anonymous user types were requested"));
		UE_LOG_ONLINE(Warning, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	NumTasksRemaining = UserTypesToQuery.Num() * PermissionsToQuery.Num();

	for (XblAnonymousUserType UserType : UserTypesToQuery)
	{
		for (XblPermission Permission : PermissionsToQuery)
		{
			if (NumTasksFailed > 0)
			{
				NumTasksRemaining--;
				continue;
			}

			FGDKAsyncBlockPtr TaskBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock)
			{
				Subsystem->ExecuteNextTick([this, LambdaAsyncBlock]()
				{
					NumTasksRemaining--;

					uint64 ResultSizeInBytes = 0;
					HRESULT Result = XblPrivacyCheckPermissionForAnonymousUserResultSize(*LambdaAsyncBlock, &ResultSizeInBytes);
					if (Result == S_OK)
					{
						TArray<uint8> ResultBuffer;
						XblPermissionCheckResult* PermissionResult = nullptr;
						ResultBuffer.Reserve(ResultSizeInBytes);
						Result = XblPrivacyCheckPermissionForAnonymousUserResult(*LambdaAsyncBlock, ResultSizeInBytes, ResultBuffer.GetData(), &PermissionResult, nullptr);
						if (Result == S_OK)
						{
							// Read our record
							const XblAnonymousUserType UserType = PermissionResult->targetUserType;
							const XblPermission Permission = PermissionResult->permissionRequested;
							const bool bIsAllowed = PermissionResult->isAllowed;

							TMap<XblPermission, bool>& PermissionsMap = ResultPermissionsMap.FindOrAdd(UserType);
							PermissionsMap.Add(Permission, bIsAllowed);
						}
						else
						{
							OnlineError.SetFromErrorCode(FString::Printf(TEXT("Error querying user privacy permissions, error: (0x%0.8X)."), Result));
							UE_LOG_ONLINE(Error, TEXT("%s"), *OnlineError.ErrorCode);

							NumTasksFailed++;
						}
					}
					else
					{
						OnlineError.SetFromErrorCode(FString::Printf(TEXT("Error querying user privacy permissions, error: (0x%0.8X)."), Result));
						UE_LOG_ONLINE(Error, TEXT("%s"), *OnlineError.ErrorCode);

						NumTasksFailed++;
					}

					if (NumTasksRemaining == 0)
					{
						bWasSuccessful = NumTasksFailed == 0;
						if (bWasSuccessful)
						{
							OnlineError.bSucceeded = true;
						}
						else
						{
							ResultPermissionsMap.Reset();
						}
						bIsComplete = true;
					}
				});
			});

			// Create our task
			HRESULT Result = XblPrivacyCheckPermissionForAnonymousUserAsync(GDKContext, Permission, UserType, *TaskBlock);
			if (Result != S_OK)
			{
				UE_LOG_ONLINE(Error, TEXT("Error starting user privacy permissions query, error: (0x%0.8X)."), Result);
				NumTasksRemaining--;
				NumTasksFailed++;
			}
		}
	}

	if (NumTasksRemaining == 0)
	{
		bWasSuccessful = NumTasksFailed == 0;
		if (bWasSuccessful)
		{
			OnlineError.bSucceeded = true;
		}
		else
		{
			ResultPermissionsMap.Reset();
		}
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions_TriggerDelegates);
	Delegate.ExecuteIfBound(OnlineError, ResultPermissionsMap);
}

#endif //WITH_GRDK