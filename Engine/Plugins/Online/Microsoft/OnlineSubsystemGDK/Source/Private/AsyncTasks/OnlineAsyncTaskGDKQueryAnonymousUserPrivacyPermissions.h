// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/privacy_c.h>
THIRD_PARTY_INCLUDES_END

class FOnlineSubsystemGDK;

/** Map of Class's Permission's Allowed status */
using FAnonymousUserPrivacyPermissionsResultsMap = TMap<XblAnonymousUserType, TMap<XblPermission, bool> >;

/**
 * Delegate used when the class's privacy permissions request has completed
 *
 * @param RequestStatus If this request was successful or not
 * @param Results A Map of Users to permission results
 */
DECLARE_DELEGATE_TwoParams(FOnGDKAnonymousUserPrivacyPermissionsQueryComplete, const FOnlineError& /*RequestStatus*/, const FAnonymousUserPrivacyPermissionsResultsMap& /*Results*/);

/**
 * Async Task to query the account details of a user
 */
class FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions(FOnlineSubsystemGDK* InGDKSubsystem,
								   FGDKContextHandle InGDKContext,
								   const TArray<XblAnonymousUserType>& InUserTypesToQuery,
								   const TArray<XblPermission>& InPermissionsToQuery,
								   const FOnGDKAnonymousUserPrivacyPermissionsQueryComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions"); }

	virtual void Initialize() override;

	virtual void TriggerDelegates();

protected:
	TArray<XblAnonymousUserType> UserTypesToQuery;
	TArray<XblPermission> PermissionsToQuery;
	FOnGDKAnonymousUserPrivacyPermissionsQueryComplete Delegate;
	FOnlineError OnlineError;
	FAnonymousUserPrivacyPermissionsResultsMap ResultPermissionsMap;
	FGDKContextHandle GDKContext;

	int NumTasksRemaining = 0;
	int NumTasksFailed = 0;
};
