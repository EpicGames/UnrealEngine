// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryAvoidList.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKQueryAvoidList::FOnlineAsyncTaskGDKQueryAvoidList(FOnlineSubsystemGDK* InGDKInterface, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InUserIdGDK)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryAvoidList"))
	, UserIdGDK(InUserIdGDK)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryAvoidList::Initialize()
{
	HRESULT Result = XblPrivacyGetAvoidListAsync(GDKContext, *AsyncBlock);

	if(Result != S_OK)
	{
		OutError = FString::Printf(TEXT("Error querying block list, error: (0x%0.8X)."), Result);
		UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAvoidList::ProcessResults()
{
	uint64 ResultCount = 0;
	HRESULT Result = XblPrivacyGetAvoidListResultCount(*AsyncBlock, &ResultCount);
	if (Result == S_OK)
	{
		if (ResultCount > 0)
		{
			uint64* AvoidListResults = new uint64[ResultCount];
			Result = XblPrivacyGetAvoidListResult(*AsyncBlock, ResultCount, AvoidListResults);
			if (Result == S_OK)
			{
				for (uint64 i = 0; i < ResultCount; ++i)
				{
					uint64 AvoidId = AvoidListResults[i];

					AvoidList.Emplace(MakeShared<FOnlineBlockedPlayerGDK>(AvoidId));
				}

				bWasSuccessful = true;
				bIsComplete = true;
			}
			else
			{
				OutError = FString::Printf(TEXT("Error querying block list, error: (0x%0.8X)."), Result);
				UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);
				bWasSuccessful = false;
				bIsComplete = true;
			}
			delete[] AvoidListResults;
		}
		else
		{
			bWasSuccessful = true;
			bIsComplete = true;
		}
		
	}
	else
	{
		OutError = FString::Printf(TEXT("Error querying block list size, error: (0x%0.8X)."), Result);
		UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryAvoidList::Finalize()
{
	if (bWasSuccessful)
	{
		Subsystem->GetFriendsGDK()->AvoidListMap.Emplace(UserIdGDK, MoveTemp(AvoidList));
	}
}

void FOnlineAsyncTaskGDKQueryAvoidList::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryAvoidList_TriggerDelegates);
	Subsystem->GetFriendsGDK()->TriggerOnQueryBlockedPlayersCompleteDelegates(*UserIdGDK, bWasSuccessful, OutError);
}

#endif //WITH_GRDK