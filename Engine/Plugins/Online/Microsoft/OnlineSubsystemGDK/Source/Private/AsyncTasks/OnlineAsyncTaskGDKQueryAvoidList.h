// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineFriendsInterfaceGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/privacy_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

/**
 * Async task to query our avoid list
 */
class FOnlineAsyncTaskGDKQueryAvoidList
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryAvoidList(FOnlineSubsystemGDK* InGDKInterface, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InUserIdGDK);
	virtual ~FOnlineAsyncTaskGDKQueryAvoidList() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryAvoidList"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

protected:
	FUniqueNetIdGDKRef UserIdGDK;

	TArray<TSharedRef<FOnlineBlockedPlayerGDK>> AvoidList;
	FString OutError;
	FGDKContextHandle GDKContext;
};
