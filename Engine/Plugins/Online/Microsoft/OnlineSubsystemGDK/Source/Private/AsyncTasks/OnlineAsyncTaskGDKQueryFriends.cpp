// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryFriends.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineFriendsInterfaceGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "Async/Async.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

extern TAutoConsoleVariable<bool> CVarXboxMpaEnabled;

FOnlineAsyncTaskGDKQueryFriends::FOnlineAsyncTaskGDKQueryFriends(FOnlineSubsystemGDK* const InGDKInterface,
																   FGDKContextHandle InGDKContext,
																   const int32 InLocalUserNum,
																   const FUniqueNetIdGDKRef& InGDKUniqueNetId,
																   const FString& InListName)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryFriends"))
	, LocalUserNum(InLocalUserNum)
	, GDKUniqueNetId(InGDKUniqueNetId)
	, ListName(InListName)
	, FoundItemCount(0)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryFriends::Initialize()
{
	HRESULT Result = XblSocialGetSocialRelationshipsAsync(GDKContext, GDKUniqueNetId->ToUint64(), XblSocialRelationshipFilter::All, 0, 0, *AsyncBlock);
	if(Result != S_OK)
	{
		OutError = FString::Printf(TEXT("Error querying friends, error: (0x%0.8X)"), Result);
		UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriends::ProcessResults()
{
	XblSocialRelationshipResultHandle RelationshipHandle = nullptr;

	HRESULT Result = XblSocialGetSocialRelationshipsResult(*AsyncBlock, &RelationshipHandle);
	if(Result != S_OK)
	{
		OutError = FString::Printf(TEXT("Error querying friends, error: (0x%0.8X)."), Result);
		UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	ProcessNextPage(RelationshipHandle);	
}

void FOnlineAsyncTaskGDKQueryFriends::GetNextPage(XblSocialRelationshipResultHandle RelationshipHandle)
{
	RemoveAsyncBlock(PagesAsyncBlock);
	PagesAsyncBlock = CreateAsyncBlock(nullptr, [this, RelationshipHandle](FGDKAsyncBlock* Block) mutable
		{
			XblSocialRelationshipResultCloseHandle(RelationshipHandle);
			HRESULT Result = XblSocialRelationshipResultGetNextResult(*Block, &RelationshipHandle);
			ProcessNextPage(RelationshipHandle);			
		});	

	HRESULT Result = XblSocialRelationshipResultGetNextAsync(GDKContext, RelationshipHandle, 0, *PagesAsyncBlock);
	if (FAILED(Result))
	{
		XblSocialRelationshipResultCloseHandle(RelationshipHandle);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
}

void FOnlineAsyncTaskGDKQueryFriends::ProcessNextPage(XblSocialRelationshipResultHandle RelationshipHandle)
{
	const XblSocialRelationship* SocialRelationships = nullptr;

	size_t ItemCount = 0;
	HRESULT Result = XblSocialRelationshipResultGetRelationships(RelationshipHandle, &SocialRelationships, &ItemCount);
	if (Result != S_OK)
	{
		OutError = FString::Printf(TEXT("Error querying friends, error: (0x%0.8X)."), Result);
		UE_LOG_ONLINE(Error, TEXT("%s"), *OutError);
		XblSocialRelationshipResultCloseHandle(RelationshipHandle);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}
	FoundItemCount = ItemCount;

	// extend friends list
	FriendsListMap.Reserve(FriendsListMap.Num()+FoundItemCount);

	for (int32 Index = 0; Index < FoundItemCount; ++Index)
	{
		const XblSocialRelationship& FriendItem = SocialRelationships[Index];
		TSharedRef<FOnlineFriendGDK> GDKFriend = MakeShared<FOnlineFriendGDK>(FriendItem);
		FriendsListMap.Emplace(GDKFriend->GetUserId(), GDKFriend);
	}

	bool bHasNext = false;
	Result = XblSocialRelationshipResultHasNext(RelationshipHandle, &bHasNext);
	if (FAILED(Result) || !bHasNext)
	{
		XblSocialRelationshipResultCloseHandle(RelationshipHandle);
		bWasSuccessful = true;
		bIsComplete = true;
		return;
	}
	GetNextPage(RelationshipHandle);
}

void FOnlineAsyncTaskGDKQueryFriends::Finalize()
{
	if (!bWasSuccessful)
	{
		return;
	}
	FOnlineFriendsGDKPtr FriendsInt = Subsystem->GetFriendsGDK();
	check(FriendsInt.IsValid());
	FOnlineFriendsListGDKMap& CurrentFriends = FriendsInt->FriendsMap.FindOrAdd(GDKUniqueNetId);
	for (const TPair<FUniqueNetIdRef, TSharedRef<FOnlineFriendGDK> >& FriendPair : CurrentFriends)
	{
		if(FriendsListMap.Find(FriendPair.Key) == nullptr)
		{
			FriendsInt->TriggerOnFriendRemovedDelegates(*GDKUniqueNetId, *FriendPair.Key);
		}
	}

	if (FoundItemCount > 0)
	{
		if (FOnlineAsyncTaskManagerGDK* const MyTaskManager = Subsystem->GetAsyncTaskManager())
		{
			const FOnlinePresenceGDKPtr PresencePtr = Subsystem->GetPresenceGDK();
			TArray<FUniqueNetIdGDKRef> UnsubscribedGDKUsers;
			TArray<uint64> UnsubscribedGDKUserXuids;

			// Sort favourites to the front of the list
			FriendsListMap.ValueSort([](const TSharedRef<FOnlineFriendGDK>& FriendA, const TSharedRef<FOnlineFriendGDK>& FriendB)
				{
					return static_cast<int32>(FriendA->IsFavorite()) >= static_cast<int32>(FriendB->IsFavorite());
				});

			// Build list of XUIDs to send to GDK for more information
			for (const TPair<FUniqueNetIdRef, TSharedRef<FOnlineFriendGDK> >& FriendPair : FriendsListMap)
			{
				TSharedRef<FOnlineFriendGDK> FriendRef = StaticCastSharedRef<FOnlineFriendGDK>(FriendPair.Value);
				uint64 UserToQueryXuid = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendPair.Key)->ToUint64();
				XUIDs.Add(UserToQueryXuid);

				if (PresencePtr.IsValid())
				{
					FUniqueNetIdGDKRef GDKFriendId = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendPair.Key);
					if (!PresencePtr->IsSubscribedToPresenceUpdates(GDKFriendId))
					{
						// If we are not subscribed to presence updates for any of our friends, we'll do that next
						UnsubscribedGDKUsers.Add(GDKFriendId);
						UnsubscribedGDKUserXuids.Add(UserToQueryXuid);
					}
				}
			}

			// Subscribe to presence updates
			if (UnsubscribedGDKUsers.Num() > 0)
			{
				// The following call may be blocking so we'll run it out of the game thread
				AsyncTask(ENamedThreads::AnyThread, [PresencePtr, GDKContext = FGDKContextHandle(GDKContext), UnsubscribedGDKUsers, UnsubscribedGDKUserXuids]()
				{
					HRESULT Result = XblPresenceTrackUsers(GDKContext, UnsubscribedGDKUserXuids.GetData(), UnsubscribedGDKUserXuids.Num());
					if (Result == S_OK)
					{
						PresencePtr->AddPresenceUpdateSubscriptions(UnsubscribedGDKUsers);
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("[FOnlineAsyncTaskGDKQueryFriends::Finalize] Error from XblPresenceTrackUsers, with code 0x%08X."), Result);
					}
				});
			}

			// Task to manage calling delegates and updating the local cache after the below tasks are finished
			FOnlineAsyncTaskGDKQueryFriendManagerTask* ManagerTask = new FOnlineAsyncTaskGDKQueryFriendManagerTask(Subsystem, MoveTemp(FriendsListMap), LocalUserNum, GDKUniqueNetId, MoveTemp(ListName));
			{
				// Request Account Information (DisplayName / Potentially Icon information in the future)
				Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriendAccountDetails>(Subsystem, GDKContext, XUIDs, *ManagerTask);

				// Request Presence Information
				Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriendPresenceDetails>(Subsystem, GDKContext, XUIDs, *ManagerTask);

				// Request Session Information
				Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriendSessionDetails>(Subsystem, GDKContext, XUIDs, *ManagerTask);
				
				// Request presence stats information if any stats were configured
				TArray<FString> PresenceStatNames = FOnlinePresenceGDK::GetAutoSubscribePresenceStatNames();
				if (PresenceStatNames.Num() > 0)
				{
					Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriendPresenceStats>(Subsystem, GDKContext, XUIDs, 0, MoveTemp(PresenceStatNames), *ManagerTask);
				}
				else
				{
					// Nothing to do if no stats are needed, no need to create a task for it.
					ManagerTask->PresenceStatsStatus = EOnlineAsyncTaskState::Done;
				}
			}
			// Add Manager Last so it gets ticked after the tasks
			MyTaskManager->AddToParallelTasks(ManagerTask);
		}
		else
		{
			bWasSuccessful = false;
			OutError = FString(TEXT("FOnlineAsyncTaskLiveQueryFriends Could not load AsyncTaskManager"));
		}
	}
	else
	{
		// We didn't find anything, so we don't need to query any additional info.  Just clear the map for this user
		FriendsInt->FriendsMap.FindOrAdd(GDKUniqueNetId).Empty();
	}
}

void FOnlineAsyncTaskGDKQueryFriends::TriggerDelegates()
{
	FOnlineFriendsGDKPtr FriendsInt = Subsystem->GetFriendsGDK();
	check(FriendsInt.IsValid());

	// If we successfully found nothing, we need to call our delegates now since we're done
	if (bWasSuccessful && FoundItemCount == 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriends_TriggerDelegates_OnFriendsChange);
		FriendsInt->TriggerOnFriendsChangeDelegates(LocalUserNum);
	}

	if (!bWasSuccessful || FoundItemCount == 0)
	{
		TArray<FOnReadFriendsListComplete> PendingFriendDelegates = MoveTemp(FriendsInt->FriendsListInProgressDelegates.FindOrAdd(GDKUniqueNetId));

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriends_TriggerDelegates_PendingFriends_Loop);
		for (const FOnReadFriendsListComplete& Delegate : PendingFriendDelegates)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriends_TriggerDelegates_PendingFriends_Delegate);
			Delegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, ListName, OutError);
		}
	}
}

void FOnlineAsyncTaskGDKQueryFriendManagerTask::Tick()
{
	if ((AccountDetailsStatus == EOnlineAsyncTaskState::Done || AccountDetailsStatus == EOnlineAsyncTaskState::Failed)
		&& (PresenceDetailsStatus == EOnlineAsyncTaskState::Done || PresenceDetailsStatus == EOnlineAsyncTaskState::Failed)
		&& (PresenceStatsStatus == EOnlineAsyncTaskState::Done || PresenceStatsStatus == EOnlineAsyncTaskState::Failed)
		&& (SessionDetailsStatus == EOnlineAsyncTaskState::Done || SessionDetailsStatus == EOnlineAsyncTaskState::Failed))
	{
		bWasSuccessful = (AccountDetailsStatus == EOnlineAsyncTaskState::Done &&
			PresenceDetailsStatus == EOnlineAsyncTaskState::Done &&
			PresenceStatsStatus == EOnlineAsyncTaskState::Done &&
			SessionDetailsStatus == EOnlineAsyncTaskState::Done);
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendManagerTask::Finalize()
{
	if (bWasSuccessful)
	{
		FUniqueNetIdGDKPtr UserId = StaticCastSharedPtr<const FUniqueNetIdGDK>(Subsystem->GetIdentityGDK()->GetUniquePlayerId(LocalUserNum));
		if (UserId.IsValid())
		{
			FOnlinePresenceGDKPtr PresenceInt = Subsystem->GetPresenceGDK();
			if (PresenceInt.IsValid())
			{
				// Copy our friends' presence into our presence cache
				for (const FOnlineFriendsListGDKMap::ElementType& FriendPair : FriendsListMap)
				{
					const FOnlineUserPresenceGDK& PresenceToCopy = static_cast<const FOnlineUserPresenceGDK&>(FriendPair.Value->GetPresence());
					PresenceInt->PresenceCache.Add(FriendPair.Key, MakeShared<FOnlineUserPresenceGDK>(PresenceToCopy));
				}
			}

			FOnlineFriendsGDKPtr FriendsInt = Subsystem->GetFriendsGDK();
			if (FriendsInt.IsValid())
			{
				FriendsInt->FriendsMap.Emplace(UserId.ToSharedRef(), MoveTemp(FriendsListMap));
			}
			// FriendsListMap is now empty here
		}
		else
		{
			bWasSuccessful = false;
		}
	}
}

void FOnlineAsyncTaskGDKQueryFriendManagerTask::TriggerDelegates()
{
	FString ErrorString;

	if (!bWasSuccessful)
	{
		if (AccountDetailsStatus == EOnlineAsyncTaskState::Failed)
		{
			ErrorString += TEXT("Failed to query account details. ");
		}
		if (PresenceDetailsStatus == EOnlineAsyncTaskState::Failed)
		{
			ErrorString += TEXT("Failed to query presence details. ");
		}
		if (PresenceStatsStatus == EOnlineAsyncTaskState::Failed)
		{
			ErrorString += TEXT("Failed to query presence stats. ");
		}
		if (SessionDetailsStatus == EOnlineAsyncTaskState::Failed)
		{
			ErrorString += TEXT("Failed to query session details. ");
		}

		if (ErrorString.IsEmpty())
		{
			ErrorString = TEXT("An unknown error has occured");
		}
	}

	// Call delegates
	FOnlineFriendsGDKPtr FriendsInt = Subsystem->GetFriendsGDK();
	check(FriendsInt.IsValid());

	TArray<FOnReadFriendsListComplete> PendingFriendDelegates = MoveTemp(FriendsInt->FriendsListInProgressDelegates.FindOrAdd(GDKUniqueNetId));
	FriendsInt->FriendsListInProgressDelegates.FindChecked(GDKUniqueNetId).Reset();

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriendManagerTask_TriggerDelegates_ReadFriendsListComplete_Loop);
		for (const FOnReadFriendsListComplete& Delegate : PendingFriendDelegates)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriendManagerTask_TriggerDelegates_ReadFriendsListComplete_Delegate);
			Delegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, ListName, ErrorString);
		}
	}
	if (bWasSuccessful)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryFriendManagerTask_TriggerDelegates_OnFriendsChange);
		FriendsInt->TriggerOnFriendsChangeDelegates(LocalUserNum);
	}
}

FOnlineAsyncTaskGDKQueryFriendAccountDetails::FOnlineAsyncTaskGDKQueryFriendAccountDetails(FOnlineSubsystemGDK* const InGDKInterface,
																							 FGDKContextHandle InGDKContext,
																							 const TArray<uint64> InXUIDs,
																							 FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryFriendAccountDetails"))
	, XUIDs(InXUIDs)
	, ManagerTask(InManagerTask)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryFriendAccountDetails::Initialize()
{
	HRESULT Result = XblProfileGetUserProfilesAsync(GDKContext, (uint64*)XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
		
	if(Result == S_OK)
	{
		ManagerTask.AccountDetailsStatus = EOnlineAsyncTaskState::InProgress;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend account details for friends list, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendAccountDetails::ProcessResults()
{
	size_t NumProfiles = 0;
	HRESULT Result = XblProfileGetUserProfilesResultCount(*AsyncBlock, &NumProfiles);
	
	if(Result == S_OK)
	{
		TArray<XblUserProfile> UserProfiles;
		UserProfiles.Reserve(NumProfiles);
		Result = XblProfileGetUserProfilesResult(*AsyncBlock, NumProfiles, UserProfiles.GetData());
		if (Result == S_OK)
		{
			UserProfiles.SetNum(NumProfiles);

			for (XblUserProfile& Profile : UserProfiles)
			{
				TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FUniqueNetIdGDK::Create(Profile.xboxUserId));

				FoundFriend->DisplayName = FOnlineUserInfoGDK::FilterPlayerName(FOnlineUserInfoGDK::GetGamertag(Profile));
				FoundFriend->UserAttributes.Emplace(FString(TEXT("Gamerscore")), FString(UTF8_TO_TCHAR(Profile.gamerscore)));
				// This is a the URI to a resizeable display image for the user.  For example, &format=png&w=64&h=64
				// Valid Format: png
				// Valid Width/Height: 64/64, 208/208, or 424/424
				FoundFriend->UserAttributes.Emplace(FString(TEXT("DisplayPictureUri")), FString(UTF8_TO_TCHAR(Profile.gameDisplayPictureResizeUri)));
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Error querying friend account details, error: (0x%0.8X)."), Result);
			bWasSuccessful = true;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend account details, error: (0x%0.8X)."), Result);
		bWasSuccessful = true;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendAccountDetails::Finalize()
{
	ManagerTask.AccountDetailsStatus = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
}

FOnlineAsyncTaskGDKQueryFriendPresenceDetails::FOnlineAsyncTaskGDKQueryFriendPresenceDetails(FOnlineSubsystemGDK* const InGDKInterface,
																							   FGDKContextHandle InGDKContext,
																							   const TArray<uint64> InXUIDs,
																							   FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryFriendPresenceDetails"))
	, XUIDs(InXUIDs)
	, ManagerTask(InManagerTask)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryFriendPresenceDetails::Initialize()
{	
	Filter.detailLevel = XblPresenceDetailLevel::All;
	HRESULT Result = XblPresenceGetPresenceForMultipleUsersAsync(GDKContext,(uint64*) XUIDs.GetData(), XUIDs.Num(), &Filter, *AsyncBlock);
	if(Result == S_OK)
	{
		//TODO WMM: ensure we get the detailed presence
		// The GetPresenceForMultipleUsersAsync(XUIDsVectorView) overload of this function should have the same defaults for the other parameters according to the documentation, but
		// doesn't return the detailed presence for some reason. This overload works however.
		ManagerTask.PresenceDetailsStatus = EOnlineAsyncTaskState::InProgress;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend presence details for friends list, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;

	}
}

void FOnlineAsyncTaskGDKQueryFriendPresenceDetails::ProcessResults()
{
	size_t NumResults = 0;
	HRESULT Result = XblPresenceGetPresenceForMultipleUsersResultCount(*AsyncBlock, &NumResults);
	if (Result == S_OK)
	{		
		XblPresenceRecordHandle* PresenceRecordHandles = new XblPresenceRecordHandle[NumResults];
		Result = XblPresenceGetPresenceForMultipleUsersResult(*AsyncBlock, PresenceRecordHandles, NumResults);
	
		if (Result == S_OK)
		{		
			bool bAllUsersFound = true;

			for (uint64 i = 0; i < NumResults; ++i)
			{
				const XblPresenceRecordHandle& PresenceRecordHandle = PresenceRecordHandles[i];

				FGDKPresenceRecordHandle PresenceHandle = FGDKPresenceRecordHandle(PresenceRecordHandle);

				// Only valid users get checked
				uint64 UserXuid;
				if (SUCCEEDED(XblPresenceRecordGetXuid(PresenceHandle, &UserXuid)))
				{
					TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FUniqueNetIdGDK::Create(UserXuid));

					// Not clobbering the presence value using the FOnlineUserPresenceLive constructor here
					// since the stats task may finish first and if it does we don't want to overwrite its results.
					FoundFriend->Presence.SetPresenceFromPresenceRecord(PresenceHandle);
				}
				else
				{
					bAllUsersFound = false;
				}
				XblPresenceRecordCloseHandle(PresenceRecordHandle);
			}

			bWasSuccessful = bAllUsersFound;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Error querying friend presence details, error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
		delete[] PresenceRecordHandles;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend presence details, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendPresenceDetails::Finalize()
{
	if (bWasSuccessful)
	{
		if (XUIDs.Num() > 0)
		{
			// Register for stat updates from each friend
			FOnlinePresenceGDKPtr PresenceInterface(Subsystem->GetPresenceGDK());
			if (PresenceInterface.IsValid())
			{
				uint64 LocalGDKUserId;
				if (SUCCEEDED(XblContextGetXboxUserId(GDKContext, &LocalGDKUserId)))
				{
					const FUniqueNetIdGDKRef LocalUserId = FUniqueNetIdGDK::Create(LocalGDKUserId);
				
					for (uint64 FriendXUID : XUIDs)
					{
						const FUniqueNetIdGDKRef FriendNetId = FUniqueNetIdGDK::Create(FriendXUID);

						// Only update if our friend is online/playing the same title
						TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FriendNetId);
						if (FoundFriend->Presence.bIsOnline && FoundFriend->Presence.bIsPlayingThisGame)
						{
							PresenceInterface->EstablishDefaultPresenceStatSubscriptions(LocalUserId, FriendNetId);
						}
					}
				}
			}
		}

		ManagerTask.PresenceDetailsStatus = EOnlineAsyncTaskState::Done;
	}
	else
	{
		ManagerTask.PresenceDetailsStatus = EOnlineAsyncTaskState::Failed;
	}
}

FOnlineAsyncTaskGDKQueryFriendPresenceStats::FOnlineAsyncTaskGDKQueryFriendPresenceStats(FOnlineSubsystemGDK* const InGDKInterface,
																						   FGDKContextHandle InGDKContext,
																						   const TArray<uint64> InXUIDs,
																						   const int32 InStartIndex,
																						   const TArray<FString>&& InPresenceStatNames,
																						   FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryFriendPresenceStats"))
	, StartIndex(InStartIndex)
	, NextRequestIndex(FMath::Min<int32>(InXUIDs.Num(), InStartIndex + MAX_USER_QUERY_COUNT))
	, XUIDs(InXUIDs)
	, PresenceStatNames(InPresenceStatNames)
	, ManagerTask(InManagerTask)
	, GDKContext(InGDKContext)
{
	for (const FString& StatName : PresenceStatNames)
	{
		//Array to store Ansi strings, that will be cleaned up when task is destroyed (Avoiding explicit mallocs)
		TArray<ANSICHAR>& AnsiCharArray = PresenceStatNamesAnsiChar.AddDefaulted_GetRef();
		AnsiCharArray.Append(TCHAR_TO_ANSI(*StatName), StatName.Len() + 1);

		//Array to store char* that we can pass to GDK
		PresenceStatNamesCharPtr.Add(AnsiCharArray.GetData());
	}
}

void FOnlineAsyncTaskGDKQueryFriendPresenceStats::Initialize()
{
	// We need to batch our queries as to not exceed the maximum amount of users per request, so we're rebuilding our smaller xuid view here	
	TArray<uint64> ThisRequestUsers(&XUIDs[StartIndex], NextRequestIndex - StartIndex);
	UE_LOG_ONLINE(Verbose, TEXT("Starting request batch of %d users, from index %d up until %d"), NextRequestIndex - StartIndex, StartIndex, NextRequestIndex);

	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);

	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsAsync(
		GDKContext,
		ThisRequestUsers.GetData(),
		ThisRequestUsers.Num(),
		Scid,
		PresenceStatNamesCharPtr.GetData(),
		PresenceStatNamesCharPtr.Num(),
		*AsyncBlock);

	if(Result == S_OK)
	{
		ManagerTask.PresenceStatsStatus = EOnlineAsyncTaskState::InProgress;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend presence stats for friends list, error: (0x%0.8X)"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendPresenceStats::ProcessResults()
{
	uint64 ResultSizeInBytes = 0;
	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsResultSize(*AsyncBlock, &ResultSizeInBytes);
	XblUserStatisticsResult* UserStatistics = nullptr;
	if (Result == S_OK)
	{
		uint64 NumResults = 0;
		TArray<uint8> BufferArray;
		BufferArray.Reserve(ResultSizeInBytes);
		Result = XblUserStatisticsGetMultipleUserStatisticsResult(*AsyncBlock, ResultSizeInBytes, BufferArray.GetData(), &UserStatistics, &NumResults, nullptr);
		if (Result == S_OK)
		{
			for (uint64 Index = 0; Index < NumResults; ++Index)
			{
				const XblUserStatisticsResult& GDKStats = UserStatistics[Index];

				// Add the returned stats to the friend's presence properties.
				TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FUniqueNetIdGDK::Create(GDKStats.xboxUserId));
				FoundFriend->Presence.SetStatusPropertiesFromStatistics(GDKStats);
			}

			// If we're not finished querying all of our users, request the next batch here
			const bool bIsFinishedProcessingAllFriends = NextRequestIndex >= static_cast<int32>(XUIDs.Num());
			if (!bIsFinishedProcessingAllFriends)
			{
				UE_LOG_ONLINE(Verbose, TEXT("Queuing next batch of user-statistic queries, starting at index %d"), NextRequestIndex);
				Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriendPresenceStats>(Subsystem, GDKContext, XUIDs, NextRequestIndex, MoveTemp(PresenceStatNames), ManagerTask);
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Error, TEXT("Error querying friend presence stats, error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend presence stats, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendPresenceStats::Finalize()
{
	if (bWasSuccessful)
	{
		const bool bIsFinishedProcessingAllFriends = NextRequestIndex >= static_cast<int32>(XUIDs.Num());
		if (bIsFinishedProcessingAllFriends)
		{
			ManagerTask.PresenceStatsStatus = EOnlineAsyncTaskState::Done;
		}
		else
		{
			// We're processing the next batch of users, so we don't end yet
		}
	}
	else
	{
		ManagerTask.PresenceStatsStatus = EOnlineAsyncTaskState::Failed;
	}
}

FOnlineAsyncTaskGDKQueryFriendSessionDetails::FOnlineAsyncTaskGDKQueryFriendSessionDetails(FOnlineSubsystemGDK* const InGDKInterface,
																							 FGDKContextHandle InGDKContext,
																							 const TArray<uint64> InXUIDs,
																							 FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryFriendSessionDetails"))
	, XUIDs(InXUIDs)
	, ManagerTask(InManagerTask)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryFriendSessionDetails::Initialize()
{
	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);
	HRESULT Result = S_OK;
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		Result = XblMultiplayerActivityGetActivityAsync(GDKContext, (uint64*)XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
	}
	else
	{
		Result = XblMultiplayerGetActivitiesForUsersAsync(GDKContext, Scid, (uint64*)XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
	}

	if(Result == S_OK)
	{
		ManagerTask.SessionDetailsStatus = EOnlineAsyncTaskState::InProgress;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend session details for friends list, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryFriendSessionDetails::ProcessResults()
{
	uint64 NumActivities = 0;
	uint64 ResultSizeInBytes = 0;
	TArray<uint8> BufferArray;
	XblMultiplayerActivityInfo* MPAActivities = nullptr;
	XblMultiplayerActivityDetails* MPSDActivities = nullptr;
	HRESULT Result = S_OK;
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		Result = XblMultiplayerActivityGetActivityResultSize(*AsyncBlock, &ResultSizeInBytes);
		if(SUCCEEDED(Result))
		{
			BufferArray.Reserve(ResultSizeInBytes);
			Result = XblMultiplayerActivityGetActivityResult(*AsyncBlock, ResultSizeInBytes, BufferArray.GetData(), &MPAActivities, &NumActivities, nullptr);
		}
	}
	else
	{
		Result = XblMultiplayerGetActivitiesForUsersResultCount(*AsyncBlock, &NumActivities);
		if (SUCCEEDED(Result) && NumActivities > 0)
		{
			MPSDActivities = new XblMultiplayerActivityDetails[NumActivities];
			Result = XblMultiplayerGetActivitiesForUsersResult(*AsyncBlock, NumActivities, MPSDActivities);
		}
	}
	
	
	if (SUCCEEDED(Result))
	{
		if (NumActivities > 0)
		{
			for (uint64 i = 0; i < NumActivities; ++i)
			{
				if (CVarXboxMpaEnabled.GetValueOnAnyThread())
				{
					const XblMultiplayerActivityInfo& UserActivity = MPAActivities[i];
					TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FUniqueNetIdGDK::Create(UserActivity.xuid));

					UE_LOG_ONLINE(VeryVerbose, TEXT("Found Friend User Activity:"));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  OwnerXboxUserId: %lld"), UserActivity.xuid);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  ConnectString: %ls"), UTF8_TO_TCHAR(UserActivity.connectionString));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  JoinRestriction: %ld"), EnumToUnderlyingType(UserActivity.joinRestriction));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  CurrentPlayers: %lld"), UserActivity.currentPlayers);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  MaxPlayers: %lld"), UserActivity.maxPlayers);


					FoundFriend->Presence.SessionId = FUniqueNetIdString::Create(FString(UserActivity.connectionString), GDK_SUBSYSTEM);
					FoundFriend->Presence.bIsJoinable = UserActivity.currentPlayers < UserActivity.maxPlayers;
				}
				else
				{
					const XblMultiplayerActivityDetails& UserActivity = MPSDActivities[i];
					TSharedRef<FOnlineFriendGDK>& FoundFriend = ManagerTask.FriendsListMap.FindChecked(FUniqueNetIdGDK::Create(UserActivity.OwnerXuid));

					UE_LOG_ONLINE(VeryVerbose, TEXT("Found Friend User Activity:"));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  OwnerXboxUserId: %lld"), UserActivity.OwnerXuid);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  bIsClosed: %d"), UserActivity.Closed);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  HandleId: %ls"), UTF8_TO_TCHAR(UserActivity.HandleId));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  JoinRestriction: %ld"), EnumToUnderlyingType(UserActivity.JoinRestriction));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  MembersCount: %d"), UserActivity.MembersCount);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  MaxMembersCount: %d"), UserActivity.MaxMembersCount);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  MultiplayerSessionReference: %ls"), *FOnlineSessionMpsdGDK::SessionReferenceToUri(UserActivity.SessionReference));
					UE_LOG_ONLINE(VeryVerbose, TEXT("  TitleId: %d"), UserActivity.TitleId);
					UE_LOG_ONLINE(VeryVerbose, TEXT("  Visibility: %ld"), EnumToUnderlyingType(UserActivity.Visibility));

					const bool bHasEmptySlot = UserActivity.MembersCount < UserActivity.MaxMembersCount;
					const bool bSessionOpen = !UserActivity.Closed;

					const bool bIsJoinable = bHasEmptySlot && bSessionOpen;

					FoundFriend->Presence.SessionId = FUniqueNetIdString::Create(FOnlineSessionMpsdGDK::SessionReferenceToUri(UserActivity.SessionReference), GDK_SUBSYSTEM);
					FoundFriend->Presence.bIsJoinable = bIsJoinable;
				}
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			bWasSuccessful = true;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error querying friend session details, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	if(MPSDActivities)
	{
		delete[] MPSDActivities;
	}
}

void FOnlineAsyncTaskGDKQueryFriendSessionDetails::Finalize()
{
	ManagerTask.SessionDetailsStatus = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
}

#endif //WITH_GRDK
