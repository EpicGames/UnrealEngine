// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineFriendsInterfaceGDK.h"

class FOnlineSubsystemGDK;

/**
 * Async task to query our friends list
 */
class FOnlineAsyncTaskGDKQueryFriends
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryFriends(FOnlineSubsystemGDK* const InGDKInterface, FGDKContextHandle InGDKContext, const int32 InLocalUserNum, const FUniqueNetIdGDKRef& InGDKUniqueNetId, const FString& InListName);
	virtual ~FOnlineAsyncTaskGDKQueryFriends() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriends"); }

	// Start in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

protected:

	virtual void GetNextPage(XblSocialRelationshipResultHandle RelationshipHandle);
	virtual void ProcessNextPage(XblSocialRelationshipResultHandle RelationshipHandle);

	int32 LocalUserNum;
	FUniqueNetIdGDKRef GDKUniqueNetId;
	FString ListName;

	volatile int32 FoundItemCount;
	FString OutError;

	FOnlineFriendsListGDKMap FriendsListMap;
	FGDKContextHandle GDKContext;
	TArray<uint64> XUIDs;

	FGDKAsyncBlockPtr PagesAsyncBlock;
};

/**
 * Async task to manage calling delegates and updating the local cache after the other friends-list tasks are completed
 */
class FOnlineAsyncTaskGDKQueryFriendManagerTask
	: public FOnlineAsyncTaskGDK
{
	friend class FOnlineAsyncTaskGDKQueryFriendAccountDetails;
	friend class FOnlineAsyncTaskGDKQueryFriendPresenceDetails;

public:
	FOnlineAsyncTaskGDKQueryFriendManagerTask(FOnlineSubsystemGDK* const InSubsystem, FOnlineFriendsListGDKMap&& InFriendsListMap, const int32 InLocalUserNum, const FUniqueNetIdGDKRef& InGDKUniqueNetId, FString&& InListName)
		: FOnlineAsyncTaskGDK(InSubsystem, TEXT("FOnlineAsyncTaskGDKQueryFriendManagerTask"), InLocalUserNum)
		, FriendsListMap(MoveTemp(InFriendsListMap))
		, LocalUserNum(InLocalUserNum)
		, GDKUniqueNetId(InGDKUniqueNetId)
		, ListName(MoveTemp(InListName))
		, AccountDetailsStatus(EOnlineAsyncTaskState::NotStarted)
		, PresenceDetailsStatus(EOnlineAsyncTaskState::NotStarted)
		, PresenceStatsStatus(EOnlineAsyncTaskState::NotStarted)
		, SessionDetailsStatus(EOnlineAsyncTaskState::NotStarted)
	{
	}

	virtual ~FOnlineAsyncTaskGDKQueryFriendManagerTask() = default;

	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriendManagerTask"); }

	virtual void Tick() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

PACKAGE_SCOPE:
	FOnlineFriendsListGDKMap FriendsListMap;
	int32 LocalUserNum;
	FUniqueNetIdGDKRef GDKUniqueNetId;
	FString ListName;

	volatile EOnlineAsyncTaskState::Type AccountDetailsStatus;
	volatile EOnlineAsyncTaskState::Type PresenceDetailsStatus;
	volatile EOnlineAsyncTaskState::Type PresenceStatsStatus;
	volatile EOnlineAsyncTaskState::Type SessionDetailsStatus;
};

/**
 * Async Task to query the account details of friends
 */
class FOnlineAsyncTaskGDKQueryFriendAccountDetails
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryFriendAccountDetails(FOnlineSubsystemGDK* const InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64> InXUIDs,
		FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask);
	virtual ~FOnlineAsyncTaskGDKQueryFriendAccountDetails() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriendAccountDetails"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();

protected:
	const TArray<uint64> XUIDs;
	FOnlineAsyncTaskGDKQueryFriendManagerTask& ManagerTask;
	FGDKContextHandle GDKContext;
};

/**
 * Async Task to query the presence details of friends
 */
class FOnlineAsyncTaskGDKQueryFriendPresenceDetails
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryFriendPresenceDetails(FOnlineSubsystemGDK* const InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64> InXUIDs,
		FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask);
	virtual ~FOnlineAsyncTaskGDKQueryFriendPresenceDetails() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriendPresenceDetails"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();

protected:
	const TArray<uint64> XUIDs;
	FOnlineAsyncTaskGDKQueryFriendManagerTask& ManagerTask;
	FGDKContextHandle GDKContext;
	XblPresenceQueryFilters Filter{};
};

/**
 * Async Task to query additional stats used for presence of friends
 */
class FOnlineAsyncTaskGDKQueryFriendPresenceStats
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryFriendPresenceStats(FOnlineSubsystemGDK* const InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64> InXUIDs,
		const int32 InStartIndex,
		const TArray<FString>&& InPresenceStatNames,
		FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask);
	virtual ~FOnlineAsyncTaskGDKQueryFriendPresenceStats() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriendPresenceStats"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();

private:
	/** Maximum amount of user's to request stats from at a time */
	static const int32 MAX_USER_QUERY_COUNT = 50;

	/** Index to start processing users from*/
	int32 StartIndex;
	/** Index of next request's start; used to signal we're finished if this is >= the user vector size */
	int32 NextRequestIndex;
	/** Vector view of all friends to query */
	const TArray<uint64> XUIDs;
	/** List of stats needing queried from the above users */
	TArray<FString> PresenceStatNames;
	/** Manager task tracking completion of all friends requests */
	FOnlineAsyncTaskGDKQueryFriendManagerTask& ManagerTask;

	FGDKContextHandle GDKContext;

	/** Array to store Ansi strings for presence stat names that will be cleaned up when task is destroyed (Avoiding explicit mallocs) */
	TArray<TArray<ANSICHAR>> PresenceStatNamesAnsiChar;

	/** Array to store char* that we can pass to GDK API (Points to data in PresenceStatNamesAnsiChar)*/
	TArray<const ANSICHAR*> PresenceStatNamesCharPtr;
};

/**
 * Async Task to query the session details of friends
 */
class FOnlineAsyncTaskGDKQueryFriendSessionDetails
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryFriendSessionDetails(FOnlineSubsystemGDK* const InGDKInterface,
		FGDKContextHandle InGDKContext,
		const TArray<uint64> InXUIDs,
		FOnlineAsyncTaskGDKQueryFriendManagerTask& InManagerTask);
	virtual ~FOnlineAsyncTaskGDKQueryFriendSessionDetails() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryFriendSessionDetails"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();

protected:
	const TArray<uint64> XUIDs;
	FOnlineAsyncTaskGDKQueryFriendManagerTask& ManagerTask;
	FGDKContextHandle GDKContext;
};
