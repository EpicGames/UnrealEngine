// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryOffers.h"
#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"

// Maximum of item-details we may request at once
const int32 MaxDetailItemRequestSize = 10;

#define DEBUG_LOG_GDK_STORE_ITEMS 0

FOnlineAsyncTaskGDKQueryOffers::FOnlineAsyncTaskGDKQueryOffers(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const TArray<FUniqueOfferId>& InOfferIds, const FOnQueryOnlineStoreOffersComplete& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKQueryOffers"))
	, OfferIds(InOfferIds)
	, Delegate(InDelegate)
	, OfferQueriedIndex(0)
	, GDKContext(InGDKContext)
{
	check(Subsystem);
	OffersData.Empty(OfferIds.Num());
}

FOnlineAsyncTaskGDKQueryOffers::~FOnlineAsyncTaskGDKQueryOffers()
{
}

void FOnlineAsyncTaskGDKQueryOffers::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	HRESULT Result = XblContextGetUser(GDKContext, GDKUserHandle.GetInitReference());
	if (Result == S_OK)
	{
		//WMM Store interface has changed drastically.. have to ensure this still works
		XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUserHandle);
		if (StoreContextHandle != nullptr)
		{
			StartQuery();
		}
		else
		{
			ErrorResponse.bSucceeded = false;
			ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error querying store offers (store context), error: (0x%0.8X)."), Result)));
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		ErrorResponse.bSucceeded = false;
		ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error querying store offers (user handle), error: (0x%0.8X)."), Result)));
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryOffers::StartQuery()
{
	OfferIdsAnsiChar.Empty();
	OfferIdsCharPtr.Empty();

	for (int32 i = OfferQueriedIndex;
		i < OfferIds.Num() && OfferIdsAnsiChar.Num() < MaxDetailItemRequestSize;
		++i, ++OfferQueriedIndex)
	{
		TArray<ANSICHAR>& AnsiCharArray = OfferIdsAnsiChar.AddDefaulted_GetRef();
		AnsiCharArray.Append(TCHAR_TO_ANSI(*(OfferIds[i])), OfferIds[i].Len() + 1);
		OfferIdsCharPtr.Add(AnsiCharArray.GetData());
	}

	XStoreProductKind ProductKinds =
		XStoreProductKind::Consumable |
		XStoreProductKind::Durable |
		XStoreProductKind::Game |
		XStoreProductKind::Pass |
		XStoreProductKind::UnmanagedConsumable;

	XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUserHandle);
	if (!StoreContextHandle)
	{
		ErrorResponse.bSucceeded = false;
		ErrorResponse.SetFromErrorMessage(FText::FromString(TEXT("Error querying store offers ContextHandle.")));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	// From GDK documentation: Restricts the results by some action stored in the product document. By default, this API returns all
	// products, even if they are not purchasable, but you can restrict this to "Purchase" if you only want purchasable, or "License"
	// if you only want licensable. Other action filters include "Fulfill", "Browse", "Curate", "Details", and "Redeem".
	const ANSICHAR* ActionFilter[] = { "Purchase" };

	CA_SUPPRESS(6054) // analyser warns ActionFilter may not be null terminated, but the size is passed into the function
	HRESULT Result = XStoreQueryProductsAsync(StoreContextHandle, ProductKinds, OfferIdsCharPtr.GetData(), OfferIdsCharPtr.Num(), ActionFilter, sizeof(ActionFilter) / sizeof(ActionFilter[0]), *AsyncBlock);
	if (Result != S_OK)
	{
		ErrorResponse.bSucceeded = false;
		ErrorResponse.SetFromErrorMessage(FText::FromString(FString::Printf(TEXT("Error querying store offers, error: (0x%0.8X)."), Result)));
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryOffers::ProcessResults()
{
	XStoreProductQueryHandle ProductQueryHandle = nullptr;
	HRESULT Result = XStoreQueryProductsResult(*AsyncBlock, &ProductQueryHandle);

	if (Result == S_OK)
	{
		Result = XStoreEnumerateProductsQuery(ProductQueryHandle, this, [](const XStoreProduct* Product, void* Context) {
			FOnlineAsyncTaskGDKQueryOffers* Owner = static_cast<FOnlineAsyncTaskGDKQueryOffers*>(Context);
			return Owner->EnumerateProductResults(Product);
		});

		if (Result == S_OK)
		{
			const bool bHasMoreToQuery = OfferQueriedIndex < OfferIds.Num();
			if (bHasMoreToQuery)
			{
				StartQuery();
			}
			else
			{
				ErrorResponse.bSucceeded = true;
				bWasSuccessful = true;
				bIsComplete = true;
			}
		}
		else
		{
			FString ErrorMessage = FString::Printf(TEXT("FOnlineStoreLive::QueryOffersById failed with code 0x%0.8X."), Result);
			ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
			ErrorResponse.bSucceeded = false;
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		FString ErrorMessage = FString::Printf(TEXT("FOnlineStoreLive::QueryOffersById failed with code 0x%0.8X."), Result);
		ErrorResponse.SetFromErrorMessage(FText::FromString(ErrorMessage));
		ErrorResponse.bSucceeded = false;
		bWasSuccessful = false;
		bIsComplete = true;
	}

	if (ProductQueryHandle)
	{
		XStoreCloseProductsQueryHandle(ProductQueryHandle);
	}
}

bool FOnlineAsyncTaskGDKQueryOffers::EnumerateProductResults(const XStoreProduct* Product)
{
	if(Product->skusCount > 0)
	{
		XStoreSku& Sku = Product->skus[0];

#if DEBUG_LOG_GDK_STORE_ITEMS && !UE_BUILD_SHIPPING
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("storeId"), Product->storeId ? UTF8_TO_TCHAR(Product->storeId) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("title"), Product->title ? UTF8_TO_TCHAR(Product->title) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("description"), Product->description ? UTF8_TO_TCHAR(Product->description) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("language"), Product->language ? UTF8_TO_TCHAR(Product->language) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("inAppOfferToken"), Product->inAppOfferToken ? UTF8_TO_TCHAR(Product->inAppOfferToken) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("linkUri"), Product->linkUri ? UTF8_TO_TCHAR(Product->linkUri) : TEXT(""));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %ld"), TEXT("productKind"), Product->productKind);
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("hasDigitalDownload"), *LexToString(Product->hasDigitalDownload));
		UE_LOG_ONLINE(Log, TEXT("ProductDetails: %s = %s"), TEXT("isInUserCollection"), *LexToString(Product->isInUserCollection));
#endif

		if (Sku.availabilitiesCount > 0)
		{
			const XStoreAvailability& ItemAvailability = Sku.availabilities[0];

#if DEBUG_LOG_GDK_STORE_ITEMS && !UE_BUILD_SHIPPING
			/* //WMM fix this
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("ContentId"),					XboxCatalogItemPriceInfo->ContentId->Data());
			for (Platform::String^ AcceptPaymentType : XboxCatalogItemPriceInfo->AcceptablePaymentInstrumentTypes)
			{
				UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("AcceptablePaymentType[]"),	AcceptPaymentType->Data());
			}
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("AvailabilityTitle"),			XboxCatalogItemPriceInfo->AvailabilityTitle->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("AvailabilityDescription"),		XboxCatalogItemPriceInfo->AvailabilityDescription->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("CurrencyCode"),					XboxCatalogItemPriceInfo->CurrencyCode->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("DisplayPrice"),					XboxCatalogItemPriceInfo->DisplayPrice->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("DisplayListPrice"),				XboxCatalogItemPriceInfo->DisplayListPrice->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("DistributionType"),				XboxCatalogItemPriceInfo->DistributionType->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %s"), TEXT("IsPurchasable"),					XboxCatalogItemPriceInfo->IsPurchasable ? TEXT("True") : TEXT("False"));
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %f"), TEXT("Price"),							XboxCatalogItemPriceInfo->Price);
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %f"), TEXT("ListPrice"),						XboxCatalogItemPriceInfo->ListPrice);
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %d"), TEXT("ConsumableQuantity"),			XboxCatalogItemPriceInfo->ConsumableQuantity);
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("PromotionalText"),				XboxCatalogItemPriceInfo->PromotionalText->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("SignedOffer"),					XboxCatalogItemPriceInfo->SignedOffer->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("OfferId"),						XboxCatalogItemPriceInfo->OfferId->Data());
			UE_LOG_ONLINE(Log, TEXT("CatalogItemAvailability: %s = %ls"), TEXT("OfferDisplayDataJson"),			XboxCatalogItemPriceInfo->OfferDisplayDataJson->Data());
			*/
#endif

			FOnlineStoreOfferGDKRef NewOffer = MakeShared<FOnlineStoreOfferGDK>();

			// Set Offer Id
			NewOffer->OfferId = FString(UTF8_TO_TCHAR(Product->storeId));

			// Set display text fields
			NewOffer->Title = FText::FromString(FString(UTF8_TO_TCHAR(Product->title)));
			NewOffer->Description = FText::FromString(FString(UTF8_TO_TCHAR(Product->description)));
			NewOffer->LongDescription = NewOffer->Description;

			// Save the SignedOffer, we need to pass this to the purchase call as the "id" //WMM this still the case?
			NewOffer->SignedOffer = FString(UTF8_TO_TCHAR(Product->inAppOfferToken));

			// Price currency code
			NewOffer->CurrencyCode = FString(FString(UTF8_TO_TCHAR(ItemAvailability.price.currencyCode)));

			// The API returns floating point prices. Converting numerical price to integers based on the number of decimals in the currency of the culture
			const FCulture& Culture = *FInternationalization::Get().GetCurrentCulture();

			const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(NewOffer->CurrencyCode);
			const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;

			// Current purchase price
			NewOffer->PriceText = FText::FromString(FString(UTF8_TO_TCHAR(ItemAvailability.price.formattedPrice)));
			const double PriceValueAsDouble = static_cast<double>(ItemAvailability.price.price) * static_cast<double>(FMath::Pow(10.0f, FormattingOptions.MaximumFractionalDigits));
			NewOffer->NumericPrice = FMath::RoundToInt(PriceValueAsDouble);

			// Base (strikethrough) price during a sale
			NewOffer->RegularPriceText = FText::FromString(FString(UTF8_TO_TCHAR(ItemAvailability.price.formattedBasePrice)));
			const double BasePriceValueAsDouble = static_cast<double>(ItemAvailability.price.basePrice) * static_cast<double>(FMath::Pow(10.0f, FormattingOptions.MaximumFractionalDigits));
			NewOffer->RegularPrice = FMath::RoundToInt(BasePriceValueAsDouble);

			// Save out our new price
			OffersData.Emplace(MoveTemp(NewOffer));
		}
	}

	return true;
}

void FOnlineAsyncTaskGDKQueryOffers::Finalize()
{
	if (ErrorResponse.bSucceeded)
	{
		FOnlineStoreGDKPtr StoreInt = Subsystem->GetStoreGDK();
		if (StoreInt.IsValid())
		{
			StoreInt->CachedOffers.Reserve(StoreInt->CachedOffers.Num() + OffersData.Num());
			for (FOnlineStoreOfferRef& StoreOffer : OffersData)
			{
				StoreInt->CachedOffers.Emplace(StoreOffer->OfferId, StoreOffer);
			}
		}
	}
}

void FOnlineAsyncTaskGDKQueryOffers::TriggerDelegates()
{
	TArray<FUniqueOfferId> NewOfferIds;
	NewOfferIds.Reserve(OffersData.Num());
	for (const FOnlineStoreOfferRef& Offer : OffersData)
	{
		NewOfferIds.AddUnique(Offer->OfferId);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryOffers_TriggerDelegates);
	Delegate.ExecuteIfBound(ErrorResponse.bSucceeded, NewOfferIds, ErrorResponse.GetErrorMessage().ToString());
}

#undef DEBUG_LOG_GDK_STORE_ITEMS

#endif //WITH_GRDK