// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END

class FOnlineSubsystemGDK;

/**
 * Async item used to marshal store offer results from the GDK system thread to the game thread.
 */
class FOnlineAsyncTaskGDKQueryOffers : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryOffers(FOnlineSubsystemGDK* InGDKSubsystem, FGDKContextHandle InGDKContext, const TArray<FUniqueOfferId>& InOfferIds, const FOnQueryOnlineStoreOffersComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryOffers();

	virtual void Tick() override;
	virtual void ProcessResults() override;
	
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryOffers");}
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	void StartQuery();
	bool EnumerateProductResults(const XStoreProduct* Product);
protected:
	TArray<FUniqueOfferId> OfferIds;
	TArray<FOnlineStoreOfferRef> OffersData;
	FOnlineError ErrorResponse;
	FOnQueryOnlineStoreOffersComplete Delegate;
	int32 OfferQueriedIndex;
	FGDKContextHandle GDKContext;
	TArray<TArray<ANSICHAR>> OfferIdsAnsiChar;
	TArray<const ANSICHAR*> OfferIdsCharPtr;
	FGDKUserHandle GDKUserHandle;
	bool bTaskStarted = false;
};
