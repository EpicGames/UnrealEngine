// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "AsyncTasks/OnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort.h"
#include "OnlineSubsystemGDK.h"
#include "XNetworking.h"

FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort(
	FOnlineSubsystemGDK* InGDKInterface,
	const FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate& InTaskCompletionDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort"))
	, TaskCompletionDelegate(InTaskCompletionDelegate)
{

}

// Starts in Game Thread
void FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::Initialize()
{
	HRESULT Result = XNetworkingQueryPreferredLocalUdpMultiplayerPortAsync(*AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::Initialize unable to resolve with code 0x%0.8X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::ProcessResults()
{
	HRESULT Result = XNetworkingQueryPreferredLocalUdpMultiplayerPortAsyncResult(*AsyncBlock, &GDKPort);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::ProcessResults unable to resolve with code 0x%0.8X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	else
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
}

// Move results and trigger delegates in Game Thread
void FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, GDKPort);
}


#endif //WITH_GRDK