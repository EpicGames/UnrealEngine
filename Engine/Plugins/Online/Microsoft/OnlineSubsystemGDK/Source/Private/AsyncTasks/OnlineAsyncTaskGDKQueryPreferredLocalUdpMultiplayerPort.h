// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to query the preferred local udp multiplayer port
 */
class FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort
	: public FOnlineAsyncTaskGDK
{
public:
	/**
	* Delegate fired when GetUserProfile Task is complete.
	*/
	DECLARE_DELEGATE_TwoParams(FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate, bool, uint16_t /*GDKPort*/);

	FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort(
		FOnlineSubsystemGDK* InGDKInterface,
		const FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate& InTaskCompletionDelegate);

	virtual ~FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort"); }

	// Starts in Game Thread
	virtual void Initialize() override;

	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	uint16_t GDKPort;
	const FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate TaskCompletionDelegate;
};
