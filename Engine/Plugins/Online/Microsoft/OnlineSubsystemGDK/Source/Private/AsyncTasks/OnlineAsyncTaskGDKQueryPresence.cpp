// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryPresence.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

void FOnlineAsyncTaskGDKQueryPresence::Initialize()
{
	uint64 UserToQueryXuid = User->ToUint64();

	PresenceAsyncBlock.context = this;
	PresenceAsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
	{
		FOnlineAsyncTaskGDKQueryPresence* Owner = static_cast<FOnlineAsyncTaskGDKQueryPresence*>(LambdaAsyncBlock->context);
		Owner->ProcessPresenceResult();
	};

	HRESULT Result = XblPresenceGetPresenceAsync(GDKContext, UserToQueryXuid, &PresenceAsyncBlock);

	if(Result == S_OK)
	{
		// Also query stats that have been configured to use as presence keys.
		TArray<FString> StatNames = FOnlinePresenceGDK::GetAutoSubscribePresenceStatNames();
		if (StatNames.Num() > 0)
		{
			FMemory::Memzero(&StatsAsyncBlock, sizeof(XAsyncBlock));
			StatsAsyncBlock.context = this;
			StatsAsyncBlock.callback = [](XAsyncBlock* LambdaAsyncBlock)
			{
				FOnlineAsyncTaskGDKQueryPresence* Owner = static_cast<FOnlineAsyncTaskGDKQueryPresence*>(LambdaAsyncBlock->context);
				Owner->ProcessStatsResult();
			};

			const char* ServiceConfigurationId = nullptr;
			XblGetScid(&ServiceConfigurationId);

			Result = XblUserStatisticsGetSingleUserStatisticAsync(GDKContext, UserToQueryXuid, ServiceConfigurationId, TCHAR_TO_UTF8(*(StatNames[0])), &StatsAsyncBlock);

			if(Result != S_OK)
			{
				UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKQueryPresence::Initialize: Error from GetSingleUserStatisticsAsync, with code 0x%08X."), Result);
				bWasSuccessful = false;
				bIsComplete = true;
			}
		}
		else
		{
			bStatsTaskIsDone = true; // Didn't do the stats task because we had no stats to search for. Pretend we did.
			bStatsTaskSucceeded = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKQueryPresence::Initialize: Error from PresenceGetPresence, with code 0x%08X."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryPresence::ProcessStatsResult()
{
	uint64 ResultSizeInBytes = 0;
	HRESULT Result = XblUserStatisticsGetSingleUserStatisticsResultSize(&StatsAsyncBlock, &ResultSizeInBytes);

	if (Result == S_OK)
	{
		StatsResultBuffer.Reserve(ResultSizeInBytes);
		Result = XblUserStatisticsGetSingleUserStatisticsResult(&StatsAsyncBlock, ResultSizeInBytes, StatsResultBuffer.GetData(), &StatisticsResult, nullptr);
		if (Result == S_OK)
		{
			bStatsTaskSucceeded = true;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKQueryPresence::ProcessStatsResult: Error from UserStatisticsGetSingleUserStatisticsResult, with code 0x%08X."), Result);
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKQueryPresence::ProcessStatsResult: Error from UserStatisticsGetSingleUserStatisticsResultNum, with code 0x%08X."), Result);
	}
	bStatsTaskIsDone = true;
}

void FOnlineAsyncTaskGDKQueryPresence::ProcessPresenceResult()
{
	uint64 ResultSizeInBytes = 0;
	HRESULT Result = XblPresenceGetPresenceResult(&PresenceAsyncBlock, PresenceRecordHandle.GetInitReference());
	if (Result == S_OK)
	{
		bPresenceTaskSucceeded = true;
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKQueryPresence::ProcessPresenceResult: Error from PresenceGetPresenceResult, with code 0x%08X."), Result);
	}
	bPresenceTaskIsDone = true;
}

void FOnlineAsyncTaskGDKQueryPresence::Tick()
{
	if (bIsComplete)
	{
		return;
	}

	if (bPresenceTaskIsDone && bStatsTaskIsDone)
	{
		bWasSuccessful = bPresenceTaskSucceeded && bStatsTaskSucceeded;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryPresence::Finalize()
{
	FOnlineAsyncTaskBasic::Finalize();

	if (!Subsystem)
	{
		bWasSuccessful = false;
		return;
	}

	FOnlinePresenceGDKPtr PresenceGDK = Subsystem->GetPresenceGDK();
	check(PresenceGDK.IsValid());

	const XblPresenceDeviceRecord* DeviceRecords = nullptr;
	uint64 DeviceRecordsCount = 0;
	HRESULT Result = XblPresenceRecordGetDeviceRecords(PresenceRecordHandle, &DeviceRecords, &DeviceRecordsCount);
	if (Result != S_OK)
	{
		bWasSuccessful = false;
		return;
	}

	TSharedRef<FOnlineUserPresenceGDK> NewPresence = MakeShared<FOnlineUserPresenceGDK>(PresenceRecordHandle);
	if (StatisticsResult != nullptr)
	{
		NewPresence->SetStatusPropertiesFromStatistics(*StatsResult);
	}
	PresenceGDK->PresenceCache.Emplace(User, NewPresence);

	// Only subscribe to stat updates for this user if they're online and playing the same game as us
	if (NewPresence->bIsOnline && NewPresence->bIsPlayingThisGame)
	{
		uint64 RequestingUserId;
		if (SUCCEEDED(XblContextGetXboxUserId(GDKContext, &RequestingUserId)))
		{
			// Make sure we've subscribed to stat updates for this user as well
			FUniqueNetIdGDKRef RequestingUser = FUniqueNetIdGDK::Create(RequestingUserId);
			PresenceGDK->EstablishDefaultPresenceStatSubscriptions(RequestingUser, User);
		}
	}
}

void FOnlineAsyncTaskGDKQueryPresence::TriggerDelegates()
{
	FOnlineAsyncTaskBasic::TriggerDelegates();

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryPresence_TriggerDelegates);
		Delegate.ExecuteIfBound(*User, bWasSuccessful);
	}

	if (bWasSuccessful)
	{
		TSharedPtr<FOnlineUserPresence> Presence;
		if (Subsystem &&
			Subsystem->GetPresenceGDK().IsValid() &&
			Subsystem->GetPresenceGDK()->GetCachedPresence(*User, Presence) == EOnlineCachedResult::Success &&
			Presence.IsValid())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryPresence_TriggerDelegates);
			Subsystem->GetPresenceGDK()->TriggerOnPresenceReceivedDelegates(*User, Presence.ToSharedRef());
		}
	}
}

#endif //WITH_GRDK