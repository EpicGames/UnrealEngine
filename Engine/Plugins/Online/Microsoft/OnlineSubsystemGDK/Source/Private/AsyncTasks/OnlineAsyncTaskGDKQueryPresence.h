// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePresenceInterfaceGDK.h"

/**
 * Custom task class that handles getting presence info as well as configured stats in parallel.
 */
class FOnlineAsyncTaskGDKQueryPresence : public FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>
{
public:
	FOnlineAsyncTaskGDKQueryPresence(
			FOnlineSubsystemGDK* const InGDKSubsystem,
			FGDKContextHandle InGDKContext,
			const FUniqueNetIdGDKRef& InUser,
			const IOnlinePresence::FOnPresenceTaskCompleteDelegate& InDelegate)
		: FOnlineAsyncTaskBasic(InGDKSubsystem)
		, GDKContext(InGDKContext)
		, User(InUser)
		, Delegate(InDelegate)
	{
	}

	virtual FString ToString() const override
	{
		return FString(TEXT("FOnlineAsyncTaskGDKQueryPresence"));
	}

	virtual void Initialize() override;
	void ProcessResult();
	virtual void Tick() override;
	void ProcessPresenceResult();
	void ProcessStatsResult();
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

private:
	/** The GDK context of the user requesting presence information. */
	FGDKContextHandle GDKContext;

	/** The user whose presence data is being queried. */
	FUniqueNetIdGDKRef User;

	/** Stored delegate passed in from IOnlinePesence::QueryPresence, to be triggered in TriggerDelegates(). */
	IOnlinePresence::FOnPresenceTaskCompleteDelegate Delegate;

	/** Result of the async presence query. Used to update cached data in Finalize(). */
	FGDKPresenceRecordHandle PresenceResult;

	/** Result of the async stats query. Used to update cached data in Finalize(). */
	XblUserStatisticsResult* StatsResult = nullptr;

	XAsyncBlock PresenceAsyncBlock = XAsyncBlock{ 0 };
	XAsyncBlock StatsAsyncBlock = XAsyncBlock{ 0 };

	bool bPresenceTaskIsDone = false;
	bool bPresenceTaskSucceeded = false;
	bool bStatsTaskIsDone = false;
	bool bStatsTaskSucceeded = false;

	TArray<uint8> StatsResultBuffer;
	XblUserStatisticsResult* StatisticsResult = nullptr;
	FGDKPresenceRecordHandle PresenceRecordHandle;
};
