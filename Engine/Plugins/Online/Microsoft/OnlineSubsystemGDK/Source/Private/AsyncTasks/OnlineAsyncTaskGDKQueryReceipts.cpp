// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryReceipts.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "Misc/Guid.h"
#include "OnlineStoreInterfaceGDK.h"

const constexpr int32 INVENTORY_PAGE_SIZE = 32;
FOnlineAsyncTaskGDKQueryReceipts::FOnlineAsyncTaskGDKQueryReceipts(FOnlineSubsystemGDK* const InGDKInterface, const FUniqueNetIdGDKRef& InUserId, FGDKContextHandle InGDKContext, const FOnQueryReceiptsComplete& InQueryReceiptsDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryReceipts"))
	, GDKContext(InGDKContext)
	, UserId(InUserId)
	, bSoftFailReceiptFailures(false)
	, ErrorResponse(true)
	, QueryReceiptsDelegate(InQueryReceiptsDelegate)
{
	check(GDKContext);
}

FOnlineAsyncTaskGDKQueryReceipts::FOnlineAsyncTaskGDKQueryReceipts(FOnlineSubsystemGDK* const InGDKInterface, const FUniqueNetIdGDKRef& InUserId, FGDKContextHandle InGDKContext, const FOnPurchaseCheckoutComplete& InPurchaseCheckoutDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKQueryReceipts"))
	, GDKContext(InGDKContext)
	, UserId(InUserId)
	, bSoftFailReceiptFailures(false)
	, ErrorResponse(true)
	, PurchaseCheckoutDelegate(InPurchaseCheckoutDelegate)
{
	check(GDKContext);
}

FOnlineAsyncTaskGDKQueryReceipts::~FOnlineAsyncTaskGDKQueryReceipts()
{			
}

void FOnlineAsyncTaskGDKQueryReceipts::TickStartTask()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	FOnlinePurchaseGDKPtr PurchaseInt = Subsystem->GetPurchaseGDK();
	if (!PurchaseInt.IsValid())
	{
		UE_LOG_ONLINE(Error, TEXT("[FOnlineAsyncTaskGDKQueryReceipts::Initialize] unable to get purchase interface."));
		FailTask(false /* Hard-fail on missing PurchaseInt (not possible?) */);
		return;
	}

	bSoftFailReceiptFailures = PurchaseInt->bSoftFailReceiptFailures;

	XblContextGetUser(GDKContext, UserHandle.GetInitReference());

	if (!PurchaseInt->ReceiptsXSTSEndpoint.IsEmpty())
	{
		RemoveAsyncBlock(AsyncBlock);
		AsyncBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock) {
			ProcessTokenResult();
		});

		HRESULT Result = XUserGetTokenAndSignatureAsync(UserHandle, XUserGetTokenAndSignatureOptions::None, "GET", TCHAR_TO_UTF8(*PurchaseInt->ReceiptsXSTSEndpoint), 0, nullptr, 0, nullptr, *AsyncBlock);
	
		if(Result != S_OK)
		{
			UE_LOG_ONLINE(Error, TEXT("[FOnlineAsyncTaskGDKQueryReceipts::Initialize] starting GetTokenAndSignatureAsync operation failed with code 0x%0.8X."), Result);
			FailTaskCode(Result, false /* Hard-fail on task starts*/);
			return;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("[FOnlineAsyncTaskGDKQueryReceipts::Initialize] Missing ReceiptsXSTSEndpoint configuration."));
		
		// missing ReceiptsXSTSEndpoint configuration will still query receipts but won't append token info
		XSTSToken.Empty();
		bXSTSTokenTaskIsSuccessful = true;
		bXSTSTokenTaskIsComplete = true;
	}

	XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(UserHandle);

	if (!StoreContextHandle)
	{
		bProductTaskIsSuccessful = false;
		bProductTaskIsComplete = true;
		return;
	}

	XStoreProductKind allProductKinds =
		XStoreProductKind::Consumable |
		XStoreProductKind::Durable |
		XStoreProductKind::Game |
		XStoreProductKind::Pass |
		XStoreProductKind::UnmanagedConsumable;

	RemoveAsyncBlock(AsyncBlockProducts);
	AsyncBlockProducts = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock) {
		ProcessProductsResult();
	});

	HRESULT Result = XStoreQueryEntitledProductsAsync(StoreContextHandle, allProductKinds, INVENTORY_PAGE_SIZE, *AsyncBlockProducts);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKQueryReceipts starting GetInventoryItems operation failed with code 0x%0.8X."), Result);
		// The XUserGetTokenAndSignatureAsync request might still be in still be in flight, so we need to wait for Tick to complete this task
		RemoveAsyncBlock(AsyncBlockProducts);
		bProductTaskIsSuccessful = false;
		bProductTaskIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::ProcessTokenResult()
{
	uint64 BufferSize = 0;
	HRESULT Result = XUserGetTokenAndSignatureResultSize(*AsyncBlock, &BufferSize);
	if (Result == S_OK)
	{
		TArray<uint8> ResultBuffer;
		ResultBuffer.Reserve(BufferSize);
		XUserGetTokenAndSignatureData* Tokens = nullptr;
		Result = XUserGetTokenAndSignatureResult(*AsyncBlock, BufferSize, ResultBuffer.GetData(), &Tokens, nullptr);
		if (Result == S_OK)
		{
			XSTSToken = UTF8_TO_TCHAR(Tokens[0].token);
			bXSTSTokenTaskIsSuccessful = true;
		}
	}
	bXSTSTokenTaskIsComplete = true;	
}

void FOnlineAsyncTaskGDKQueryReceipts::EnumerateProductsResult(XStoreProductQueryHandle QueryHandle)
{
	
	HRESULT Result = XStoreEnumerateProductsQuery(QueryHandle, this, [](const XStoreProduct* Product, void* Context)
	{
		if (Context)
		{
			FOnlineAsyncTaskGDKQueryReceipts* Owner = static_cast<FOnlineAsyncTaskGDKQueryReceipts*>(Context);

			Owner->ProductArray.Add(Product);
		}
		return true;
	});

	if (Result == S_OK)
	{
		if (XStoreProductsQueryHasMorePages(QueryHandle))
		{
			RemoveAsyncBlock(AsyncBlockProducts);
			AsyncBlockProducts = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock) {
				ProcessProductsNextPageResult();
			});
			
			Result = XStoreProductsQueryNextPageAsync(QueryHandle, *AsyncBlockProducts);
			if (Result != S_OK)
			{
				UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKQueryReceipts failed to query additional entitled product pages. ErrorCode=[0x%0.8X]"), Result);

				bProductTaskIsComplete = true;
			}
			return;
		}
		else
		{
			bProductTaskIsSuccessful = true;
			bProductTaskIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKQueryReceipts failed to enumerate queried entitled products. ErrorCode=[0x%0.8X]"), Result);

		bProductTaskIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::ProcessProductsResult()
{
	XStoreProductQueryHandle ProductQueryHandle = nullptr;
	HRESULT Result = XStoreQueryEntitledProductsResult(*AsyncBlockProducts, &ProductQueryHandle);
	if (Result == S_OK)
	{
		EnumerateProductsResult(ProductQueryHandle);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKQueryReceipts failed to get queried entitled products results. ErrorCode=[0x%0.8X]"), Result);

		bProductTaskIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::ProcessProductsNextPageResult()
{
	XStoreProductQueryHandle ProductQueryHandle = nullptr;
	HRESULT Result = XStoreProductsQueryNextPageResult(*AsyncBlockProducts, &ProductQueryHandle);
	if (Result == S_OK)
	{
		EnumerateProductsResult(ProductQueryHandle);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKQueryReceipts failed to get additional queried entitled product result pages. ErrorCode=[0x%0.8X]"), Result);

		bProductTaskIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::Tick()
{
	TickStartTask();

	if (bIsComplete)
	{
		return;
	}

	if (bXSTSTokenTaskIsComplete && bProductTaskIsComplete)
	{
		if (bProductTaskIsSuccessful && bXSTSTokenTaskIsSuccessful)
		{

			FPurchaseReceipt PurchaseReceipt;

			const int32 FoundItems = ProductArray.Num();
			UE_LOG_ONLINE(Log, TEXT("[FOnlineAsyncTaskLiveQueryReceipts] Found %d Receipts"), FoundItems);

			if (ProductArray.Num() > 0)
			{
				for (const XStoreProduct* PurchasedProduct : ProductArray)
				{
					check(PurchasedProduct);
					const XStoreProduct& Product = *PurchasedProduct;

					// Find the first SKU that is in user's collection.
					const XStoreCollectionData* CollectionData = nullptr;
					for (uint32_t SkuIndex = 0; SkuIndex < Product.skusCount; ++SkuIndex)
					{
						const XStoreSku& Sku = Product.skus[SkuIndex];
						if (Sku.isInUserCollection)
						{
							CollectionData = &Sku.collectionData;
							break;
						}
					}
					// We are querying the user collection, so there must be at least one SKU that's in user's collection.
					check(CollectionData);

					int64 AcquiredDate  = 0;
					uint32 Quantity = 0;
					if (CollectionData != nullptr)
					{
						AcquiredDate = (int64)CollectionData->acquiredDate;

						if (static_cast<uint32>(Product.productKind & XStoreProductKind::Consumable) != 0)
						{
							Quantity = CollectionData->quantity;
						}
						else
						{
							Quantity = 1; // CollectionData->quantity may be 0 for non-consumables, so we set it to 1
						}
					}

					const FString ProductId(UTF8_TO_TCHAR(Product.storeId));

					UE_LOG_ONLINE(Log, TEXT("Start ReceiptItemDetails"));
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %u"), TEXT("Quantity"), Quantity);
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %lld"), TEXT("AcquiredDate"), AcquiredDate);
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %d"), TEXT("ProductKind"), EnumToUnderlyingType(Product.productKind));
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %s"), TEXT("StoreId"), UTF8_TO_TCHAR(Product.storeId));
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %s"), TEXT("Title"), UTF8_TO_TCHAR(Product.title));
					UE_LOG_ONLINE(Log, TEXT("ReceiptItemDetails: %s = %s"), TEXT("Url"), UTF8_TO_TCHAR(Product.linkUri));
					UE_LOG_ONLINE(Log, TEXT("End ReceiptItemDetails"));

					FPurchaseReceipt::FReceiptOfferEntry OfferEntry(FString(), ProductId, Quantity);
					EPurchaseValidationMethod ValidationMethod = Subsystem->GetPurchaseGDK()->GetPurchaseValidationMethod(Product.productKind);
					{
						// We generate them in index order, and later consume these receipts in the reverse order, so that index == quantity at the time
						for (uint32 ConsumableIndex = 1; ConsumableIndex <= Quantity; ++ConsumableIndex)
						{
							FPurchaseReceipt::FLineItemInfo LineItem;
							LineItem.ItemName = ProductId;
							if (static_cast<uint32>(Product.productKind & XStoreProductKind::Pass) != 0)
							{
								// For passes, we include the XUID, productId, and the acquired date to make each purchase unique.
								LineItem.UniqueId = FString::Printf(TEXT("%s:%s:%lld"), *UserId->ToString(), *ProductId, AcquiredDate);
							}
							else
							{
								// For everything else we maintain our existing behavior.
								LineItem.UniqueId = FString::Printf(TEXT("%s:%d:%s"), *ProductId, ConsumableIndex, *FGuid::NewGuid().ToString());
							}
							if (ValidationMethod == EPurchaseValidationMethod::XSTSToken)
							{
								LineItem.ValidationInfo = FString::Printf(TEXT("%s:%s"), *UserId->ToString(), *XSTSToken);
							}
							else if (ValidationMethod == EPurchaseValidationMethod::UserCollectionsId)
							{
								LineItem.ValidationInfo = FString::Printf(TEXT("%s:UserCollectionsId::ServiceTicket:"), *UserId->ToString());
							}
							else if (ValidationMethod == EPurchaseValidationMethod::UserPurchaseId)
							{
								LineItem.ValidationInfo = FString::Printf(TEXT("%s:UserPurchaseId::ServiceTicket:"), *UserId->ToString());
							}
							OfferEntry.LineItems.Emplace(MoveTemp(LineItem));
						}
					}

					PurchaseReceipt.ReceiptOffers.Emplace(MoveTemp(OfferEntry));
				}
				// Add our receipt into our array
				ReceiptData.Add(MoveTemp(PurchaseReceipt));
			}

			bWasSuccessful = ErrorResponse.bSucceeded;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("[FOnlineAsyncTaskLiveQueryReceipts] bProductTaskIsSuccessful=%s bXSTSTokenTaskIsSuccessful=%s"), 
				*LexToString(bProductTaskIsSuccessful), *LexToString(bXSTSTokenTaskIsSuccessful));

			FailTask(false);
		}		
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::Finalize()
{
	if (ErrorResponse.bSucceeded)
	{
		FOnlinePurchaseGDKPtr PurchaseInt = Subsystem->GetPurchaseGDK();
		if (PurchaseInt.IsValid())
		{
			PurchaseInt->UserCachedReceipts.Emplace(MoveTemp(UserId), ReceiptData);
		}
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryReceipts_TriggerDelegates);

	// We support multiple delegate types for code-reuse reasons, so execute whichever are bound
	if (QueryReceiptsDelegate.IsBound())
	{
		QueryReceiptsDelegate.ExecuteIfBound(ErrorResponse);
	}

	if (PurchaseCheckoutDelegate.IsBound())
	{
		if (ReceiptData.IsValidIndex(0) && ErrorResponse.bSucceeded)
		{
			PurchaseCheckoutDelegate.ExecuteIfBound(ErrorResponse, MakeShared<FPurchaseReceipt>(ReceiptData[0]));
		}
		else
		{
			PurchaseCheckoutDelegate.ExecuteIfBound(ErrorResponse, MakeShared<FPurchaseReceipt>());
		}
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::FailTask_Internal(const bool bSoftFail)
{
	ErrorResponse.bSucceeded = bSoftFail;

	bWasSuccessful = bSoftFail;
	if (!bSoftFail)
	{
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryReceipts::FailTask(const bool bSoftFail)
{
	// Call FText version with generic message
	FailTask(NSLOCTEXT("OnlineSubsystem", "FailedToRetrievePurchases", "An error has occurred while querying purchases; purchased content may be temporarily unavailable."), bSoftFail);
}

void FOnlineAsyncTaskGDKQueryReceipts::FailTask(const FText ErrorMessage, const bool bSoftFail)
{
	ErrorResponse.SetFromErrorMessage(ErrorMessage);
	FailTask_Internal(bSoftFail);
}

void FOnlineAsyncTaskGDKQueryReceipts::FailTaskCode(const int32 ErrorCodeInt, const bool bSoftFail)
{
	static const FText ServiceName = Subsystem->GetOnlineServiceName();

	FFormatNamedArguments FormatArguments;
	FormatArguments.Add(TEXT("ServiceName"), ServiceName);
	FormatArguments.Add(TEXT("ErrorCode"), FText::FromString(FString::Printf(TEXT("0x%08X"), ErrorCodeInt)));
	const FText ErrorMessageText(FText::Format(NSLOCTEXT("OnlineSubsystem", "FailedToRetrievePurchasesWithCode", "Unable to verify purchase with {ServiceName} services; purchased content may be temporarily unavailable. {ServiceName} Error Code: {ErrorCode}"), FormatArguments));
	ErrorResponse = OnlinePurchaseGDK::Errors::HResultError(ErrorCodeInt, ErrorMessageText);

	FailTask_Internal(bSoftFail);
}

#endif //WITH_GRDK