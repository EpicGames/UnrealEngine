// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineError.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async item used to query a specific user's receipts
 */
class FOnlineAsyncTaskGDKQueryReceipts
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryReceipts(FOnlineSubsystemGDK* const InGDKInterface, const FUniqueNetIdGDKRef& InUserId, FGDKContextHandle InGDKContext, const FOnQueryReceiptsComplete& InQueryReceiptsDelegate);
	FOnlineAsyncTaskGDKQueryReceipts(FOnlineSubsystemGDK* const InGDKInterface, const FUniqueNetIdGDKRef& InUserId, FGDKContextHandle InGDKContext, const FOnPurchaseCheckoutComplete& InPurchaseCheckoutDelegate);

	virtual ~FOnlineAsyncTaskGDKQueryReceipts();

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKQueryReceipts: %s"), *ErrorResponse.ToLogString()); }

	void ProcessTokenResult();
	void ProcessProductsResult();
	void EnumerateProductsResult(XStoreProductQueryHandle QueryHandle);
	void ProcessProductsNextPageResult();

	//~ Called on Online Thread
	void Tick();

	//~ Called on Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

private:
	void TickStartTask();
	void FailTask_Internal(const bool bSoftFail);
	void FailTask(const bool bSoftFail);
	void FailTask(const FText ErrorMessage, const bool bSoftFail);
	void FailTaskCode(const int32 ErrorCode, const bool bSoftFail);

private:
	FGDKContextHandle GDKContext;
	FUniqueNetIdGDKRef UserId;

	/** If true, instead of hard-failing on request failures, return success with an error message; this will replace our cached items with partial results */
	bool bSoftFailReceiptFailures;

	TArray<FPurchaseReceipt> ReceiptData;
	FOnlineError ErrorResponse;

	const FOnQueryReceiptsComplete QueryReceiptsDelegate;
	const FOnPurchaseCheckoutComplete PurchaseCheckoutDelegate;
	FGDKAsyncBlockPtr AsyncBlockProducts;
	FString XSTSToken;
	bool bXSTSTokenTaskIsSuccessful = false;
	bool bXSTSTokenTaskIsComplete = false;
	bool bProductTaskIsSuccessful = false;
	bool bProductTaskIsComplete = false;
	bool bTaskStarted = false;
	TArray<const XStoreProduct*> ProductArray;
	FGDKUserHandle UserHandle;
};
