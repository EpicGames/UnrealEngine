// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryUserPrivacyPermissions.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKQueryUserPrivacyPermissions::FOnlineAsyncTaskGDKQueryUserPrivacyPermissions(FOnlineSubsystemGDK* InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	const TArray<FUniqueNetIdRef>& InUsersToQuery,
	const TArray<XblPermission>& InPermissionsToQuery,
	const FOnGDKUserPrivacyPermissionsQueryComplete& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKQueryUserPrivacyPermissions"))
	, UsersToQuery(InUsersToQuery)
	, PermissionsToQuery(InPermissionsToQuery)
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryUserPrivacyPermissions::Initialize()
{
	// Ensure we have permissions
	if (PermissionsToQuery.Num() < 1)
	{
		OnlineError.SetFromErrorCode(TEXT("Error starting user privacy permissions query, no permissions were requested"));
		UE_LOG_ONLINE(Warning, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	// Convert our TArray of UniqueNetIds into an array of XUID strings
	TArray<uint64> XUIDs;
	for (const FUniqueNetIdRef& UserRef : UsersToQuery)
	{
		const FUniqueNetIdGDKRef GDKUserRef = StaticCastSharedRef<const FUniqueNetIdGDK>(UserRef);
		if (GDKUserRef->IsValid())
		{
			XUIDs.Add(GDKUserRef->ToUint64());
		}
	}

	// Ensure we have some players
	if (XUIDs.Num() < 1)
	{
		OnlineError.SetFromErrorCode(TEXT("Error starting user privacy permissions query, no permissions were requested"));
		UE_LOG_ONLINE(Warning, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	// Ensure we don't have TOO many players
	// TODO: figure out actual maximum and replace the 50 here
	if (XUIDs.Num() > 50)
	{
		OnlineError.SetFromErrorCode(TEXT("Error starting user privacy permissions query, more than 50 users were requested"));
		UE_LOG_ONLINE(Warning, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	// Create our task
	HRESULT Result = XblPrivacyBatchCheckPermissionAsync(GDKContext, PermissionsToQuery.GetData(), PermissionsToQuery.Num(), XUIDs.GetData(), XUIDs.Num(), nullptr, 0, *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error starting user privacy permissions query, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryUserPrivacyPermissions::ProcessResults()
{
	uint64 ResultSizeInBytes = 0;
	HRESULT Result = XblPrivacyBatchCheckPermissionResultSize(*AsyncBlock, &ResultSizeInBytes);
	if (Result == S_OK)
	{
		TArray<uint8> ResultBuffer;
		XblPermissionCheckResult* PermissionResults = nullptr;
		uint64 NumPermissionResults = 0;
		ResultBuffer.Reserve(ResultSizeInBytes);
		Result = XblPrivacyBatchCheckPermissionResult(*AsyncBlock, ResultSizeInBytes, ResultBuffer.GetData(), &PermissionResults, &NumPermissionResults, nullptr);
		if (Result == S_OK)
		{
			// Read our records
			for (int32 PermissionRecordIndex = 0; PermissionRecordIndex < NumPermissionResults; ++PermissionRecordIndex)
			{
				XblPermissionCheckResult& UserPermissionsResult = PermissionResults[PermissionRecordIndex];

				FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Create(UserPermissionsResult.targetXuid);

				TMap<XblPermission, bool>& PermissionsMap = UserPermissionsMap.FindOrAdd(GDKUserId);
				PermissionsMap.Add(UserPermissionsResult.permissionRequested, UserPermissionsResult.isAllowed);
			}
			OnlineError.bSucceeded = true;
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			OnlineError.SetFromErrorCode(FString::Printf(TEXT("Error querying user privacy permissions, error: (0x%0.8X)."), Result));
			UE_LOG_ONLINE(Error, TEXT("%s"), *OnlineError.ErrorCode);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		OnlineError.SetFromErrorCode(FString::Printf(TEXT("Error querying user privacy permissions, error: (0x%0.8X)."), Result));
		UE_LOG_ONLINE(Error, TEXT("%s"), *OnlineError.ErrorCode);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryUserPrivacyPermissions::Finalize()
{
	// TODO: cache our results?
}

void FOnlineAsyncTaskGDKQueryUserPrivacyPermissions::TriggerDelegates()
{
	FUniqueNetIdGDKRef RequestingNetId = FUniqueNetIdGDK::Create(GDKContext);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryUserPrivacyPermissions_TriggerDelegates);
	Delegate.ExecuteIfBound(OnlineError, RequestingNetId, UserPermissionsMap);
}

#endif //WITH_GRDK