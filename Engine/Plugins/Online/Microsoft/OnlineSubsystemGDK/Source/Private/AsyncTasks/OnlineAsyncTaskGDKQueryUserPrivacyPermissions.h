// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/privacy_c.h>
THIRD_PARTY_INCLUDES_END

class FOnlineSubsystemGDK;

/** Map of User's Permission's Allowed status */
using FPrivacyPermissionsResultsMap = TUniqueNetIdMap<TMap<XblPermission, bool>>;

/**
 * Delegate used when the user's privacy permissions request has completed
 *
 * @param RequestStatus If this request was successful or not
 * @param RequestingUser The user who generated this request
 * @param Results A Map of Users to permission results
 */
DECLARE_DELEGATE_ThreeParams(FOnGDKUserPrivacyPermissionsQueryComplete, const FOnlineError& /*RequestStatus*/, const FUniqueNetIdRef& /*RequestingUser*/, const FPrivacyPermissionsResultsMap& /*Results*/);

/**
 * Async Task to query the account details of a user
 */
class FOnlineAsyncTaskGDKQueryUserPrivacyPermissions
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryUserPrivacyPermissions(FOnlineSubsystemGDK* InGDKSubsystem,
		FGDKContextHandle InGDKContext,
		const TArray<FUniqueNetIdRef>& InUsersToQuery,
		const TArray<XblPermission>& InPermissionsToQuery,
		const FOnGDKUserPrivacyPermissionsQueryComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryUserPrivacyPermissions() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryUserPrivacyPermissions"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();
	virtual void TriggerDelegates();

protected:
	TArray<FUniqueNetIdRef> UsersToQuery;
	TArray<XblPermission> PermissionsToQuery;
	FOnGDKUserPrivacyPermissionsQueryComplete Delegate;
	FOnlineError OnlineError;
	FPrivacyPermissionsResultsMap UserPermissionsMap;
	FGDKContextHandle GDKContext;
};
