// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKQueryUsers.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineUserInterfaceGDK.h"

FOnlineAsyncTaskGDKQueryUsers::FOnlineAsyncTaskGDKQueryUsers(FOnlineSubsystemGDK* InGDKSubsystem,
															   FGDKContextHandle InGDKContext,
															   const TArray<FUniqueNetIdRef>& InUsersToQuery,
															   int32 InLocalUserNum,
															   const FOnQueryUserInfoComplete& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKQueryUsers"))
	, UsersToQuery(InUsersToQuery)
	, LocalUserNum(InLocalUserNum)
	, Delegate(InDelegate)
	, OnlineError(false)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKQueryUsers::Initialize()
{
	// Convert our TArray of UniqueNetIds into a IVectorView of XUID strings
	TArray<uint64> XUIDs;
	for (const FUniqueNetIdRef& UserRef : UsersToQuery)
	{
		const FUniqueNetIdGDKRef GDKUserRef = StaticCastSharedRef<const FUniqueNetIdGDK>(UserRef);
		XUIDs.Add(GDKUserRef->ToUint64());
	}

	HRESULT Result = XblProfileGetUserProfilesAsync(GDKContext, XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error starting user account details query, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryUsers::ProcessResults()
{
	size_t FoundProfileCount = 0;
	HRESULT Result = XblProfileGetUserProfilesResultCount(*AsyncBlock, &FoundProfileCount);
	if (Result == S_OK)
	{
		TArray<XblUserProfile> Profiles;
		Profiles.SetNumZeroed(FoundProfileCount);
		Result = XblProfileGetUserProfilesResult(*AsyncBlock, Profiles.Num(), Profiles.GetData());
		if (Result == S_OK)
		{
			UserInfoMap.Empty(FoundProfileCount);
			for (const XblUserProfile& Profile : Profiles)
			{
				TSharedRef<FOnlineUserInfoGDK> NewUser = MakeShared<FOnlineUserInfoGDK>(&Profile);
				UserInfoMap.Add(NewUser->GetUserId(), NewUser);
			}

			OnlineError.bSucceeded = true;
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			OnlineError.bSucceeded = false;
			OnlineError.SetFromErrorCode(TEXT("Unable to query users"));
			UE_LOG_ONLINE(Error, TEXT("Error querying user account details, error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		OnlineError.bSucceeded = false;
		OnlineError.SetFromErrorCode(TEXT("Unable to query users"));
		UE_LOG_ONLINE(Error, TEXT("Error querying user account details, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKQueryUsers::Finalize()
{
	FOnlineUserGDKPtr UserIntPtr = Subsystem->GetUsersGDK();
	if (UserIntPtr.IsValid())
	{
		for (TUniqueNetIdMap<TSharedRef<FOnlineUserInfoGDK>>::ElementType& Pair : UserInfoMap)
		{
			UserIntPtr->UsersMap.Add(Pair.Key, Pair.Value);
		}
	}
}

void FOnlineAsyncTaskGDKQueryUsers::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKQueryUsers_TriggerDelegates);
	Delegate.Broadcast(LocalUserNum, OnlineError.bSucceeded, UsersToQuery, OnlineError.ErrorCode);
}

#endif //WITH_GRDK