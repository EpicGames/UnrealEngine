// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineUserInterfaceGDK.h"
#include "OnlineError.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to query the account details of a user
 */
class FOnlineAsyncTaskGDKQueryUsers
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKQueryUsers(FOnlineSubsystemGDK* InGDKSubsystem,
								   FGDKContextHandle InGDKContext,
								   const TArray<FUniqueNetIdRef>& InUsersToQuery,
								   int32 InLocalUserNum,
								   const FOnQueryUserInfoComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKQueryUsers() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKQueryUsers"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void Finalize();
	virtual void TriggerDelegates();

protected:
	TArray<FUniqueNetIdRef> UsersToQuery;
	int32 LocalUserNum;
	FOnQueryUserInfoComplete Delegate;
	FOnlineError OnlineError;
	TUniqueNetIdMap<TSharedRef<FOnlineUserInfoGDK>> UserInfoMap;
	FGDKContextHandle GDKContext;
};
