// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKReadLeaderboards.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineLeaderboardInterfaceGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetLeaderboard.h"

FOnlineAsyncTaskGDKReadLeaderboards::FOnlineAsyncTaskGDKReadLeaderboards(
	FOnlineSubsystemGDK* InGDKSubsystem,
	FGDKContextHandle InGDKContext,
	TArray<FOnlineLeaderboardReadRef> InLeaderboardReads,
	const FOnlineLeaderboardReadRef& InReadObject,
	const TArray<FUniqueNetIdRef>& InPlayers,
	XblSocialGroupType InSocialGroupType,
	const FString& InStatName,
	bool InSkipToUser,
	const FOnReadLeaderboardsCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskBasic(InGDKSubsystem),
	LeaderboardReads(InLeaderboardReads),
	ReadObject(InReadObject),
	Players(InPlayers),
	Delegate(InDelegate),
	SocialGroupType(InSocialGroupType),
	SkipToUser(InSkipToUser),
	GDKContext(InGDKContext),
	StatName(InStatName)
{
}

void FOnlineAsyncTaskGDKReadLeaderboards::Initialize()
{
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;

	// Clear out any existing data
	ReadObject->Rows.Empty();

	LeaderboardReads.Empty();

	for (FUniqueNetIdRef Player : Players)
	{
		FOnlineLeaderboardReadRef LeaderboardRead = FOnlineLeaderboardReadRef(new FOnlineLeaderboardRead());
		LeaderboardRead->LeaderboardName = ReadObject->LeaderboardName;
		for (const FColumnMetaData& ColumnMetaData : ReadObject->ColumnMetadata)
		{
			LeaderboardRead->ColumnMetadata.Add(ColumnMetaData);
		}
		LeaderboardRead->SortedColumn = ReadObject->SortedColumn;
		LeaderboardReads.Add(LeaderboardRead);

		FUniqueNetIdGDKRef GDKId = StaticCastSharedRef<const FUniqueNetIdGDK>(Player);
		uint64 PlayerId = GDKId->ToUint64();

		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGetLeaderboard>(
			Subsystem,
			GDKContext,
			ReadObject->LeaderboardName,
			PlayerId,
			SocialGroupType,
			LeaderboardRead,
			SkipToUser,
			StatName,
			true,
			FOnGetLeaderboardCompleteDelegate::CreateRaw(this, &FOnlineAsyncTaskGDKReadLeaderboards::OnGetLeaderboardComplete));
	}

	//SHouldn't have to do this now - This class is handling both. 
		//FOnlineAsyncTaskGDKGetLeaderboardForUsers* AsyncTask = new FOnlineAsyncTaskGDKGetLeaderboardForUsers(GDKSubsystem, LeaderboardReads, ReadObject);
		//AsyncTaskManager->AddToInQueue(AsyncTask);
}

void FOnlineAsyncTaskGDKReadLeaderboards::Tick()
{
	//wait for all LeaderboardReads to be populated with data
	bool bComplete = true;
	for(int i = 0; i < LeaderboardReads.Num(); i++)
	{
		if(LeaderboardReads[i]->ReadState == EOnlineAsyncTaskState::NotStarted || LeaderboardReads[i]->ReadState == EOnlineAsyncTaskState::InProgress)
		{
			bComplete = false;
			break;
		}
	}

	//Once all requests are completed, if all LeaderboardReads were successful then this async task was successful
	if(bComplete)
	{
		bool bSuccessful = true;
		for(int i = 0; i < LeaderboardReads.Num(); i++)
		{
			if(LeaderboardReads[i]->ReadState != EOnlineAsyncTaskState::Done)
			{
				bSuccessful = false;
				break;
			}
		}
		bWasSuccessful = bSuccessful;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKReadLeaderboards::OnGetLeaderboardComplete(bool bSuccess)
{
	//WMM Instead of ticking, we should only handle things on callback.
}

void FOnlineAsyncTaskGDKReadLeaderboards::Finalize()
{
	FOnlineAsyncTaskBasic::Finalize();
	if (bWasSuccessful && LeaderboardReads.Num() > 0)
	{		
		//For each leaderboard result returned by an async request
		for(int ResultIndex = 0; ResultIndex < LeaderboardReads.Num(); ResultIndex++)
		{
			//take every row in the result and copy it into the single FOnlineLeaderboardRead passed into this request
			for(int ResultRow = 0; ResultRow < LeaderboardReads[ResultIndex]->Rows.Num(); ResultRow++)
			{
				//add the leaderboard row into the single FOnlineLeaderboardRead in sorted order by rank
				int InsertIndex = 0;
				for(; InsertIndex < ReadObject->Rows.Num(); InsertIndex++)
				{
					if(ReadObject->Rows[InsertIndex].Rank >= LeaderboardReads[ResultIndex]->Rows[ResultRow].Rank)
					{
						break;
					}
				}
				ReadObject->Rows.Insert(LeaderboardReads[ResultIndex]->Rows[ResultRow], InsertIndex);
			}
		}
	}

	ReadObject->ReadState = bWasSuccessful ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
}

void FOnlineAsyncTaskGDKReadLeaderboards::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKReadLeaderboards_TriggerDelegates);
	Delegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK