// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineLeaderboardInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineStats.h"
#include "OnlineSubsystemGDKTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/leaderboard_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"


/**
 *	Async task to retrieve a leaderboard containing a list of users from Live
 */
class FOnlineAsyncTaskGDKReadLeaderboards : public FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>, public TSharedFromThis<FOnlineAsyncTaskGDKReadLeaderboards, ESPMode::ThreadSafe>
{
public:
	/**
	 * Constructor
	 *
	 * @param InGDKSubsystem - The GDKOnlineSubsystem used to retrieve the leaderboards interface
	 * @param InLeaderboardReads - The array of LeaderboardReads to consolidate into the ReadObject
	 * @param InReadObject - The storage location for the returned leaderboard data
	 */
		FOnlineAsyncTaskGDKReadLeaderboards(
			FOnlineSubsystemGDK* InGDKSubsystem,
			FGDKContextHandle InGDKContext,
			TArray<FOnlineLeaderboardReadRef> InLeaderboardReads,
			const FOnlineLeaderboardReadRef& InReadObject,
			const TArray<FUniqueNetIdRef>& InPlayers,
			XblSocialGroupType InSocialGroupType,
			const FString& InStatName,
			bool InSkipToUser,
			const FOnReadLeaderboardsCompleteDelegate& InDelegate);

	/**
	 *	Get a human readable description of task
	 */
	virtual FString ToString() const { return FString::Printf(TEXT("FOnlineAsyncTaskGDKReadLeaderboards bWasSuccessful: %d"), WasSuccessful());}
	virtual void Initialize();
	virtual void Tick();
	virtual void Finalize();
	virtual void TriggerDelegates();
	void OnGetLeaderboardComplete(bool bSuccess);

private:
	/** List of ReadObjects to be populated by single user leaderboard requests */
	TArray<FOnlineLeaderboardReadRef> LeaderboardReads;
	/** Handle to the read object where the data will be stored */
	FOnlineLeaderboardReadPtr ReadObject;
	TArray<FUniqueNetIdRef> Players;
	FOnReadLeaderboardsCompleteDelegate Delegate;
	XblSocialGroupType SocialGroupType;
	bool SkipToUser;
	FGDKContextHandle GDKContext;
	FString StatName;
};
