// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKRedeemCode.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineAsyncTaskGDKQueryReceipts.h"
#include "OnlineSubsystemGDK.h"
#include "Misc/CoreDelegates.h"
#include "OnlineStoreInterfaceGDK.h"

FOnlineAsyncTaskGDKRedeemCode::FOnlineAsyncTaskGDKRedeemCode(FOnlineSubsystemGDK* const InGDKInterface, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdGDK, const FRedeemCodeRequest& InRedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& InDelegate)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKRedeemCode"), 0)
	, LocalUser(InGDKUser)
	, GDKContext(InGDKContext)
	, NetIdGDK(InNetIdGDK)
	, Delegate(InDelegate)
	, RedeemCodeRequest(InRedeemCodeRequest)
{
}


void FOnlineAsyncTaskGDKRedeemCode::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	XStoreContextHandle StoreContextHandle = Subsystem->GetStoreGDK()->GetStoreContextHandle(LocalUser);
	if (!StoreContextHandle)
	{
		ErrorResult.SetFromErrorMessage(NSLOCTEXT("OnlineSubsystemLive", "RedeemCodeFailed", "Failed to find or create store context."));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XStoreShowRedeemTokenUIAsync(StoreContextHandle, TCHAR_TO_UTF8(*RedeemCodeRequest.Code), nullptr, 0, false, *AsyncBlock);
	if(Result != S_OK)
	{
		ErrorResult.SetFromErrorMessage(NSLOCTEXT("OnlineSubsystemLive", "RedeemCodeFailed", "Unable to redeem code."));
		bWasSuccessful = false;
		bIsComplete = true;
		UE_LOG_ONLINE(Warning, TEXT("Error starting Code Redemption: %s error: (0x%0.8X)."), *RedeemCodeRequest.Code, Result);
	}
}

void FOnlineAsyncTaskGDKRedeemCode::ProcessResults()
{
	HRESULT Result = XStoreShowRedeemTokenUIResult(*AsyncBlock);
	if (Result == S_OK)
	{
		UE_LOG_ONLINE(Log, TEXT("Redeem Code: Code Redemption UI now displaying."));
		bWasSuccessful = true;
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Code Redemption UI, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
	}

	bIsComplete = true;
}

void FOnlineAsyncTaskGDKRedeemCode::Finalize()
{
	ErrorResult.bSucceeded = bWasSuccessful;
	if (bWasSuccessful)
	{
		Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryReceipts>(Subsystem, NetIdGDK, GDKContext, Delegate);
	}
}

void FOnlineAsyncTaskGDKRedeemCode::TriggerDelegates()
{
	FOnlinePurchaseGDKPtr PurchaseInt = Subsystem->GetPurchaseGDK();
	if (PurchaseInt.IsValid())
	{
		PurchaseInt->bIsCurrentlyInCheckout = false;
	}

	// Only trigger delegates now if we failed, otherwise our QueryReceipts call will do it for us
	if (!bWasSuccessful)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKRedeemCode_TriggerDelegates);
		Delegate.ExecuteIfBound(ErrorResult, MakeShared<FPurchaseReceipt>());
	}
}

#endif //WITH_GRDK
