// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineError.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

/**
 * Async task to redeem a code
 */
class FOnlineAsyncTaskGDKRedeemCode
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKRedeemCode(FOnlineSubsystemGDK* const InGDKInterface, FGDKUserHandle InGDKUser, FGDKContextHandle InGDKContext, const FUniqueNetIdGDKRef& InNetIdGDK, const FRedeemCodeRequest& InRedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& InDelegate);
	virtual ~FOnlineAsyncTaskGDKRedeemCode() = default;

	// FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKRedeemCode"); }

	// FOnlineAsyncItemBasic
	virtual void Tick() override;

	virtual void ProcessResults() override;

	// FOnlineAsyncTask Interface
	// Move results and trigger delegates in Game Thread
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

	void OnApplicationReactivated();

protected:
	FGDKUserHandle LocalUser;
	FGDKContextHandle GDKContext;
	FUniqueNetIdGDKRef NetIdGDK;
	FOnPurchaseRedeemCodeComplete Delegate;
	FOnlineError ErrorResult;
	FRedeemCodeRequest RedeemCodeRequest;
	bool bTaskStarted = false;
};
