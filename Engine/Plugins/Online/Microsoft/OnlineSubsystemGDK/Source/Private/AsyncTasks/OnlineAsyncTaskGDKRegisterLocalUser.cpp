// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKRegisterLocalUser.h"
#include "OnlineSubsystemGDKPrivate.h"
#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#endif //WITH_ENGINE
#include "Misc/Base64.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineAsyncTaskGDKSetSessionActivity.h"
#include "SocketSubsystem.h"
#include "Misc/Base64.h"
#include "Misc/OutputDeviceRedirector.h"

FOnlineAsyncTaskGDKRegisterLocalUser::FOnlineAsyncTaskGDKRegisterLocalUser(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	const FName InSessionName,
	FUniqueNetIdGDKRef InUserId,
	FGDKMultiplayerSessionHandle InGDKSession,
	XblMultiplayerSessionChangeTypes InSubscriptionType,
	const FOnRegisterLocalPlayerCompleteDelegate& InDelegate,
	TArray<const XblMultiplayerSessionMember*> InInitializationGroup)
	: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, TEXT("FOnlineAsyncTaskGDKRegisterLocalUser"), InContext, InSessionName, InGDKSession)
	, UserId(InUserId)
	, SubscriptionType(InSubscriptionType)
	, Delegate(InDelegate)
	, Result(EOnJoinSessionCompleteResult::Success)
	, InitializationGroup(InInitializationGroup)
{
}

bool FOnlineAsyncTaskGDKRegisterLocalUser::UpdateSession(FGDKMultiplayerSessionHandle Session)
{
	// Don't join if the session is full.
	FGDKUserHandle UserHandle;
	XblContextGetUser(GDKContext, UserHandle.GetInitReference());

	if (!Subsystem->GetSessionInterfaceGDK()->GetMpsdImpl()->CanUserJoinSession(UserHandle, Session))
	{
		Result = EOnJoinSessionCompleteResult::SessionIsFull;
		return false;
	}

	XblMultiplayerSessionJoin(Session, nullptr, true, true);
	
	// The client's external/WAN address should be set for the user so that other clients can initiate
	// connections.  Xbox platform doesn't currently have a way to retrieve this address so the local
	// address is used instead.  This means that clients will only be able to connect to each other if
	// they are on the same local network.
	bool BindAll = false;
	TSharedRef<FInternetAddr> LocalIp = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, BindAll);
#if WITH_ENGINE
	LocalIp->SetPort(FURL::UrlConfig.DefaultPort);
#else
	FString Port;
	// Allow the command line to override the default port
	if (FParse::Value(FCommandLine::Get(),TEXT("Port="),Port) == false)
	{
		Port = GConfig->GetStr( TEXT("URL"), TEXT("Port"), GEngineIni );
	}
	LocalIp->SetPort(FCString::Atoi( *Port ));
#endif //WITH_ENGINE
	if (LocalIp->IsValid())
	{
		XblMultiplayerSessionCurrentUserSetSecureDeviceAddressBase64(Session, TCHAR_TO_UTF8(*FBase64::Encode(LocalIp->ToString(true))));
	}

	// Indicate what events to subscribe to
	XblMultiplayerSessionSetSessionChangeSubscription(Session, XblMultiplayerSessionChangeTypes::Everything);

	// Specify init groups if necessary
	if (InitializationGroup.Num() > 0)
	{
		TArray<uint32> InitializationGroupIds;
		for (const XblMultiplayerSessionMember* Member : InitializationGroup)
		{
			InitializationGroupIds.Add(Member->MemberId);
		}
		XblMultiplayerSessionCurrentUserSetMembersInGroup(Session, InitializationGroupIds.GetData(), InitializationGroupIds.Num());
	}
	return true;
}

void FOnlineAsyncTaskGDKRegisterLocalUser::TriggerDelegates()
{
	if (!WasSuccessful())
	{
		Result = EOnJoinSessionCompleteResult::UnknownError;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKRegisterLocalUser_TriggerDelegates);
	Delegate.ExecuteIfBound(*UserId, Result);
}

void FOnlineAsyncTaskGDKRegisterLocalUser::Finalize()
{
	FOnlineAsyncTaskGDKSafeWriteSession::Finalize();

	// Set this as the user's activity
	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	if (SessionInt.IsValid())
	{
		SessionInt->GetMpsdImpl()->SetUserActiveSessionActivity(*UserId, GetLatestGDKSession());
	}
}

#endif //WITH_GRDK