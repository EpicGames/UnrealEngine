// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSafeWriteSession.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemGDKTypes.h"

/** 
 * Async task used to have another local user Join() a Live session.
 */
class FOnlineAsyncTaskGDKRegisterLocalUser : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	FOnlineAsyncTaskGDKRegisterLocalUser(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FUniqueNetIdGDKRef InUserId,
		FGDKMultiplayerSessionHandle InGDKSession,
		XblMultiplayerSessionChangeTypes InSubscriptionType,
		const FOnRegisterLocalPlayerCompleteDelegate& InDelegate,
		TArray<const XblMultiplayerSessionMember*> InInitializationGroup);

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("RegisterLocalUser"); }
	virtual void TriggerDelegates() override;
	virtual void Finalize() override;

private:

	// FOnlineAsyncTaskGDKSafeWriteSession
	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session);

	FUniqueNetIdGDKRef UserId;
	XblMultiplayerSessionChangeTypes SubscriptionType;
	FOnRegisterLocalPlayerCompleteDelegate Delegate;

	EOnJoinSessionCompleteResult::Type Result;

	// Allow initialization group to be specified, so QoS knows which users are on the same console
	TArray<const XblMultiplayerSessionMember*> InitializationGroup;
};
