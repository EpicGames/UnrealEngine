// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKResolvePrivilegeWithUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKResolvePrivilegeWithUI::FOnlineAsyncTaskGDKResolvePrivilegeWithUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	FGDKUserHandle InGDKUser,
	XUserPrivilegeOptions InOptions,
	XUserPrivilege InPrivilege,
	const FOnQueryGDKResolvePrivilegeWithUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKResolvePrivilegeWithUI"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKUser(InGDKUser)
	, Options(InOptions)
	, Privilege(InPrivilege)
{
}

void FOnlineAsyncTaskGDKResolvePrivilegeWithUI::Initialize()
{
	HRESULT Result = XUserResolvePrivilegeWithUiAsync(GDKUser, Options, Privilege, *AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error calling XUserResolvePrivilegeWithUiAsync, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKResolvePrivilegeWithUI::ProcessResults()
{
	HRESULT Result = XUserResolvePrivilegeWithUiResult(*AsyncBlock);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("FOnlineAsyncTaskGDKResolvePrivilegeWithUI: Failed to resolve privilege with UI: error 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKResolvePrivilegeWithUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKResolvePrivilegeWithUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK