// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSafeWriteSession.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"

FOnlineAsyncTaskGDKSafeWriteSession::FOnlineAsyncTaskGDKSafeWriteSession(
	FOnlineSubsystemGDK* const InGDKSubsystem,
	const FString& AsyncTaskName,
	FGDKContextHandle InGDKContext,
	const FName InSessionName,
	FGDKMultiplayerSessionHandle InGDKSession,
	const int32 InMaxRetryCount
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, AsyncTaskName)
	, SessionWriteMode(XblMultiplayerSessionWriteMode::SynchronizedUpdate)
	, GDKContext(InGDKContext)
	, SessionName(InSessionName)
	, GDKSession(InGDKSession)
	, GDKSessionReference(nullptr)
	, MaxRetryCount(InMaxRetryCount)
	, TotalRetryAttempts(0)
	, TotalSessionWrites(0)
	, TotalSessionReads(0)
	, bUpdatedSession(false)
{
	check(Subsystem != nullptr);

	check(GDKSession.IsValid());

	GDKSessionReference = XblMultiplayerSessionSessionReference(GDKSession);
	check(GDKSessionReference != nullptr);

	check(GDKContext.IsValid());
	check(!SessionName.IsNone());
	check(MaxRetryCount > 0);
}

FOnlineAsyncTaskGDKSafeWriteSession::FOnlineAsyncTaskGDKSafeWriteSession(
	FOnlineSubsystemGDK* const InGDKSubsystem,
	const FString& AsyncTaskName,
	FGDKContextHandle InGDKContext,
	const FName InSessionName,
	const int32 InMaxRetryCount
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, AsyncTaskName)
	, SessionWriteMode(XblMultiplayerSessionWriteMode::SynchronizedUpdate)
	, GDKContext(InGDKContext)
	, SessionName(InSessionName)
	, GDKSession(nullptr)
	, GDKSessionReference(nullptr)
	, MaxRetryCount(InMaxRetryCount)
	, TotalRetryAttempts(0)
	, TotalSessionWrites(0)
	, TotalSessionReads(0)
	, bUpdatedSession(false)
{
	check(Subsystem != nullptr);
	check(GDKContext.IsValid());
	check(!SessionName.IsNone());
	check(MaxRetryCount > 0);
}

void FOnlineAsyncTaskGDKSafeWriteSession::OnFailed()
{
	bWasSuccessful = false;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKSafeWriteSession::Retry()
{
	if (TotalRetryAttempts >= MaxRetryCount)
	{
		OnFailed();
		return;
	}

	++TotalRetryAttempts;

	if (!GDKSession.IsValid())
	{
		++TotalSessionReads;
		RemoveAsyncBlock(AsyncBlock);
		AsyncBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock){
			ProcessGetSessionResult();
		});

		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::Retry XblMultiplayerGetSessionAsync called..."));
		HRESULT Result = XblMultiplayerGetSessionAsync(GDKContext, GDKSessionReference, *AsyncBlock);
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::Retry XblMultiplayerGetSessionAsync returned Result: (0x%0.8X)"), Result);

		if (Result != S_OK)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to get session from session reference."));
			OnFailed();
		}
	}
	else
	{
		TryWriteSession();
	}
}

void FOnlineAsyncTaskGDKSafeWriteSession::ProcessGetSessionResult()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessGetSessionResult XblMultiplayerGetSessionResult called..."));
	HRESULT Result = XblMultiplayerGetSessionResult(*AsyncBlock, GDKSession.GetInitReference());
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessGetSessionResult XblMultiplayerGetSessionResult returned Result: (0x%0.8X)"), Result);
	if (Result == S_OK)
	{
		TryWriteSession();
	}
	else
	{
		if (Result == HTTP_E_STATUS_NOT_FOUND)
		{
			GDKSession.Clear();
			Retry();
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to get session from session reference."));
			OnFailed();
		}
	}
}


void FOnlineAsyncTaskGDKSafeWriteSession::TryWriteSession()
{
	if (!GDKSession.IsValid())
	{
		OnFailed();
		return;
	}

	bUpdatedSession = UpdateSession(GDKSession);
	if (!bUpdatedSession)
	{
		// Subclass decided not to change the session, we're done here.
		bWasSuccessful = true;
		bIsComplete = true;
		return;
	}

	++TotalSessionWrites;
	RemoveAsyncBlock(WriteSessionAsyncBlock);
	WriteSessionAsyncBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock){
		ProcessWriteSessionResult();
	});

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::TryWriteSession XblMultiplayerSessionGetInfo called..."));
	const XblMultiplayerSessionInfo* SessionInfo = XblMultiplayerSessionGetInfo(GDKSession);
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::TryWriteSession XblMultiplayerSessionGetInfo returned"));

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::TryWriteSession XblMultiplayerWriteSessionByHandleAsync called..."));
	HRESULT Result = XblMultiplayerWriteSessionByHandleAsync(GDKContext, GDKSession, SessionWriteMode, SessionInfo->SearchHandleId, *WriteSessionAsyncBlock);
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::TryWriteSession XblMultiplayerWriteSessionByHandleAsync returned Result: (0x%0.8X)"), Result);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to write session."));
		OnFailed();
	}
}

void FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult()
{
	FGDKMultiplayerSessionHandle NewGDKSession;
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult XblMultiplayerWriteSessionByHandleResult called..."));
	HRESULT Result = XblMultiplayerWriteSessionByHandleResult(*WriteSessionAsyncBlock, NewGDKSession.GetInitReference());
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult XblMultiplayerWriteSessionByHandleResult returned Result: (0x%0.8X)."), Result);

	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult(): XblMultiplayerWriteSessionResult, Result: (0x%0.8X)."), Result);

	// CDA Need to handle both versions of collecting write result here. August and before returns it above, November and later requires the XblMultiplayerSessionWriteStatus call.
	if (Result == S_OK)
	{
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult XblMultiplayerSessionWriteStatus called..."));
		XblWriteSessionStatus status = XblMultiplayerSessionWriteStatus(NewGDKSession);
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::ProcessWriteSessionResult XblMultiplayerSessionWriteStatus returned"));
		if (status == XblWriteSessionStatus::OutOfSync)
		{
			// Save updated session so we can re-apply our changes to it
			GDKSession = NewGDKSession;
			Retry();
			return;
		}
		else if (FAILED(status))
		{
			OnFailed();
			return;
		}
		GDKSession = NewGDKSession;
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else if (Result == HTTP_E_STATUS_PRECOND_FAILED) //Records out of sync
	{
		// Save updated session so we can re-apply our changes to it
		GDKSession = NewGDKSession;
		Retry();
	}
	else
	{
		OnFailed();
	}
}

void FOnlineAsyncTaskGDKSafeWriteSession::Finalize()
{
	Subsystem->CacheGDKSession(SessionName, GDKSession);
	//WMM TODO: Make sure this doesn't break anything.. FOnlineSessionGDK::OnHostInvalid called this, seems like it should be safe to call whenever we write session
	Subsystem->GetSessionInterfaceGDK()->GetMpsdImpl()->DetermineSessionHost(SessionName, GDKSession);
}

void FOnlineAsyncTaskGDKSafeWriteSession::Initialize()
{
	check(GDKSessionReference != nullptr);

	if (!bIsComplete)
	{
		Retry();
	}
}

void FOnlineAsyncTaskGDKSafeWriteSession::Tick()
{
	if (AsyncBlock)
	{
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::Tick XblMultiplayerGetSessionAsync Status: (0x%0.8X)."), AsyncBlock->GetStatus());
	}
	if (WriteSessionAsyncBlock)
	{
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskGDKSafeWriteSession::Tick XblMultiplayerWriteSessionByHandleAsync Status: (0x%0.8X)."), WriteSessionAsyncBlock->GetStatus());
	}
}

#endif //WITH_GRDK