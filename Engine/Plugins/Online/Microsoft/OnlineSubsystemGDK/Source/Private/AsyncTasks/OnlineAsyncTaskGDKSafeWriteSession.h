// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

class FOnlineSubsystemGDK;

class FOnlineAsyncTaskGDKSafeWriteSession : public FOnlineAsyncTaskGDK
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InLiveContext the GDK context of the user who's updating the session
	 * @param InSessionName the name of the session to update
	 * @param InGDKSession the GDK session object to update
	 * @param MaxRetryCount number of times to retry the session write
	 */
	FOnlineAsyncTaskGDKSafeWriteSession(
		FOnlineSubsystemGDK* const InGDKSubsystem,
		const FString& AsyncTaskName,
		FGDKContextHandle InGDKContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession,
		const int32 InMaxRetryCount = DefaultMaxRetryCount);

	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.  It relies on overloading tasks to have populated
	 * at least the GDK Session Reference before Initialize is called in this
	 * base class.  In practice, this means tasks must be processed in Serial
	 * rather than Parallel, on the async task manager.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InGDKContext the GDK context of the user who's updating the session
	 * @param InSessionName the name of the session to update
	 * @param MaxRetryCount number of times to retry the session write
	 */
	FOnlineAsyncTaskGDKSafeWriteSession(
		FOnlineSubsystemGDK* const InGDKSubsystem,
		const FString& AsyncTaskName,
		FGDKContextHandle InLiveContext,
		const FName InSessionName,
		const int32 InMaxRetryCount);

	// FOnlineAsyncItem
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncTaskGDKSafeWriteSession %s"), *GetAsyncSafeWriteTaskInfoString());
	}

	FString GetAsyncSafeWriteTaskInfoString() const
	{
		return FString::Printf(TEXT("SessionName=[%s] DidSessionUpdate=[%d] UpdateAttempts=[%d/%d] SessionWrites=[%d] SessionReads=[%d]"),
			*SessionName.ToString(), bUpdatedSession, TotalRetryAttempts, MaxRetryCount, TotalSessionWrites, TotalSessionReads);
	}

	virtual void Finalize() override;
	virtual void Initialize() override;
	virtual void Tick() override;
	void ProcessGetSessionResult();
	virtual void ProcessWriteSessionResult();
	static const int32 DefaultMaxRetryCount = 5;

protected:
	FGDKMultiplayerSessionHandle GetLatestGDKSession() const { return GDKSession; }
	FName GetSessionName() const { return SessionName; }
	bool GetDidUpdateSession() const { return bUpdatedSession; }
	FGDKContextHandle GetGDKContext() const { return GDKContext; }
	const XblMultiplayerSessionReference& GetSessionReference() const { return *GDKSessionReference; }

	/** Called when a task has failed to mark it complete and as unsuccessful */
	void OnFailed();

	/** Called to start our write flow */
	void Retry();

private:

	/** Called when we have a LiveSession object to write to */
	void TryWriteSession();

	/**
	 * Overridden by subclasses to perform the update on the local copy of the session document.
	 *
	 * @param Session The session to update
	 * @return true if the session was updated, false if not
	 */
	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session) { return false; };

protected:
	/** What method of session write do we wish to use for this update */
	XblMultiplayerSessionWriteMode SessionWriteMode;
	/** Who this session is being written by */
	FGDKContextHandle GDKContext;
	/** Saved SessionName for completion delegates, if needed */
	FName SessionName;
	/** GDK Session object to be written to (may be null) */
	FGDKMultiplayerSessionHandle GDKSession;
	/** GDK Session Reference object (must not be null) */
	const XblMultiplayerSessionReference* GDKSessionReference;
	/** Maximum amount of times to try writing upon write-conflicts */
	int32 MaxRetryCount;
	/** Maximum amount of times to attempt our read/write flow */
	int32 TotalRetryAttempts;
	/** Total amount of times we sent a write request for this session */
	int32 TotalSessionWrites;
	/** Total amount of times we downloaded a new copy of the session document */
	int32 TotalSessionReads;
	/** Store whether a subclass modified the session, so it can be referred to in Finalize() */
	bool bUpdatedSession;

	FGDKAsyncBlockPtr WriteSessionAsyncBlock;
};
