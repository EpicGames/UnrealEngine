// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSanitizeString.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineError.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/string_verify_c.h>
THIRD_PARTY_INCLUDES_END

namespace 
{
	FString SanitizeString(const FString& StringToSanitize, const FString& OffensiveSubString)
	{
		FString SanitizedString = StringToSanitize;
		int32 iSubStringIndex = StringToSanitize.Find(OffensiveSubString);

		//if the substring is not found (can happen when the offensive word has whitespaces in the middle and the VerifyString still catches it)
		if (iSubStringIndex < 0)
		{
			iSubStringIndex = 0;
		}

		for (int32 iCharPos = iSubStringIndex; iCharPos < SanitizedString.Len(); ++iCharPos)
		{
			SanitizedString[iCharPos] = TEXT('*');
		}
		return SanitizedString;
	}
}

void FOnlineAsyncTaskGDKSanitizeSingleString::Initialize()
{
	HRESULT Result = XblStringVerifyStringAsync(GDKContext, TCHAR_TO_UTF8(*StringToSanitize), *AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKVerifyString - Error verifying single string. Error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSanitizeSingleString::ProcessResults()
{
	uint64 ResultSizeInBytes = 0;

	HRESULT Result = XblStringVerifyStringResultSize(*AsyncBlock, &ResultSizeInBytes);
	if (Result == S_OK)
	{
		TArray<uint8> StringVerifiedBuffer;
		XblVerifyStringResult* VerifyStringResult = nullptr;

		StringVerifiedBuffer.SetNumUninitialized(ResultSizeInBytes);
		uint64 BytesRead = 0;
		
		Result = XblStringVerifyStringResult(*AsyncBlock, ResultSizeInBytes, StringVerifiedBuffer.GetData(), &VerifyStringResult, &BytesRead);
		if (Result == S_OK)
		{
			if (VerifyStringResult && VerifyStringResult->resultCode != XblVerifyStringResultCode::Success)
			{
				SanitizedString = SanitizeString(StringToSanitize, UTF8_TO_TCHAR(VerifyStringResult->firstOffendingSubstring));
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKSanitizeSingleString - XblStringVerifyStringResult - Error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKSanitizeSingleString - XblStringVerifyStringResultSize - Error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSanitizeSingleString::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKSanitizeSingleString_TriggerDelegates);
	Delegate.ExecuteIfBound(bWasSuccessful, SanitizedString);
}

void FOnlineAsyncTaskGDKSanitizeMultipleStrings::Initialize()
{
	TArray<FTCHARToUTF8> StringArray;
	TArray<const char*> CharPtrArray;
	StringArray.Reserve(StringArrayToSanitize.Num());
	CharPtrArray.Reserve(StringArrayToSanitize.Num());

	for (const FString& StringToSanitize : StringArrayToSanitize)
	{
		FTCHARToUTF8& Converter = StringArray.Emplace_GetRef(*StringToSanitize);
		CharPtrArray.Emplace(Converter.Get());
	}

	HRESULT Result = XblStringVerifyStringsAsync(GDKContext, const_cast<const char**>(CharPtrArray.GetData()), StringArrayToSanitize.Num(), *AsyncBlock);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineAsyncTaskGDKSanitizeMultipleStrings - Error verifying multiple strings. Error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSanitizeMultipleStrings::ProcessResults()
{
	uint64 ResultSizeInBytes = 0;

	HRESULT Result = XblStringVerifyStringsResultSize(*AsyncBlock, &ResultSizeInBytes);
	if (Result == S_OK)
	{
		TArray<uint8> StringVerifiedBuffer;
		XblVerifyStringResult* VerifyStringResult = nullptr;

		StringVerifiedBuffer.SetNumUninitialized(ResultSizeInBytes);
		uint64 BytesRead = 0;
		uint64 ResultCount = 0;

		Result = XblStringVerifyStringsResult(*AsyncBlock, ResultSizeInBytes, StringVerifiedBuffer.GetData(), &VerifyStringResult, &ResultCount, &BytesRead);
		if (Result == S_OK)
		{
			for (uint64 ResultIndex = 0; ResultIndex < ResultCount; ++ResultIndex)
			{
				XblVerifyStringResult& StringVerified = VerifyStringResult[ResultIndex];
				//if resultCode is success conserve the string (it returns null)
				if (StringVerified.resultCode == XblVerifyStringResultCode::Success) 
				{
					SanitizedStringArray.Add(StringArrayToSanitize[ResultIndex]);
				}
				else
				{
					SanitizedStringArray.Add(SanitizeString(StringArrayToSanitize[ResultIndex], UTF8_TO_TCHAR(StringVerified.firstOffendingSubstring)));
				}
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKSanitizeMultipleStrings - XblStringVerifyStringsResult - Error: (0x%0.8X)."), Result);
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKSanitizeMultipleStrings - XblStringVerifyStringsResultSize - Error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSanitizeMultipleStrings::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKSanitizeMultipleStrings_TriggerDelegates);
	Delegate.ExecuteIfBound(bWasSuccessful, SanitizedStringArray);
}


#endif //WITH_GRDK