// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineStatsInterfaceGDK.h"
#include "OnlineError.h"

class FOnlineSubsystemGDK;

/**
 * Delegate used when the single string has been sanitized
 *
 * @param bSuccess if this delegate is successful
 * @param SanitizedString The sanitized string returned 
 */
DECLARE_DELEGATE_TwoParams(FOnlineAsyncTaskGDKSanitizeSingleStringComplete, bool /*bSuccess*/, const FString& /*SanitizedString*/);

/**
 * Async Task to sanitize profanities in single strings
 */
class FOnlineAsyncTaskGDKSanitizeSingleString : public FOnlineAsyncTaskGDK
{
public:

	FOnlineAsyncTaskGDKSanitizeSingleString(FOnlineSubsystemGDK* const InGDKSubsystem
		, FGDKContextHandle InGDKContext 
		, const FString& InStringToSanitize
		, const FOnlineAsyncTaskGDKSanitizeSingleStringComplete& InDelegate)
		: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKSanitizeSingleString"))
		, GDKContext(InGDKContext)
		, StringToSanitize(InStringToSanitize)
		, SanitizedString(InStringToSanitize)
		, Delegate(InDelegate)
	{
	}
	virtual FString ToString() const override
	{
		return FString(TEXT("FOnlineAsyncTaskGDKSanitizeSingleString"));
	}

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

private:
	FGDKContextHandle GDKContext;
	const FString StringToSanitize;
	FString SanitizedString;

	FOnlineAsyncTaskGDKSanitizeSingleStringComplete Delegate;
};

/**
 * Delegate used when the array of strings has been sanitized
 *
 * @param bSuccess if this delegate is successful
 * @param SanitizedStrings The sanitized string's array returned
 */
DECLARE_DELEGATE_TwoParams(FOnlineAsyncTaskGDKSanitizeMultipleStringsComplete, bool /*bSuccess*/, const TArray<FString>& /*SanitizedStrings*/);

/**
 * Async Task to Sanitize profanities in multiple strings
 */
class FOnlineAsyncTaskGDKSanitizeMultipleStrings : public FOnlineAsyncTaskGDK
{
public:

	FOnlineAsyncTaskGDKSanitizeMultipleStrings(FOnlineSubsystemGDK* const InGDKSubsystem
		, FGDKContextHandle InGDKContext
		, const TArray<FString>& InStringArrayToSanitize
		, const FOnlineAsyncTaskGDKSanitizeMultipleStringsComplete& InDelegate)
		: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKSanitizeMultipleStrings"))
		, GDKContext(InGDKContext)
		, StringArrayToSanitize(InStringArrayToSanitize)
		, Delegate(InDelegate)
	{
	}
	virtual FString ToString() const override
	{
		return FString(TEXT("FOnlineAsyncTaskGDKSanitizeMultipleStrings"));
	}

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

private:
	FGDKContextHandle GDKContext;
	const TArray<FString> StringArrayToSanitize;
	TArray<FString> SanitizedStringArray;

	FOnlineAsyncTaskGDKSanitizeMultipleStringsComplete Delegate;
};
