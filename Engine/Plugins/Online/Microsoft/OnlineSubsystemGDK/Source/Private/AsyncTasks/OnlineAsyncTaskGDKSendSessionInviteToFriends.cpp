// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSendSessionInviteToFriends.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "Interfaces/OnlineSessionInterface.h"

FOnlineAsyncTaskGDKSendSessionInviteToFriends::FOnlineAsyncTaskGDKSendSessionInviteToFriends(FOnlineSubsystemGDK* InGDKSubsystem,
																							   FGDKContextHandle InGDKContext,
																							   FGDKMultiplayerSessionHandle InGDKSession,
																							   const TArray<uint64>& InFriendsToInvite)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKSendSessionInviteToFriends"))
	, GDKSession(InGDKSession)
	, FriendsToInvite(InFriendsToInvite)
	, GDKContext(InGDKContext)
{
}

void FOnlineAsyncTaskGDKSendSessionInviteToFriends::Initialize()
{
	const XblMultiplayerSessionReference* GDKSessionReference = XblMultiplayerSessionSessionReference(GDKSession);

	HRESULT Result = XblMultiplayerSendInvitesAsync(GDKContext, GDKSessionReference, FriendsToInvite.GetData(), FriendsToInvite.Num(), Subsystem->TitleId, nullptr, nullptr, *AsyncBlock);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error starting SendSessionInvitesToFriends, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSendSessionInviteToFriends::ProcessResults()
{
	TArray<XblMultiplayerInviteHandle> ResultArray;
	ResultArray.Reserve(FriendsToInvite.Num());
	HRESULT Result = XblMultiplayerSendInvitesResult(*AsyncBlock, FriendsToInvite.Num(), ResultArray.GetData());
	if(Result == S_OK)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("SendSessionInvitesToFriends Success"));
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error SendSessionInvitesToFriends, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

#endif //WITH_GRDK