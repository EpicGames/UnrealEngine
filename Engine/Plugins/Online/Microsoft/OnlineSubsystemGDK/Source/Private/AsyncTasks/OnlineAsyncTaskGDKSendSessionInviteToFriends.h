// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to send a session invite to friends
 */
class FOnlineAsyncTaskGDKSendSessionInviteToFriends
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKSendSessionInviteToFriends(FOnlineSubsystemGDK* InGDKSubsystem,
												   FGDKContextHandle InGDKContext,
												   FGDKMultiplayerSessionHandle InGDKSession,
												   const TArray<uint64>& InFriendsToInvite);
	virtual ~FOnlineAsyncTaskGDKSendSessionInviteToFriends() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKSendSessionInviteToFriends"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

protected:
	FGDKMultiplayerSessionHandle GDKSession;
	TArray<uint64> FriendsToInvite;
	FGDKContextHandle GDKContext;
};
