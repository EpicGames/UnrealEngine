// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSessionBase.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemSessionSettings.h"
#include "OnlineSubsystemGDK.h"

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

FOnlineAsyncTaskGDKSessionBase::FOnlineAsyncTaskGDKSessionBase(
	class FOnlineSubsystemGDK* InGDKSubsystem,
	const FString& AsyncTaskName,
	int32		InUserIndex,
	FName		InSessionName,
	const FOnlineSessionSettings& NewSessionSettings
)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, AsyncTaskName, InUserIndex)
	, GDKSession(nullptr)
	, SessionSettings(NewSessionSettings)
	, SessionName(InSessionName)
	, SessionTemplateName()
	, SessionMaxSeats(0)
	, ClientMatchmakingCapable(false)
	, PartyEnabledSession(true)
	, CustomConstantsJson(FString(TEXT("{}")))
	, InitiatorUserIds(TArray<uint64>())
{
	ParseSessionSettings( &SessionSettings );
}


//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

FOnlineAsyncTaskGDKSessionBase::FOnlineAsyncTaskGDKSessionBase( FOnlineAsyncTaskGDKSessionBase* PreviousTask )
	: FOnlineAsyncTaskGDK( PreviousTask->Subsystem, PreviousTask->AsyncTaskName, PreviousTask->UserIndex )
{
	// Copy entire previous task data
	*this = *PreviousTask;

	//. But we are not complete or even started yet!
	bWasSuccessful = false;
	bIsComplete = false;
}

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

FOnlineAsyncTaskGDKSessionBase::~FOnlineAsyncTaskGDKSessionBase()
{
}

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

void FOnlineAsyncTaskGDKSessionBase::ParseSessionSettings( FOnlineSessionSettings* InSessionSettings )
{
	if ( InSessionSettings )
	{
		SessionMaxSeats = InSessionSettings->NumPublicConnections;

		const FOnlineSessionSetting* HopperSetting = InSessionSettings->Settings.Find(SETTING_MATCHING_HOPPER);
		if (HopperSetting)
		{
			HopperSetting->Data.GetValue(MatchHopperName);
		}

		const FOnlineSessionSetting* TimeoutSetting = InSessionSettings->Settings.Find( SETTING_MATCHING_TIMEOUT );
		if ( TimeoutSetting )
		{
			TimeoutSetting->Data.GetValue( MatchTimeout );
		}

		const FOnlineSessionSetting* AttributesSetting = InSessionSettings->Settings.Find( SETTING_MATCHING_ATTRIBUTES );
		if ( AttributesSetting )
		{
			AttributesSetting->Data.GetValue( MatchTicketAttributes );
		}

		const FOnlineSessionSetting* TemplateSetting = InSessionSettings->Settings.Find(SETTING_SESSION_TEMPLATE_NAME);
		if ( TemplateSetting )
		{
			TemplateSetting->Data.GetValue( SessionTemplateName );
		}

		const FOnlineSessionSetting* CustomJsonSetting = InSessionSettings->Settings.Find( SETTING_CUSTOM );
		if ( CustomJsonSetting )
		{
			FString InJson;

			CustomJsonSetting->Data.GetValue( InJson );

			if ( InJson.Len() )
			{
				CustomConstantsJson = InJson;
			}
		}

		const FOnlineSessionSetting* PartyEnabledSessionSetting = InSessionSettings->Settings.Find( SETTING_PARTY_ENABLED_SESSION );
		if( PartyEnabledSessionSetting )
		{
			PartyEnabledSessionSetting->Data.GetValue( PartyEnabledSession );
		}
	}
}

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

void FOnlineAsyncTaskGDKSessionBase::ParseSearchSettings( const FOnlineSessionSearch* InSearchSettings )
{
	if ( InSearchSettings )
	{
		InSearchSettings->QuerySettings.Get(SETTING_MATCHING_HOPPER, MatchHopperName);
		InSearchSettings->QuerySettings.Get(SETTING_SESSION_TEMPLATE_NAME, SessionTemplateName);
		MatchTimeout = InSearchSettings->TimeoutInSeconds;
	}
}

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

bool FOnlineAsyncTaskGDKSessionBase::SettingsAreValid()
{
	if (	MatchTimeout
		&&  SessionTemplateName.Len()
		&&  MatchHopperName.Len() )
	{
		return true;
	}

	return false;
}

//------------------------------- End of file ---------------------------------

#endif //WITH_GRDK