// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSessionSettings.h"

//-----------------------------------------------------------------------------
//
//-----------------------------------------------------------------------------

/**
 *  Async task base class for session related operations
 */
class FOnlineAsyncTaskGDKSessionBase : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKSessionBase(
	class FOnlineSubsystemGDK* GDKSubsystem,
		const FString& AsyncTaskName,
		int32		InUserIndex,
		FName		SessionName,
		const FOnlineSessionSettings& NewSessionSettings
		);

	//. Copy constructor for new tasks based on old (e.g. Matchmaking advertisements)
	FOnlineAsyncTaskGDKSessionBase( FOnlineAsyncTaskGDKSessionBase* PreviousTask );

	virtual ~FOnlineAsyncTaskGDKSessionBase();

	virtual	bool					SettingsAreValid();

	const FName&					GetSessionName()	{ return SessionName; }

protected:
	void							ParseSessionSettings( FOnlineSessionSettings* InSessionSettings );
	void							ParseSearchSettings( const FOnlineSessionSearch* InSearchSettings );

protected:
	FGDKMultiplayerSessionHandle		GDKSession;
	FOnlineSessionSettings			SessionSettings;
	FName							SessionName;
	FString							SessionTemplateName;
	uint32							SessionMaxSeats;

	FString							MatchHopperName;
	FString							MatchTicketAttributes;
	float							MatchTimeout;

	bool							ClientMatchmakingCapable;
	bool							PartyEnabledSession;
	FString							CustomConstantsJson;
	TArray<uint64>					InitiatorUserIds;
};


//------------------------------- End of file ---------------------------------
