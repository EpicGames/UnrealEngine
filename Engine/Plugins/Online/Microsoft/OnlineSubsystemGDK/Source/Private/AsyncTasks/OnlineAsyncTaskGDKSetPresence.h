// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKSetPresence
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKSetPresence(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FOnlineUserPresenceStatus& InPresenceStatus,
		const FString& InPresenceIdString,
		const FOnSetGDKPresenceCompleteDelegate& InTaskCompletionDelegate
);
	
	virtual ~FOnlineAsyncTaskGDKSetPresence() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKSetPresence"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnSetGDKPresenceCompleteDelegate TaskCompletionDelegate;
	const FOnlineUserPresenceStatus PresenceStatus;
	const FString PresenceIdString;
};
