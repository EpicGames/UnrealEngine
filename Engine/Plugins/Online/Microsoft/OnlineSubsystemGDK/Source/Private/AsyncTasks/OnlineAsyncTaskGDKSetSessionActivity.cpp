// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSetSessionActivity.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"

FOnlineAsyncTaskGDKSetSessionActivity::FOnlineAsyncTaskGDKSetSessionActivity(FOnlineSubsystemGDK* InGDKSubsystem,
																				   FGDKContextHandle InGDKContext,
																				   FGDKMultiplayerSessionHandle InGDKSession)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKSetSessionActivity"))
	, GDKSession(InGDKSession)
	, GDKContext(InGDKContext)
{
	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());
}

void FOnlineAsyncTaskGDKSetSessionActivity::Initialize()
{	
	UE_LOG_ONLINE_SESSION(Log, TEXT("Setting sessionActivity"));
	const XblMultiplayerSessionReference*SessionReference = XblMultiplayerSessionSessionReference(GDKSession);
	HRESULT Result = XblMultiplayerSetActivityAsync(GDKContext, SessionReference, *AsyncBlock);
	if(Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Error, TEXT("Error starting SetActivityAsync, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSetSessionActivity::ProcessResults()
{
	bWasSuccessful = true;
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKSetSessionActivity::TriggerDelegates()
{
	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	check(SessionInt.IsValid());
	const TSharedPtr<FOnlineSessionMpsdGDK>& SessionIntMpsd = SessionInt->GetMpsdImpl();

	SessionIntMpsd->ClearSessionActivityInProgress();
	
	FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(GDKContext);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKSetSessionActivity_TriggerDelegates);
	SessionIntMpsd->OnSetUserActiveSessionActivityComplete(*UserNetId, true);
}

#endif //WITH_GRDK