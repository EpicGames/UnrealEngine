// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to set the current session for a user
 */
class FOnlineAsyncTaskGDKSetSessionActivity
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKSetSessionActivity(FOnlineSubsystemGDK* InGDKSubsystem,
										   FGDKContextHandle InGDKContext,										   
										   FGDKMultiplayerSessionHandle GDKSession);

	virtual ~FOnlineAsyncTaskGDKSetSessionActivity() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKSetSessionActivity"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;

	virtual void TriggerDelegates() override;

protected:	
	FGDKMultiplayerSessionHandle GDKSession;
	FGDKContextHandle GDKContext;
};
