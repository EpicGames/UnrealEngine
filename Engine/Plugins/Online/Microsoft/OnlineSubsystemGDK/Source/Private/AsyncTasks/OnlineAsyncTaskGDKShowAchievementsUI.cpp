// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKShowAchievementsUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKShowAchievementsUI::FOnlineAsyncTaskGDKShowAchievementsUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	FGDKUserHandle InGDKUser,
	const FOnQueryGDKShowAchievementsUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKShowAchievementsUI"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKUser(InGDKUser)
{
}

void FOnlineAsyncTaskGDKShowAchievementsUI::Initialize()
{
	HRESULT Result = XGameUiShowAchievementsAsync(
		*AsyncBlock,
		GDKUser,
		Subsystem->TitleId);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Achievements UI, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowAchievementsUI::ProcessResults()
{
	HRESULT Result = XGameUiShowAchievementsResult(*AsyncBlock);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Failed to show invite UI with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	
}

void FOnlineAsyncTaskGDKShowAchievementsUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKShowAchievementsUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK