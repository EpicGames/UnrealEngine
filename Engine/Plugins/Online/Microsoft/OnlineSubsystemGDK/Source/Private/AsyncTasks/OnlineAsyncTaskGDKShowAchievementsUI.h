// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKShowAchievementsUI
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKShowAchievementsUI(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		FGDKUserHandle InGDKUser,
		const FOnQueryGDKShowAchievementsUICompleteDelegate& InTaskCompletionDelegate
);
	
	virtual ~FOnlineAsyncTaskGDKShowAchievementsUI() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKShowAchievementsUI"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnQueryGDKShowAchievementsUICompleteDelegate TaskCompletionDelegate;
	FGDKUserHandle GDKUser;
};
