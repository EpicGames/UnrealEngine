// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKShowLoginUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XUser.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKShowLoginUI::FOnlineAsyncTaskGDKShowLoginUI(
	FOnlineSubsystemGDK* InGDKInterface,
	bool bInAllowGuestLogin,
	const FOnQueryGDKShowLoginUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKShowLoginUI"))
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, bAllowGuestLogin(bInAllowGuestLogin)
{
}

void FOnlineAsyncTaskGDKShowLoginUI::Initialize()
{
	XUserAddOptions Options = bAllowGuestLogin ? XUserAddOptions::AllowGuests : XUserAddOptions::None;
	IGDKRuntimeModule::Get().PickUserAsync(Options, FGDKPickUserCompleteDelegate::CreateLambda([this](HRESULT hInResult, FGDKUserHandle User) {
		UE_CLOGF(FAILED(hInResult), LogOnline, Warning, "FOnlineAsyncTaskGDKShowLoginUI failed: 0x%0.8X", hInResult);
		NewUser = User;
		bWasSuccessful = SUCCEEDED(hInResult);
		hResult = hInResult;
		bIsComplete = true;
	}));
}

void FOnlineAsyncTaskGDKShowLoginUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKShowLoginUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful, hResult, NewUser);
}

#endif //WITH_GRDK