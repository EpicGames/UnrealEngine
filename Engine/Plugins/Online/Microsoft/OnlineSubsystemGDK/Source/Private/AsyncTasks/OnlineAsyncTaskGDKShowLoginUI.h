// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKShowLoginUI
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKShowLoginUI(
		FOnlineSubsystemGDK* InGDKInterface,
		bool bAllowGuestLogin,
		const FOnQueryGDKShowLoginUICompleteDelegate& InTaskCompletionDelegate);

	virtual ~FOnlineAsyncTaskGDKShowLoginUI() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKShowLoginUI"); }

	// Starts in Game Thread
	virtual void Initialize() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	const FOnQueryGDKShowLoginUICompleteDelegate TaskCompletionDelegate;
	FGDKUserHandle NewUser;
	HRESULT hResult;
	bool bAllowGuestLogin;
};
