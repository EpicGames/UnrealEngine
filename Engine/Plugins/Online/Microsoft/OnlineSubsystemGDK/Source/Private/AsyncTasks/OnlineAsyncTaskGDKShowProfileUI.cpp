// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKShowProfileUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKShowProfileUI::FOnlineAsyncTaskGDKShowProfileUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	FGDKUserHandle InGDKUser,
	uint64 InTargetUser,
	const FOnQueryGDKShowProfileUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKShowProfileUI"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKUser(InGDKUser)
	, TargetUser(InTargetUser)
{
}

void FOnlineAsyncTaskGDKShowProfileUI::Initialize()
{
	HRESULT Result = XGameUiShowPlayerProfileCardAsync(*AsyncBlock, GDKUser, TargetUser);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Profile UI, error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowProfileUI::ProcessResults()
{
	HRESULT Result = XGameUiShowPlayerProfileCardResult(*AsyncBlock);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("FOnlineAsyncTaskGDKShowProfileUI: Failed to show profile UI with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowProfileUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKShowProfileUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK