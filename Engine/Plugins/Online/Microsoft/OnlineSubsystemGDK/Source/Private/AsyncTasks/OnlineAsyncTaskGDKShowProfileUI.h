// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKShowProfileUI
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKShowProfileUI(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		FGDKUserHandle InGDKUser,
		uint64 InTargetUser,
		const FOnQueryGDKShowProfileUICompleteDelegate& InTaskCompletionDelegate
);
	
	virtual ~FOnlineAsyncTaskGDKShowProfileUI() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKShowProfileUI"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnQueryGDKShowProfileUICompleteDelegate TaskCompletionDelegate;
	FGDKUserHandle GDKUser;
	uint64 TargetUser;
};
