// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKShowSendGameInvitesUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKShowSendGameInvitesUI::FOnlineAsyncTaskGDKShowSendGameInvitesUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	FGDKMultiplayerSessionHandle InGDKSession,
	const FOnQueryGDKShowSendGameInvitesUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKShowSendGameInvitesUI"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, GDKSession(InGDKSession)
{
}

void FOnlineAsyncTaskGDKShowSendGameInvitesUI::Initialize()
{
	FGDKUserHandle RequestingUser;
	XblContextGetUser(GDKContext, RequestingUser.GetInitReference());
	const XblMultiplayerSessionReference* GDKSessionReference = XblMultiplayerSessionSessionReference(GDKSession);

	HRESULT Result = XGameUiShowSendGameInviteAsync(
		*AsyncBlock,
		RequestingUser,
		GDKSessionReference->Scid,
		GDKSessionReference->SessionTemplateName,
		GDKSessionReference->SessionName,
		nullptr,
		nullptr);

	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Send Game Invite UI, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowSendGameInvitesUI::ProcessResults()
{
	HRESULT Result = XGameUiShowSendGameInviteResult(*AsyncBlock);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Failed to show invite UI with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
	
}

void FOnlineAsyncTaskGDKShowSendGameInvitesUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKShowSendGameInvitesUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK