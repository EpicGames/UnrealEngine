// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKShowSendGameInvitesUI
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKShowSendGameInvitesUI(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		FGDKMultiplayerSessionHandle InGDKSession,
		const FOnQueryGDKShowSendGameInvitesUICompleteDelegate& InTaskCompletionDelegate
);
	
	virtual ~FOnlineAsyncTaskGDKShowSendGameInvitesUI() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKShowSendGameInvitesUI"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	// Process in Online Thread
	virtual void ProcessResults() override;

	// Move results and trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FGDKContextHandle GDKContext;
	const FOnQueryGDKShowAchievementsUICompleteDelegate TaskCompletionDelegate;
	FGDKMultiplayerSessionHandle GDKSession;
};
