// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKShowStoreUI.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStoreInterfaceGDK.h"

FOnlineAsyncTaskGDKShowStoreUI::FOnlineAsyncTaskGDKShowStoreUI(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const FShowStoreParams& InShowParams,
	const FOnQueryGDKShowStoreUICompleteDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKShowStoreUI"))
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, ShowParams(InShowParams)
{
}

void FOnlineAsyncTaskGDKShowStoreUI::Tick()
{
	if (bTaskStarted)
	{
		return;
	}
	bTaskStarted = true;

	if (ShowParams.ProductId.IsEmpty())
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Store UI, Browsing not supported - ProductId required for purchase."));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());

	if (Subsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Store UI, Title/Store ID Mismatch"));
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	XStoreContextHandle StoreContext = Subsystem->GetStoreGDK()->GetStoreContextHandle(GDKUser);
	if (!StoreContext)
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Store UI, failed to get store context."));

		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	HRESULT Result = XStoreShowPurchaseUIAsync(StoreContext, TCHAR_TO_UTF8(*ShowParams.ProductId), nullptr, nullptr, *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Error showing Store UI, error: (0x%0.8X)."), Result);

		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowStoreUI::ProcessResults()
{
	HRESULT Result = XStoreShowPurchaseUIResult(*AsyncBlock);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		bIsComplete = true;
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("FOnlineAsyncTaskGDKShowAchievementsUI: Failed to show store UI with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKShowStoreUI::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKShowStoreUI_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(bWasSuccessful);
}

#endif //WITH_GRDK