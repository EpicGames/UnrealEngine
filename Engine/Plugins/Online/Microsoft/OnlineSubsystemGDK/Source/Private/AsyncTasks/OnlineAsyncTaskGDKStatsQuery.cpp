// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "AsyncTasks/OnlineAsyncTaskGDKStatsQuery.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStatsInterfaceGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineError.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/user_statistics_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.stats"

FOnlineAsyncTaskGDKStatsQuery::FOnlineAsyncTaskGDKStatsQuery( FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InGDKContext, const int32 InUserIndex, const TUniqueNetIdMap<uint64>& InUserMapping, const FOnlineStatsQueryUserStatsComplete& InDelegate )
	: FOnlineAsyncTaskGDK( InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKStatsQuery"), InUserIndex)
	, UserMapping(InUserMapping)
	, GDKContext(InGDKContext)
	, SingleUserDelegate(InDelegate)
{
}

FOnlineAsyncTaskGDKStatsQuery::FOnlineAsyncTaskGDKStatsQuery( FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InGDKContext, const int32 InUserIndex, const TUniqueNetIdMap<uint64>& InUserMapping, const TArray<FString>& InStatNames, const FOnlineStatsQueryUsersStatsComplete& InDelegate )
	: FOnlineAsyncTaskGDK( InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKStatsQuery"), InUserIndex )
	, UserMapping(InUserMapping)
	, StatNames(InStatNames)
	, GDKContext(InGDKContext)
	, MultipleUsersDelegate(InDelegate)
{
}

void FOnlineAsyncTaskGDKStatsQuery::Initialize()
{
	bool bSuccess = false;

	// early out if there are users who are still pending registration
	TArray<uint64> QueryUserIds;
	for( auto& UserPair : UserMapping )
	{
		uint64 User = UserPair.Value;
		QueryUserIds.Add(User);
	}

	// Retrieve and prepare stat names for query
	if (StatNames.Num() == 0)
	{
		const FOnlinePresenceGDKPtr PresencePtr = Subsystem->GetPresenceGDK();
		if (PresencePtr.IsValid())
		{
			StatNames = PresencePtr->GetAutoSubscribePresenceStatNames();
		}
	}

	TArray<TArray<ANSICHAR>> StatisticNamesStore;
	TArray<const ANSICHAR*> StatNamesCharPtr;

	for (const FString& StatName : StatNames)
	{
		//Array to store Ansi strings, that will be cleaned up when task is destroyed (Avoiding explicit mallocs)
		TArray<ANSICHAR>& AnsiCharArray = StatisticNamesStore.AddDefaulted_GetRef();
		AnsiCharArray.Append(TCHAR_TO_ANSI(*StatName), StatName.Len() + 1);
		
		//Array to store char* that we can pass to GDK
		StatNamesCharPtr.Add(AnsiCharArray.GetData());
	}

	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);

	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsAsync(GDKContext, QueryUserIds.GetData(), QueryUserIds.Num(), Scid, StatNamesCharPtr.GetData(), StatNamesCharPtr.Num(), *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("FOnlineAsyncTaskLiveStatsQuery Error: (0x%0.8X)."), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKStatsQuery::ProcessResults()
{
	bWasSuccessful = false;

	uint64 ResultSizeInBytes = 0;
	HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsResultSize(*AsyncBlock, &ResultSizeInBytes);
	if (Result == S_OK)
	{
		TArray<uint8> ResultBuffer;
		ResultBuffer.SetNumUninitialized(ResultSizeInBytes);

		XblUserStatisticsResult* StatisticsResultsArray{};
		uint64 ResultsCount = 0;
		uint64 BytesRead = 0;

		// If a value for this stat has never been set for this user, the query will return no entry for it
		Result = XblUserStatisticsGetMultipleUserStatisticsResult(*AsyncBlock, ResultSizeInBytes, ResultBuffer.GetData(), &StatisticsResultsArray, &ResultsCount, &BytesRead);
		if (Result == S_OK)
		{
			FOnlineStatsGDKPtr GDKStats = Subsystem->GetStatsGDK();
			check(GDKStats.IsValid());

			for (uint64 ResultsIndex = 0; ResultsIndex < ResultsCount; ++ResultsIndex)
			{
				const XblUserStatisticsResult& StatisticsResultForUser = StatisticsResultsArray[ResultsIndex];

				FUniqueNetIdGDKRef UserId = FUniqueNetIdGDK::Create(StatisticsResultForUser.xboxUserId);
				TSharedRef<FOnlineStatsUserStats> UserStats = MakeShared<FOnlineStatsUserStats>(UserId);

				if (StatisticsResultForUser.serviceConfigStatisticsCount > 0)
				{
					for (uint32 i = 0; i < StatisticsResultForUser.serviceConfigStatisticsCount; ++i)
					{
						const XblServiceConfigurationStatistic& ServiceConfigStat = StatisticsResultForUser.serviceConfigStatistics[i];

						if (ServiceConfigStat.statisticsCount > 0)
						{
							for (uint32 j = 0; j < ServiceConfigStat.statisticsCount; ++j)
							{
								const XblStatistic& Stat = ServiceConfigStat.statistics[j];
								FString StatNameStr(UTF8_TO_TCHAR(Stat.statisticName));
								FOnlineStatValue& StatToUpdate = UserStats->Stats.Add(StatNameStr);
								GDKStats->CopyStatValue(StatToUpdate, Stat.value, Stat.statisticType);
							}
						}
					}
				}

				UserStatsResult.Emplace(UserStats);
			}
			bWasSuccessful = true;
			bIsComplete = true;
		}
		else
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("FOnlineAsyncTaskLiveStatsQuery - XblUserStatisticsGetMultipleUserStatisticsResult - Error: (0x%0.8X)."), Result);
			bIsComplete = true;
		}
	}
	else
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("FOnlineAsyncTaskLiveStatsQuery - XblUserStatisticsGetMultipleUserStatisticsResultSize - Error: (0x%0.8X)."), Result);
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKStatsQuery::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsQuery_TriggerDelegates);
	if (bWasSuccessful)
	{
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsQuery_TriggerDelegates_Success_Multiple);
			MultipleUsersDelegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), UserStatsResult);
		}
		if( UserStatsResult.Num() <= 1 )
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsQuery_TriggerDelegates_Success_Single);
			SingleUserDelegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::Success), UserStatsResult[0]);
		}
	}
	else
	{
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsQuery_TriggerDelegates_Failure_Multiple);		
			MultipleUsersDelegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), UserStatsResult);
		}
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsQuery_TriggerDelegates_Failure_Single);
			SingleUserDelegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidParams), nullptr);
		}
	}
}
#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK