// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "Interfaces/OnlineStatsInterface.h"
#include "OnlineStatsInterfaceGDK.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to query stats
 */
class FOnlineAsyncTaskGDKStatsQuery : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKStatsQuery( FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InGDKContext, const int32 InUserIndex, const TUniqueNetIdMap<uint64>& InUserMapping, const FOnlineStatsQueryUserStatsComplete& InDelegate );
	FOnlineAsyncTaskGDKStatsQuery( FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InGDKContext, const int32 InUserIndex, const TUniqueNetIdMap<uint64>& InUserMapping, const TArray<FString>& InStatNames, const FOnlineStatsQueryUsersStatsComplete& InDelegate );
	virtual FString ToString() const override { return TEXT("QueryStats"); }
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual void ProcessResults() override;

protected:
	TUniqueNetIdMap<uint64> UserMapping;
	TArray<FString> StatNames;
	FGDKContextHandle GDKContext;

	TArray<TSharedRef<const FOnlineStatsUserStats>> UserStatsResult;

	FOnlineStatsQueryUserStatsComplete SingleUserDelegate;
	FOnlineStatsQueryUsersStatsComplete MultipleUsersDelegate;
};
