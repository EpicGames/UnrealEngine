// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "AsyncTasks/OnlineAsyncTaskGDKStatsUpdate.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineStatsInterfaceGDK.h"
#include "OnlineError.h"
#include "GDKThreadCheck.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/title_managed_statistics_c.h>
THIRD_PARTY_INCLUDES_END

#define LOCTEXT_NAMESPACE "OnlineSubsystemLive"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.stats"

FOnlineAsyncTaskGDKStatsUpdate::FOnlineAsyncTaskGDKStatsUpdate(FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InUserContext, const TArray<FOnlineStatsUserUpdatedStats>& InUpdatedUserStats, const FOnlineStatsUpdateStatsComplete& InDelegate, const TMap<FUniqueNetIdGDKRef, uint64>& InGDKUserMapping)
	: FOnlineAsyncTaskGDK(InGDKSubsystem, TEXT("FOnlineAsyncTaskGDKStatsUpdate"))
	, GDKContext(InUserContext)
	, UpdatedUserStats(InUpdatedUserStats)
	, GDKUserMapping(InGDKUserMapping)
	, Delegate(InDelegate)
{
}

HRESULT FOnlineAsyncTaskGDKStatsUpdate::UpdateAllUserStats(FUniqueNetIdGDKRef GDKId)
{
	TArray<TArray<ANSICHAR>> CharBuffers;
	TArray<XblTitleManagedStatistic> StatisticArray;
	FOnlineStatsGDKPtr GDKStats = Subsystem->GetStatsGDK();
	uint64 User = GDKUserMapping.FindChecked(GDKId);

	// We need to iterate all of the users stats, and send them all as an updated document to Xbox Live
	TSharedPtr<const FOnlineStatsUserStats> UserStats = GDKStats->GetStats(GDKId);
	if (UserStats.IsValid())
	{
		for (TPair<FString, FVariantData> CurrentStat : UserStats->Stats)
		{
			XblTitleManagedStatistic& CurrentStatistic = StatisticArray.AddZeroed_GetRef();

			// We make a copy and a conversion of the FString Stat Name to const char*
			// This approach, however complicated, works better than its simple equivalents, which caused runtime errors 
			const FTCHARToUTF8 StatName(*CurrentStat.Key);
			int32 StatNameSize = FCStringAnsi::Strlen(StatName.Get()) + 1;
			char* StatNameCharPtr = (char*)FMemory::Malloc(StatNameSize);
			FCStringAnsi::Strncpy(StatNameCharPtr, StatName.Get(), StatNameSize);

			CurrentStatistic.statisticName = StatNameCharPtr;

			switch (CurrentStat.Value.GetType())
			{
			case EOnlineKeyValuePairDataType::Type::Int32:
			{
				int32 TmpValue = 0;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Int64:
			{
				int64 TmpValue = 0;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Double:
			{
				double TmpValue = 0.0f;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Float:
			{
				float TmpValue = 0.0f;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::UInt32:
			{
				uint32 TmpValue = 0;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::UInt64:
			{
				uint64 TmpValue = 0;
				CurrentStat.Value.GetValue(TmpValue);
				CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
				CurrentStatistic.numberValue = TmpValue;
				CurrentStatistic.stringValue = "";
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Blob:
			case EOnlineKeyValuePairDataType::Type::Bool:
			case EOnlineKeyValuePairDataType::Type::String:
			case EOnlineKeyValuePairDataType::Type::Json:
			case EOnlineKeyValuePairDataType::Type::Empty:
			{
				CurrentStatistic.statisticType = XblTitleManagedStatType::String;
				CurrentStatistic.numberValue = 0;
				TArray<ANSICHAR>& CharBuffer = CharBuffers.AddDefaulted_GetRef();
				const auto ConvertedStatName = TStringConversion<FTCHARToUTF8_Convert>(*CurrentStat.Value.ToString());
				CharBuffer.Append(ConvertedStatName.Get(), FCStringAnsi::Strlen(ConvertedStatName.Get()));
				CurrentStatistic.stringValue = CharBuffer.GetData();
				break;
			}

			default:
			{
				UE_LOG_ONLINE_STATS(Warning, TEXT("Encountered Unknown StatType %d"), CurrentStat.Value.GetType());
				CurrentStatistic.statisticType = XblTitleManagedStatType::String;
				CurrentStatistic.numberValue = 0;
				CurrentStatistic.stringValue = "";
				break;
			}
			}
		}
	}

	// This method is not updating stats successfully in this version of the GDK
	return XblTitleManagedStatsWriteAsync(GDKContext, User, StatisticArray.GetData(), StatisticArray.Num(), *AsyncBlock);
}

void FOnlineAsyncTaskGDKStatsUpdate::Initialize()
{
	FOnlineStatsGDKPtr GDKStats = Subsystem->GetStatsGDK();
	check(GDKStats.IsValid());

	// Apply the stat updates
	bool bSuccess = true;

	for (const FOnlineStatsUserUpdatedStats& UpdatedUserStat : UpdatedUserStats)
	{
		FUniqueNetIdGDKRef GDKId = FUniqueNetIdGDK::Cast(*UpdatedUserStat.Account);

		if (GDKId->IsValid())
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XUserFindUserById is not safe to be called on time-sensitive threads

			uint64 User = GDKUserMapping.FindChecked(GDKId); //TODO: why not just use GDKId->ToUint64()?
			FGDKUserHandle UserHandle;
			XUserFindUserById(User, UserHandle.GetInitReference());

			for (const TPair<FString, FOnlineStatUpdate>& UpdatePair : UpdatedUserStat.Stats)
			{
				const bool bStatUpdateSuccess = GDKStats->ApplyStatUpdate(GDKId, UpdatePair.Key, UpdatePair.Value);

				if (!bStatUpdateSuccess)
				{
					UE_LOG_ONLINE_STATS(Warning, TEXT("Stat update not possible for stat %s"), *UpdatePair.Key);
				}

				bSuccess &= bStatUpdateSuccess;
			}

			if (bSuccess)
			{
				HRESULT Result = UpdateAllUserStats(GDKId);
				if (Result != S_OK)
				{
					UE_LOG_ONLINE_STATS(Warning, TEXT("FOnlineAsyncTaskGDKStatsUpdate - XblTitleManagedStatsWriteAsync - Error: (0x%0.8X)."), Result);

					bSuccess = false;
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("%s is not a local valid user"), *UpdatedUserStat.Account->ToDebugString());
			bSuccess &= false;
		}
	}

	bWasSuccessful = bSuccess;
	if (!bSuccess)
	{
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKStatsUpdate::ProcessResults()
{
	bIsComplete = true;
}

void FOnlineAsyncTaskGDKStatsUpdate::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKStatsUpdate_TriggerDelegates);
	Delegate.ExecuteIfBound(bWasSuccessful ? FOnlineError::Success() : ONLINE_ERROR(EOnlineErrorResult::InvalidParams));
}

#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK