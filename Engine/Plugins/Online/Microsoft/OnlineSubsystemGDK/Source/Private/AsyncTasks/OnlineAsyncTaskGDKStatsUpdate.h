// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "Interfaces/OnlineStatsInterface.h"

class FOnlineSubsystemGDK;

/**
 * Async Task to upload updated stats to service
 */
class FOnlineAsyncTaskGDKStatsUpdate : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKStatsUpdate(FOnlineSubsystemGDK* const InGDKSubsystem, FGDKContextHandle InUserContext, const TArray<FOnlineStatsUserUpdatedStats>& InUpdatedUserStats, const FOnlineStatsUpdateStatsComplete& InDelegate, const TMap<FUniqueNetIdGDKRef, uint64>& InGDKUserMapping);

	virtual FString ToString() const override { return TEXT("UpdateStats"); }

	virtual void Initialize() override;
	virtual void ProcessResults() override;
	virtual void TriggerDelegates() override;

private:
	HRESULT UpdateAllUserStats(FUniqueNetIdGDKRef GDKId);

private:
	FGDKContextHandle GDKContext;
	const TArray<FOnlineStatsUserUpdatedStats> UpdatedUserStats;
	TMap<FUniqueNetIdGDKRef, uint64> GDKUserMapping;
	FOnlineStatsUpdateStatsComplete Delegate;
};
