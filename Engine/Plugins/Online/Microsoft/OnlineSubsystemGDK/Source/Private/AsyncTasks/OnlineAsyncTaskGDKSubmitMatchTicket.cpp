// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKSubmitMatchTicket.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineAsyncTaskGDKCancelMatchmaking.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "OnlineAsyncTaskGDKSetSessionActivity.h"

THIRD_PARTY_INCLUDES_START
#include <Ws2tcpip.h>
THIRD_PARTY_INCLUDES_END

FOnlineAsyncTaskGDKSubmitMatchTicket::FOnlineAsyncTaskGDKSubmitMatchTicket(
	FOnlineSubsystemGDK* InSubsystem,
	uint64 InSearchingUser,
	FName InSessionName,
	FGDKMultiplayerSessionHandle InGDKSession,
	const FString& InHopperName,
	const FString& InTicketAttributes,
	uint64 InTicketTimeout,
	XblPreserveSessionMode InTicketPreservation,
	bool InCancelExistingTicket
)
	: FOnlineAsyncTaskGDK(InSubsystem, TEXT("FOnlineAsyncTaskGDKSubmitMatchTicket"), 0)
	, SearchingUser(InSearchingUser)
	, SessionName(InSessionName)
	, MatchGDKSession(InGDKSession)
	, HopperName(InHopperName)
	, TicketAttributes(InTicketAttributes)
	, TicketTimeout(InTicketTimeout)
	, TicketPreservation(InTicketPreservation)
	, CancelExistingTicket(InCancelExistingTicket)
{
	// Find the existing ticket id on the game thread if we need to cancel it
	if (CancelExistingTicket)
	{
		FOnlineMatchTicketInfoPtr CurrentTicket;
		Subsystem->GetMatchmakingInterfaceGDK()->GetMatchmakingTicket(SessionName, CurrentTicket);

		if (CurrentTicket.IsValid())
		{
			TicketIdToCancel = CurrentTicket->TicketId;
		}
	}
}

void FOnlineAsyncTaskGDKSubmitMatchTicket::Initialize()
{
	if (CancelExistingTicket)
	{
		FGDKContextHandle Context = Subsystem->GetGDKContext(SearchingUser);

		const ANSICHAR* Scid = nullptr;
		XblGetScid(&Scid);

		RemoveAsyncBlock(AsyncBlock);
		AsyncBlock = CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock){
			UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskLiveSubmitMatchTicket: successfully deleted ticket %s."), *TicketIdToCancel);
			CreateMatchmakingTicket();
		});

		HRESULT Result = XblMatchmakingDeleteMatchTicketAsync(Context, Scid, TCHAR_TO_UTF8(*HopperName), TCHAR_TO_UTF8(*TicketIdToCancel), *AsyncBlock);
		if (Result != S_OK)
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskGDKSubmitMatchTicket: failed to cancel existing ticket."));
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
	else
	{
		CreateMatchmakingTicket();
	}
}

void FOnlineAsyncTaskGDKSubmitMatchTicket::CreateMatchmakingTicket()
{
	FGDKContextHandle GDKContext = Subsystem->GetGDKContext(SearchingUser);

	RemoveAsyncBlock(AsyncBlock);
	AsyncBlock = CreateAsyncBlock();

	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);
	UE_LOG_ONLINE(Log, TEXT("CreateMatchTicketAsync attempt to create ticket"));
	const XblMultiplayerSessionReference* MatchSessionRef = XblMultiplayerSessionSessionReference(MatchGDKSession);
	HRESULT Result = XblMatchmakingCreateMatchTicketAsync(GDKContext, *MatchSessionRef, Scid, TCHAR_TO_UTF8(*HopperName), TicketTimeout, TicketPreservation, TCHAR_TO_UTF8(*TicketAttributes), *AsyncBlock);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("CreateMatchTicketAsync failed with 0x%0.8X"), Result);
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKSubmitMatchTicket::ProcessResults()
{
	XblCreateMatchTicketResponse CreateTicketResponse;
	HRESULT Result = XblMatchmakingCreateMatchTicketResult(*AsyncBlock, &CreateTicketResponse);
	if (Result == S_OK)
	{
		bWasSuccessful = true;
		ResultTicketId = UTF8_TO_TCHAR(CreateTicketResponse.matchTicketId);
		ResultWaitTime = ((float)CreateTicketResponse.estimatedWaitTime) / 10000000.0;
		UE_LOG_ONLINE(Log, TEXT("Matchmaking ticket created... (%s)"), *ResultTicketId);
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("CreateMatchTicketAsync failed with 0x%0.8X"), Result);
		bWasSuccessful = false;
	}

	bIsComplete = true;
}

void FOnlineAsyncTaskGDKSubmitMatchTicket::Finalize()
{
	// Get the GDK session info
	FOnlineMatchTicketInfoPtr Ticket;
	// If the named session is null, it may have already been destroyed

	Subsystem->GetMatchmakingInterfaceGDK()->GetMatchmakingTicket(SessionName, Ticket);

	//. Store the TicketID
	FGDKContextHandle UserContext = Subsystem->GetGDKContext(SearchingUser);

	// Abort if matchmaking was canceled
	if (Ticket.IsValid() == false || Ticket->MatchmakingState == EOnlineGDKMatchmakingState::UserCancelled)
	{
		bWasSuccessful = false;

		// Queue the task here since queued tasks can start ticking before the previous task has run Finalize().
		// @todo: fix so that Finalize() is guaranteed to finish first?
		FOnlineAsyncTaskGDKCancelMatchmaking* CancelMatchTask = new FOnlineAsyncTaskGDKCancelMatchmaking(
				Subsystem,
				UserContext,
				SessionName,
				Ticket);
		Subsystem->QueueAsyncTask(CancelMatchTask);
		return;
	}

	if (!bWasSuccessful)
	{
		Subsystem->GetMatchmakingInterfaceGDK()->RemoveMatchmakingTicket(SessionName);
		return;
	}

	// Update our current activity
	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	if (SessionInt.IsValid() && UserContext)
	{
		uint64 UserId;
		if (XblContextGetXboxUserId(UserContext, &UserId) == S_OK)
		{
			FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(UserId);
			SessionInt->GetMpsdImpl()->SetUserActiveSessionActivity(*UserNetId, MatchGDKSession);
		}
	}

	Ticket->HopperName = HopperName;
	Ticket->TicketId = ResultTicketId;
	Ticket->EstimatedWaitTimeInSeconds = ResultWaitTime;
	Ticket->MatchmakingState = EOnlineGDKMatchmakingState::WaitingForGameSession;
}

void FOnlineAsyncTaskGDKSubmitMatchTicket::TriggerDelegates()
{
	if (!bWasSuccessful)
	{
		// Only fire delegates on failure. It's unfortunate to break up the logic like this,
		// but on success the delegates will be fired when the session interface completes
		// the initialization flow in OnSessionChanged.
		auto MatchmakingInterface = Subsystem->GetMatchmakingInterfaceGDK();
		check(MatchmakingInterface.IsValid());

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKSubmitMatchTicket_TriggerDelegates);
		MatchmakingInterface->TriggerOnMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
	}
}

#endif //WITH_GRDK