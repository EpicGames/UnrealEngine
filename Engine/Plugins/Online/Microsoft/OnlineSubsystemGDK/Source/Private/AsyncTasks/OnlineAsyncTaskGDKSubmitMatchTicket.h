// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/matchmaking_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task that creates a matchmaking ticket and submits it.
 */
class FOnlineAsyncTaskGDKSubmitMatchTicket : public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKSubmitMatchTicket(
		FOnlineSubsystemGDK* InSubsystem,
		uint64 InSearchingUser,
		FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession,
		const FString& InHopperName,
		const FString& InTicketAttributes,
		uint64 InTicketTimeout,
		XblPreserveSessionMode InTicketPreservation,
		bool InCancelExistingTicket);

	virtual void Initialize() override;

	virtual FString ToString() const override { return TEXT("SubmitMatchTicket"); }
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual void ProcessResults() override;
private:
	void CreateMatchmakingTicket();

	uint64 SearchingUser;
	FName SessionName;
	FGDKMultiplayerSessionHandle MatchGDKSession;
	FString HopperName;
	FString TicketAttributes;
	uint64 TicketTimeout;
	XblPreserveSessionMode TicketPreservation;
	bool CancelExistingTicket;

	FString TicketIdToCancel;
	XblCreateMatchTicketResponse* Response = nullptr;
	XAsyncBlock DeleteAsyncBlock;
	FString ResultTicketId;
	float ResultWaitTime;
};

//------------------------------- End of file ---------------------------------
