// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKUnregisterLocalUser.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"

FOnlineAsyncTaskGDKUnregisterLocalUser::FOnlineAsyncTaskGDKUnregisterLocalUser(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	const FName InSessionName,
	FGDKMultiplayerSessionHandle InGDKSession,
	const FUniqueNetIdGDKRef& InUserId,
	const FOnUnregisterLocalPlayerCompleteDelegate& InDelegate
)
	: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, TEXT("FOnlineAsyncTaskGDKUnregisterLocalUser"), InContext, InSessionName, InGDKSession)
	, UserId(InUserId)
	, Delegate(InDelegate)
{
}

bool FOnlineAsyncTaskGDKUnregisterLocalUser::UpdateSession(FGDKMultiplayerSessionHandle Session)
{
	XblMultiplayerSessionLeave(Session);
	return true;
}

void FOnlineAsyncTaskGDKUnregisterLocalUser::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKUnregisterLocalUser_TriggerDelegates);
	Delegate.ExecuteIfBound(*UserId, bWasSuccessful);
}

#endif //WITH_GRDK