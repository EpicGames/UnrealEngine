// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSafeWriteSession.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystemGDKTypes.h"

/** 
 * Async task used to have another local user Leave() a Live session.
 */
class FOnlineAsyncTaskGDKUnregisterLocalUser : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	FOnlineAsyncTaskGDKUnregisterLocalUser(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession,
		const FUniqueNetIdGDKRef& InUserId,
		const FOnUnregisterLocalPlayerCompleteDelegate& InDelegate);

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKUnregisterLocalUser %s"), *GetAsyncSafeWriteTaskInfoString()); }
	virtual void TriggerDelegates() override;

private:

	// FOnlineAsyncTaskGDKSafeWriteSession
	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session);

	FUniqueNetIdGDKRef UserId;
	FOnUnregisterLocalPlayerCompleteDelegate Delegate;
};
