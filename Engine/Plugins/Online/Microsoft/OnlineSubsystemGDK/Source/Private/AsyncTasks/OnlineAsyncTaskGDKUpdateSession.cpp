// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKUpdateSession.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskGDKSetSessionActivity.h"
#include "OnlineAsyncTaskGDKClearSessionActivity.h"

FOnlineAsyncTaskGDKUpdateSession::FOnlineAsyncTaskGDKUpdateSession(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	const FName InSessionName,
	const FOnlineSessionSettings& InUpdatedSessionSettings,
	const int32 InMaxRetryCount
)
	: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, TEXT("FOnlineAsyncTaskGDKUpdateSession"), InContext, InSessionName, InMaxRetryCount)
	, UpdatedSessionSettings(InUpdatedSessionSettings)
{
}

void FOnlineAsyncTaskGDKUpdateSession::Initialize()
{
	check(IsInGameThread());

	FOnlineSessionGDKPtr SessionInt = Subsystem->GetSessionInterfaceGDK();
	check(SessionInt.IsValid());

	// Pull our latest GDK session
	FNamedOnlineSessionPtr NamedSession = SessionInt->GetNamedSessionPtr(GetSessionName());
	if (NamedSession.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoMpsdGDK> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		if (SessionInfo.IsValid())
		{
			if (FGDKMultiplayerSessionHandle LocalGDKSession = SessionInfo->GetGDKMultiplayerSession())
			{
				GDKSession = LocalGDKSession;
				GDKSessionReference = XblMultiplayerSessionSessionReference(LocalGDKSession);
			}
			else if (const XblMultiplayerSessionReference* LocalGDKSessionReference = SessionInfo->GetGDKMultiplayerSessionRef())
			{
				GDKSessionReference = LocalGDKSessionReference;
			}
		}
	}

	if (GDKSessionReference)
	{
		FOnlineAsyncTaskGDKSafeWriteSession::Initialize();
	}
	else
	{
		OnFailed();
	}
}

bool FOnlineAsyncTaskGDKUpdateSession::UpdateSession(FGDKMultiplayerSessionHandle Session)
{
	const bool SessionSettingsChanged = FOnlineSessionMpsdGDK::WriteSettingsToGDKJson(UpdatedSessionSettings, Session, FGDKUserHandle(), Subsystem);

	// TODO: there might be other good things to update here?

	UE_CLOG_ONLINE_SESSION(!SessionSettingsChanged, Log, TEXT("Found no settings to update for session %s"), *GetSessionName().ToString());
	return SessionSettingsChanged;
}

void FOnlineAsyncTaskGDKUpdateSession::Finalize()
{
	FOnlineAsyncTaskGDKSafeWriteSession::Finalize();

	// Update our session activity status, if applicable
	FOnlineSessionGDKPtr SessionInterface = Subsystem->GetSessionInterfaceGDK();
	if (SessionInterface.IsValid())
	{
		if(GDKContext)
		{
			uint64 UserId;

			if (XblContextGetXboxUserId(GDKContext, &UserId) == S_OK)
			{
				FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(UserId);
				const bool bShouldBeInActiveSession = FOnlineSessionMpsdGDK::AreInvitesAndJoinViaPresenceAllowed(UpdatedSessionSettings);

				const FGDKMultiplayerSessionHandle GDKSessionHandle = bShouldBeInActiveSession ? GDKSession : FGDKMultiplayerSessionHandle();
				SessionInterface->GetMpsdImpl()->SetUserActiveSessionActivity(*UserNetId, GDKSessionHandle);
			}
		}
	}
}

void FOnlineAsyncTaskGDKUpdateSession::TriggerDelegates()
{
	IOnlineSessionPtr SessionInterface = Subsystem->GetSessionInterface();
	if (SessionInterface.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKUpdateSession_TriggerDelegates);
		SessionInterface->TriggerOnUpdateSessionCompleteDelegates(GetSessionName(), WasSuccessful());
	}
}

#endif //WITH_GRDK