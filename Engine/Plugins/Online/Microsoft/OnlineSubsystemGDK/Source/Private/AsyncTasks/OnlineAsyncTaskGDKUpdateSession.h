// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSafeWriteSession.h"
#include "OnlineSessionSettings.h"

class FOnlineSubsystemGDK;

class FOnlineAsyncTaskGDKUpdateSession : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	/**
	 * This constructor takes the Live SessionReference directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the GDK context of the user who's updating the session
	 * @param InSessionName the name of the session to update
	 * @param InUpdatedSessionSettings the session settings to write to the session object
	 * @param InMaxRetryCount Maximum number of times to retry the session write
	 */
	FOnlineAsyncTaskGDKUpdateSession(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		const FOnlineSessionSettings& InUpdatedSessionSettings,
		const int32 InMaxRetryCount);

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKUpdateSession %s"), *GetAsyncSafeWriteTaskInfoString()); }

protected:
	virtual void Initialize() override;

	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session) override;

	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

	FOnlineSessionSettings UpdatedSessionSettings;
};
