// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKUpdateSessionMember.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSessionGDK.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "OnlineSubsystemSessionSettings.h"


FOnlineAsyncTaskGDKUpdateSessionMember::FOnlineAsyncTaskGDKUpdateSessionMember(
	FOnlineSubsystemGDK* const InSubsystem,
	FGDKContextHandle InContext,
	const FName InSessionName,
	FGDKMultiplayerSessionHandle InGDKSession,
	const int32 InMaxRetryCount
)
	: FOnlineAsyncTaskGDKSafeWriteSession(InSubsystem, TEXT("FOnlineAsyncTaskGDKUpdateSessionMember"), InContext, InSessionName, InGDKSession, InMaxRetryCount)
{
}

bool FOnlineAsyncTaskGDKUpdateSessionMember::UpdateSession(FGDKMultiplayerSessionHandle Session)
{
	auto SessionInterface = Subsystem->GetSessionInterface();

	if (SessionInterface.IsValid() && XblMultiplayerSessionCurrentUser(Session))
	{
		if (auto NamedSession = SessionInterface->GetNamedSession(GetSessionName()))
		{
			const XblMultiplayerSessionMember* CurrentUser = XblMultiplayerSessionCurrentUser(Session);
				
			FString Key = FString::Printf(TEXT("%s%lld"), SETTING_GROUP_NAME_PREFIX, CurrentUser->Xuid);
			FString Setting;

			if (NamedSession->SessionSettings.Get(FName(*Key), Setting))
			{
				TArray<const char*> GroupNameCharPtrArray;
				GroupNameCharPtrArray.Add(TCHAR_TO_UTF8(*Setting));
				XblMultiplayerSessionCurrentUserSetGroups(Session, GroupNameCharPtrArray.GetData(), GroupNameCharPtrArray.Num());
			}

			Key = FString::Printf(TEXT("%s%lld"), SETTING_SESSION_MEMBER_CONSTANT_CUSTOM_JSON_XUID_PREFIX, CurrentUser->Xuid);
			if (NamedSession->SessionSettings.Get(FName(*Key), Setting))
			{
				TSharedPtr< FJsonObject > JObj;
				TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( Setting );

				if ( FJsonSerializer::Deserialize(Reader, JObj) && JObj.IsValid() )
				{
					for ( auto it = JObj->Values.CreateConstIterator(); it; ++it )
					{
						const FJsonObject::FStringType&	JSettingName = it.Key();
						const TSharedPtr<FJsonValue>&	JSettingValue = it.Value();

						XblMultiplayerSessionCurrentUserSetCustomPropertyJson(Session, TCHAR_TO_UTF8(*JSettingName), TCHAR_TO_UTF8(*JSettingValue->AsString()));
					}
				}
			}
		}

		return true;
	}

	return false;
}


void FOnlineAsyncTaskGDKUpdateSessionMember::TriggerDelegates()
{
	auto SessionInterface = Subsystem->GetSessionInterface();
	if (SessionInterface.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKUpdateSessionMember_TriggerDelegates);
		SessionInterface->TriggerOnUpdateSessionCompleteDelegates(GetSessionName(), WasSuccessful());
	}
}

#endif //WITH_GRDK