// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskGDKSafeWriteSession.h"

class FOnlineAsyncTaskGDKUpdateSessionMember : public FOnlineAsyncTaskGDKSafeWriteSession
{
public:
	/**
	 * This constructor takes the GDK Session directly and is safe to call
	 * in non-game threads.
	 *
	 * @param InSubsystem the owning GDK online subsystem
	 * @param InContext the GDK context of the user who's updating the session
	 * @param InSessionName the name of the session to update
	 * @param InGDKSession the GDK session we're writing to
	 * @param InMaxRetryCount number of times to retry the session write
	 */
	FOnlineAsyncTaskGDKUpdateSessionMember(
		FOnlineSubsystemGDK* const InSubsystem,
		FGDKContextHandle InContext,
		const FName InSessionName,
		FGDKMultiplayerSessionHandle InGDKSession,
		const int32 InMaxRetryCount);

	// FOnlineAsyncItem
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskGDKUpdateSessionMember %s"), *GetAsyncSafeWriteTaskInfoString()); }

protected:

	virtual bool UpdateSession(FGDKMultiplayerSessionHandle Session) override;

	virtual void TriggerDelegates();

	FName SessionIdentifier;
};
