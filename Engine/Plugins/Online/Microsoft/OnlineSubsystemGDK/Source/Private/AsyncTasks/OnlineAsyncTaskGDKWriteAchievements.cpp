// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskGDKWriteAchievements.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineAchievementsInterfaceGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/achievements_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineAsyncTaskGDKWriteAchievements::FOnlineAsyncTaskGDKWriteAchievements(
	FOnlineSubsystemGDK* InGDKInterface, 
	FGDKContextHandle InGDKContext, 
	const FUniqueNetIdGDKRef& InUserIdGDK, 
	FOnlineAchievementsWriteRef& WriteObject,
	const FOnAchievementsWrittenDelegate& InTaskCompletionDelegate
	)
	: FOnlineAsyncTaskGDK(InGDKInterface, TEXT("FOnlineAsyncTaskGDKWriteAchievements"))
	, UserIdGDK(InUserIdGDK)
	, GDKContext(InGDKContext)
	, TaskCompletionDelegate(InTaskCompletionDelegate)
	, WriteObjectRef(WriteObject)
	, AchievementDataArray(WriteObject->Properties)
	, bErrorProcessingAPICalls(false)
{
}

void FOnlineAsyncTaskGDKWriteAchievements::Initialize()
{
	if (Subsystem->GetAchievementsInterfaceGDK().IsValid() == false)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: Failed to get Achievements interface"));
		WriteObjectRef->WriteState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	if (AchievementDataArray.Num() < 1)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: No achievements to write"));
		WriteObjectRef->WriteState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;
		return;
	}

	NumOutstandingAPICalls = 0;

	bool bErrorProcessingEvents = false;

	for (FStatPropertyArray::TConstIterator It(AchievementDataArray); It; ++It)
	{
		float Percent = 0.0f;

		It.Value().GetValue(Percent);

		// Clamp so we don't have an invalid value
		Percent = FMath::Clamp(Percent, 0.0f, 100.0f);

		// The XBL back end wants the achievement ID, which is the number assigned to the achievement
		// This is the the order in which the achievements are created on XDP/UDC, starting from 1
		FString AchievementName = It.Key();

		int32 AchievementId = Subsystem->GetAchievementsInterfaceGDK()->GetAchievementIdFromName(AchievementName);

		if (Subsystem->GetAchievementsInterfaceGDK()->AchievementMode == EGDKAchievementMode::Mode2013)
		{
			if (Percent < 100.0f)
			{
				continue;
			}

			if (AchievementId == INVALID_ACHIEVEMENT_ID)
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: No mapping for achievement %s"), *AchievementName);
				bErrorProcessingEvents = true;
				continue;
			}

			FString AchievementEventName = Subsystem->GetAchievementsInterfaceGDK()->GetAchievementEventName();

			FOnlineEventParms Parms;
			Parms.Add(TEXT("AchievementIndex"), FVariantData(AchievementId));

			if (!Subsystem->GetEventsInterface()->TriggerEvent(*UserIdGDK, *AchievementEventName, Parms))
			{
				bErrorProcessingEvents = true;
			}
		}
		else
		{
			// Direct unlocking of achievements means we don't care about percentages

			if (AchievementId == INVALID_ACHIEVEMENT_ID)
			{
				UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: No mapping for achievement %s"), *AchievementName);
				bErrorProcessingAPICalls = true;
				continue;
			}

			NumOutstandingAPICalls++;
			WriteObjectRef->WriteState = EOnlineAsyncTaskState::InProgress;

			FGDKAsyncBlockPtr& AchievementAsyncBlock = AsyncBlocksByAchievementName.Emplace(AchievementName, CreateAsyncBlock(nullptr, [this, AchievementName](FGDKAsyncBlock* LambdaAsyncBlock) {
				HandleWriteAchievementComplete(AchievementName);
			}));

			FString AchievementIdStr = FString::FromInt(AchievementId);
			HRESULT Result = XblAchievementsUpdateAchievementAsync(GDKContext, UserIdGDK->ToUint64(), TCHAR_TO_UTF8(*AchievementIdStr), Percent, *AchievementAsyncBlock);

			if (FAILED(Result) && Result != HTTP_E_STATUS_NOT_MODIFIED)
			{
				UE_LOG_ONLINE(Error, TEXT("Error updating achievement %s, error: (0x%0.8X)."), *AchievementName, Result);

				bErrorProcessingAPICalls = true;
			}
		}
	}

	if (Subsystem->GetAchievementsInterfaceGDK()->AchievementMode == EGDKAchievementMode::Mode2013)
	{
		if (bErrorProcessingEvents)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: Not all achievement events triggerd succesfully"));
			WriteObjectRef->WriteState = EOnlineAsyncTaskState::Failed;
			bWasSuccessful = false;			
		}
		else
		{
			WriteObjectRef->WriteState = EOnlineAsyncTaskState::Done;
			bWasSuccessful = true;
		}

		bIsComplete = true;
	}
	else if (NumOutstandingAPICalls < 1 && bErrorProcessingAPICalls) // As long as one API call is made, we'll have to wait until it ends
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAsyncTaskGDKWriteAchievements::Initialize: All achievements failed enumeration."));
		WriteObjectRef->WriteState = EOnlineAsyncTaskState::Failed;
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKWriteAchievements::HandleWriteAchievementComplete(FString AchievementName)
{
	if (FGDKAsyncBlockPtr* AchievementAsyncBlock = AsyncBlocksByAchievementName.Find(AchievementName))
	{
		HRESULT Result = XAsyncGetStatus(**AchievementAsyncBlock, false);
		if (FAILED(Result) && Result != HTTP_E_STATUS_NOT_MODIFIED)
		{
			UE_LOG_ONLINE(Error, TEXT("Error updating achievement %s, error: (0x%0.8X)."), *AchievementName, Result);

			bErrorProcessingAPICalls = true;
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("AsyncBlock not found for achievement %s."), *AchievementName);

		bErrorProcessingAPICalls = true;
	}

	// We decrement the counter at the end instead of the beginning, in case Tick gets called in the middle of the code above
	--NumOutstandingAPICalls;
}

void FOnlineAsyncTaskGDKWriteAchievements::Tick()
{
	//We have completed all of our writes
	if (NumOutstandingAPICalls < 1)
	{
		// If there was an error at any point while processing API calls, we report as a failure.
		// Other calls might have processed successfully, but we'll only report as a success if they all do.
		bWasSuccessful = !bErrorProcessingAPICalls;

		if (bWasSuccessful)
		{
			WriteObjectRef->WriteState = EOnlineAsyncTaskState::Done;
		}
		else
		{
			WriteObjectRef->WriteState = EOnlineAsyncTaskState::Failed;
		}

		bIsComplete = true;
	}
}

void FOnlineAsyncTaskGDKWriteAchievements::TriggerDelegates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAsyncTaskGDKWriteAchievements_TriggerDelegates);
	TaskCompletionDelegate.ExecuteIfBound(*UserIdGDK, bWasSuccessful);
}

#endif //WITH_GRDK