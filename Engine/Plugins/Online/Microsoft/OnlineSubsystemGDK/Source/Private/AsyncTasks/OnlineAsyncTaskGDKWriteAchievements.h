// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineAsyncTaskManagerGDK.h"
#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 * Async task to write achievements
 */
class FOnlineAsyncTaskGDKWriteAchievements
	: public FOnlineAsyncTaskGDK
{
public:
	FOnlineAsyncTaskGDKWriteAchievements(
		FOnlineSubsystemGDK* InGDKInterface,
		FGDKContextHandle InGDKContext,
		const FUniqueNetIdGDKRef& InUserIdGDK,
		FOnlineAchievementsWriteRef& WriteObject,
		const FOnAchievementsWrittenDelegate& InDelegate
		);
	
	virtual ~FOnlineAsyncTaskGDKWriteAchievements() = default;

	//~ Begin FOnlineAsyncItem Interface
	virtual FString ToString() const override { return TEXT("FOnlineAsyncTaskGDKWriteAchievements"); }

	// Starts in Game Thread
	virtual void Initialize() override;
	
	// Process in Online Thread
	void HandleWriteAchievementComplete(FString AchievementName);
	
	// Tick to check that all subtasks have completed
	virtual void Tick() override;
	
	// Trigger delegates in Game Thread
	virtual void TriggerDelegates() override;

protected:
	FUniqueNetIdGDKRef UserIdGDK;
	FGDKContextHandle GDKContext;
	const FOnAchievementsWrittenDelegate TaskCompletionDelegate;
	FOnlineAchievementsWriteRef WriteObjectRef;
	FStatPropertyArray AchievementDataArray;

	bool bErrorProcessingAPICalls;
	uint32 NumOutstandingAPICalls;
	TMap<FString, FGDKAsyncBlockPtr> AsyncBlocksByAchievementName;
};
