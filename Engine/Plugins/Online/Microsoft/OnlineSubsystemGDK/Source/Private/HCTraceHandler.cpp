// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "HCTraceHandler.h"

#if !NO_LOGGING

#include "Logging/LogMacros.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(LogOnlineHCTrace, Display, All);

FHCTraceHandler::FHCTraceHandler()
{
	OnLogVerbosityChangedHandle = FCoreDelegates::OnLogVerbosityChanged.AddRaw(this, &FHCTraceHandler::OnLogVerbosityChanged);
	OnLogVerbosityChanged(LogOnlineHCTrace.GetCategoryName(), ELogVerbosity::Log, LogOnlineHCTrace.GetVerbosity());

	HCTraceSetClientCallback(&FHCTraceHandler::TraceCallback);
}

FHCTraceHandler::~FHCTraceHandler()
{
	FCoreDelegates::OnLogVerbosityChanged.Remove(OnLogVerbosityChangedHandle);
	OnLogVerbosityChangedHandle.Reset();
}

HCTraceLevel FHCTraceHandler::GetTraceLevel(ELogVerbosity::Type LogVerbosity)
{
	switch (LogVerbosity)
	{
	case ELogVerbosity::NoLogging:		// Intentional fall through
	case ELogVerbosity::Fatal:			return HCTraceLevel::Off;
	case ELogVerbosity::Error:			return HCTraceLevel::Error;
	case ELogVerbosity::Warning:		return HCTraceLevel::Warning;
	default:							// Intentional fall through
	case ELogVerbosity::Display:		// Intentional fall through
	case ELogVerbosity::Log:			return HCTraceLevel::Important;
	case ELogVerbosity::Verbose:		return HCTraceLevel::Information;
	case ELogVerbosity::VeryVerbose:	return HCTraceLevel::Verbose;
	}
}

void FHCTraceHandler::OnLogVerbosityChanged(const FLogCategoryName& CategoryName, ELogVerbosity::Type /*OldVerbosity*/, ELogVerbosity::Type NewVerbosity)
{
	if (CategoryName == LogOnlineHCTrace.GetCategoryName())
	{
		const HRESULT Result = HCSettingsSetTraceLevel(GetTraceLevel(NewVerbosity));
		if (Result != S_OK)
		{
			UE_LOGF(LogOnlineHCTrace, Error, "Failed to set trace level, error: (0x%0.8X).", Result);
		}
	}
}

void FHCTraceHandler::TraceCallback(const char* AreaName, HCTraceLevel TraceLevel, uint64_t ThreadId, uint64_t Timestamp, const char* Message)
{
#define HCTraceLOG(Level) UE_LOGF(LogOnlineHCTrace, Level, "ThreadId=[%llu] Area=[%ls] Message=[%ls]", ThreadId, UTF8_TO_TCHAR(AreaName), UTF8_TO_TCHAR(Message))

	switch (TraceLevel)
	{
	case HCTraceLevel::Error:		HCTraceLOG(Error); break;
	case HCTraceLevel::Warning:		HCTraceLOG(Warning); break;
	case HCTraceLevel::Important:	HCTraceLOG(Log); break;
	case HCTraceLevel::Information:	HCTraceLOG(Verbose); break;
	case HCTraceLevel::Verbose:		HCTraceLOG(VeryVerbose); break;
	case HCTraceLevel::Off:
	default:
		// do nothing
		break;
	}
#undef HCTraceLOG
}

#endif // !NO_LOGGING
#endif //WITH_GRDK