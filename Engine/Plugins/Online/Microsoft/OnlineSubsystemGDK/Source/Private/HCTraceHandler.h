// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Logging/LogCategory.h"
#include "Misc/Build.h"

#if NO_LOGGING

struct FHCTraceHandler {};

#else

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <httpClient/trace.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

struct FHCTraceHandler
{
	FHCTraceHandler();
	~FHCTraceHandler();

private:

	void OnLogVerbosityChanged(const FLogCategoryName& CategoryName, ELogVerbosity::Type OldVerbosity, ELogVerbosity::Type NewVerbosity);
	static void TraceCallback(const char* AreaName, HCTraceLevel TraceLevel, uint64_t ThreadId, uint64_t Timestamp, const char* Message);
	HCTraceLevel GetTraceLevel(ELogVerbosity::Type Verbosity);

	FDelegateHandle OnLogVerbosityChangedHandle;
};

#endif // !NO_LOGGING