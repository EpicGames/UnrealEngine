// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "MessageSanitizerGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSanitizeString.h"
#include "OnlineError.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(LogOnlineSanitizeGDK, Display, All);

FMessageSanitizerGDK::FMessageSanitizerGDK(FOnlineSubsystemGDK* InGDKSubsystem) :
	GDKSubsystem(InGDKSubsystem)
{
	check(GDKSubsystem);
	AppResumeDelegateHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FMessageSanitizerGDK::HandleAppResume);
	AppReactivatedDelegateHandle = FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FMessageSanitizerGDK::HandleAppReactivated);
}

FMessageSanitizerGDK::~FMessageSanitizerGDK()
{
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(AppResumeDelegateHandle);
}

void FMessageSanitizerGDK::HandleAppResume()
{
	UE_LOGF(LogOnlineSanitizeGDK, Verbose, "FMessageSanitizerGDK::HandleAppResume");
	ResetBlockedUserCache();
}

void FMessageSanitizerGDK::HandleAppReactivated()
{
	UE_LOGF(LogOnlineSanitizeGDK, Verbose, "FMessageSanitizerGDK::HandleAppReactivated");
	ResetBlockedUserCache();
}

void FMessageSanitizerGDK::SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate)
{
	bool bUseDisplayNameSanitizer = true;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bUseDisplayNameSanitizer"), bUseDisplayNameSanitizer, GEngineIni);
	if (bUseDisplayNameSanitizer)
	{
		if (const FString* FoundStringPtr = WordMap.Find(DisplayName))
		{

			GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, FoundString = FString(*FoundStringPtr)]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayName_Delegate);
					CompletionDelegate.ExecuteIfBound(true, FoundString);
				});
		}
		else
		{
			const IOnlineIdentityPtr IdentityInterface = GDKSubsystem->GetIdentityInterface();
			if (!IdentityInterface.IsValid())
			{
				UE_LOGF(LogOnlineSanitizeGDK, Warning, "SanitizeDisplayName: couldn't get the identity interface");
				GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayName]()
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayName_Delegate);
						CompletionDelegate.ExecuteIfBound(false, DisplayName);
					});
				return;
			}

			const FUniqueNetIdPtr UserId = GetFirstSignedInUser(IdentityInterface);
			if (!UserId.IsValid())
			{
				UE_LOGF(LogOnlineSanitizeGDK, Warning, "SanitizeDisplayName: invalid UserId");
				GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayName]()
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayName_Delegate);
						CompletionDelegate.ExecuteIfBound(false, DisplayName);
					});
				return;
			}
			FGDKContextHandle Context = GDKSubsystem->GetGDKContext(*UserId);
			GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKSanitizeSingleString>(
				GDKSubsystem
				,Context
				,DisplayName
				, FOnlineAsyncTaskGDKSanitizeSingleStringComplete::CreateThreadSafeSP(this, &FMessageSanitizerGDK::HandleMessageSanitized, CompletionDelegate, DisplayName)
				);
		}
	}
	else 
	{
		GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayName]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayName_Delegate);
			CompletionDelegate.ExecuteIfBound(true, DisplayName);
		});
	}
}

void FMessageSanitizerGDK::SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate)
{
	bool bUseDisplayNameSanitizer = true;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bUseDisplayNameSanitizer"), bUseDisplayNameSanitizer, GEngineIni);
	if (bUseDisplayNameSanitizer)
	{
		TArray<FString> UnsanitizedArray;
		TArray<FString> AlreadyProcessedMessages;
		TArray<int32> AlreadyProcessedIndex;
		for (int32 iMessageIndex = 0; iMessageIndex < DisplayNames.Num(); iMessageIndex++)
		{
			const FString& DisplayName = DisplayNames[iMessageIndex];

			const FString* FoundString = WordMap.Find(DisplayName);
			if (!FoundString)
			{
				UnsanitizedArray.Add(*DisplayName);
			}
			else
			{
				AlreadyProcessedMessages.Add(*FoundString);
				AlreadyProcessedIndex.Add(iMessageIndex);
			}
		}

		if (UnsanitizedArray.Num() > 0)
		{
			const IOnlineIdentityPtr IdentityInterface = GDKSubsystem->GetIdentityInterface();
			if (!IdentityInterface.IsValid())
			{
				UE_LOGF(LogOnlineSanitizeGDK, Warning, "SanitizeDisplayNames: couldn't get the identity interface");
				GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayNames]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayNames_Delegate);
					CompletionDelegate.ExecuteIfBound(false, DisplayNames);
				});
				return;
			}

			const FUniqueNetIdPtr UserId = GetFirstSignedInUser(IdentityInterface);
			if (!UserId.IsValid())
			{
				UE_LOGF(LogOnlineSanitizeGDK, Warning, "SanitizeDisplayNames: invalid UserId");
				GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayNames]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayNames_Delegate);
					CompletionDelegate.ExecuteIfBound(false, DisplayNames);
				});
				return;
			}
			FGDKContextHandle Context = GDKSubsystem->GetGDKContext(*UserId);
			GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKSanitizeMultipleStrings>(
				GDKSubsystem
				, Context
				, UnsanitizedArray
				, FOnlineAsyncTaskGDKSanitizeMultipleStringsComplete::CreateThreadSafeSP(this, &FMessageSanitizerGDK::HandleMessageArraySanitized, CompletionDelegate, AlreadyProcessedMessages, AlreadyProcessedIndex, UnsanitizedArray)
				);
		}
		else if (AlreadyProcessedMessages.Num() > 0)
		{
			GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, AlreadyProcessedMessages]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayNames_Delegate);
				CompletionDelegate.ExecuteIfBound(true, AlreadyProcessedMessages);
			});
		}
		else
		{
			GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayNames]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayNames_Delegate);
				CompletionDelegate.ExecuteIfBound(true, DisplayNames);
			});
		}
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, DisplayNames]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_SanitizeDisplayNames_Delegate);
			CompletionDelegate.ExecuteIfBound(true, DisplayNames);
		});
	}
}

void FMessageSanitizerGDK::QueryBlockedUser(int32 LocalUserNum, const FString& FromUserIdStr, const FString& FromPlatform, const FOnQueryUserBlockedResponse& CompletionDelegate)
{
	IOnlineIdentityPtr IdentityInt = GDKSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdPtr LocalUserId = IdentityInt->GetUniquePlayerId(LocalUserNum);
		if (LocalUserId.IsValid())
		{
			const FString LocalUserIdStr = LocalUserId->ToString();
			FBlockedUserData& UserBlockQueries = UserBlockData.FindOrAdd(LocalUserIdStr);
			UserBlockQueries.LocalUserId = LocalUserIdStr;

			FRemoteUserBlockInfo& RemoteUserData = UserBlockQueries.RemoteUserData.FindOrAdd(FromUserIdStr);
			if (RemoteUserData.State == EOnlineAsyncTaskState::NotStarted ||
				RemoteUserData.State == EOnlineAsyncTaskState::Failed)
			{
				// Fail catchall, set to InProgress if successfully setup
				RemoteUserData.State = EOnlineAsyncTaskState::Failed;
				RemoteUserData.RemoteUserId = FromUserIdStr;
				RemoteUserData.bIsSamePlatform = IOnlineSubsystem::GetLocalPlatformName() == FromPlatform;

				FOnlineUserGDKPtr UserInt = GDKSubsystem->GetUsersGDK();
				if (UserInt.IsValid())
				{
					if (RemoteUserData.bIsSamePlatform)
					{
						FUniqueNetIdPtr FromUserId = IdentityInt->CreateUniquePlayerId(FromUserIdStr);
						if (FromUserId.IsValid())
						{
							RemoteUserData.State = EOnlineAsyncTaskState::InProgress;

							UserInt->QueryUserCommunicationPermissions(
								*LocalUserId,
								{ FromUserId.ToSharedRef() },
								{ COMMUNICATE_USING_TEXT },
								FOnGDKCommunicationPermissionsQueryComplete::CreateThreadSafeSP(this, &FMessageSanitizerGDK::OnQueryBlockedUserComplete, CompletionDelegate)
							);
						}
					}
					else
					{
						bool bUseCrossNetworkTextPermission = true;
						GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bUseCrossNetworkTextPermission"), bUseCrossNetworkTextPermission, GEngineIni);
						if (bUseCrossNetworkTextPermission)
						{
							RemoteUserData.State = EOnlineAsyncTaskState::InProgress;

							TArray<FString> OtherPlatformIds;
							OtherPlatformIds.Add(FromUserIdStr);

							UserInt->QueryAnonymousUserCommunicationPermissions(
								*LocalUserId,
								{ XblAnonymousUserType::CrossNetworkUser, XblAnonymousUserType::CrossNetworkFriend },
								{ COMMUNICATE_USING_TEXT },
								FOnGDKAnonymousUserCommunicationPermissionsQueryComplete::CreateThreadSafeSP(this, &FMessageSanitizerGDK::OnQueryCrossPlatformPermissionsComplete, LocalUserId.ToSharedRef(), OtherPlatformIds, CompletionDelegate)
							);
						}
					}
				}
			}

			if (RemoteUserData.State == EOnlineAsyncTaskState::InProgress)
			{
				// Just wait for the existing query to finish
				RemoteUserData.Delegates.Add(CompletionDelegate);
			}
			else 
			{
				// Query previously complete or failed, return a response
				ensure(RemoteUserData.State == EOnlineAsyncTaskState::Done || RemoteUserData.State == EOnlineAsyncTaskState::Failed);
				ensure(RemoteUserData.Delegates.Num() == 0);
				
				UE_LOGF(LogOnlineSanitizeGDK, Verbose, "QueryBlockedUser - cached result local(%ls) remote(%ls) blocked:%d", *LocalUserIdStr, *FromUserIdStr, RemoteUserData.bIsBlocked);

				FBlockedQueryResult Result;
				Result.UserId = FromUserIdStr;
				Result.bIsBlocked = (RemoteUserData.State != EOnlineAsyncTaskState::Failed) ? RemoteUserData.bIsBlocked : true;
				Result.bIsBlockedNonFriends = RemoteUserData.bIsBlockedNonFriends;
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_QueryBlockedUser_Delegate);
				CompletionDelegate.ExecuteIfBound(Result);
			}
		}
	}
}

void FMessageSanitizerGDK::FRemoteUserBlockInfo::UpdatePermission(bool bSucceeded, bool bInIsBlocked, bool bInIsBlockedNonFriends)
{
	UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete - user(%ls) blocked:%d blockedNonFriends:%d", *RemoteUserId, bInIsBlocked, bInIsBlockedNonFriends);

	bIsBlocked = bInIsBlocked;
	bIsBlockedNonFriends = bInIsBlockedNonFriends;
	State = bSucceeded ? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;

	// fire delegates for any/all QueryBlockedUser calls made while this one was in flight
	const FBlockedQueryResult Result(bIsBlocked, bIsBlockedNonFriends, RemoteUserId);
	const TArray<FOnQueryUserBlockedResponse> DelegatesCopy = MoveTemp(Delegates);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_FRemoteUserBlockInfo_UpdatePermission_DelegateLoop);
	for (const FOnQueryUserBlockedResponse& Delegate : DelegatesCopy)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_FRemoteUserBlockInfo_UpdatePermission_Delegates);
		Delegate.ExecuteIfBound(Result);
	}
}

void FMessageSanitizerGDK::OnQueryBlockedUserComplete(const FOnlineError& RequestStatus, const FUniqueNetIdRef& RequestingUser, const FCommunicationPermissionResultsMap& Results, FOnQueryUserBlockedResponse CompletionDelegate)
{
	if (RequestingUser->IsValid())
	{
		const FString LocalUserId = RequestingUser->ToString();

		FBlockedUserData* UserBlockQueries = UserBlockData.Find(LocalUserId);
		if (UserBlockQueries)
		{
			for (const TPair<FUniqueNetIdRef, bool>& Result : Results)
			{
				FString RemoteUserId = Result.Key->ToString();
				bool bHasPermission = Result.Value;

				FRemoteUserBlockInfo* RemoteUserData = UserBlockQueries->RemoteUserData.Find(RemoteUserId);
				if (RemoteUserData)
				{
					UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete - user(%ls) blocked:%d", *RemoteUserId, !bHasPermission);
					const bool bIsBlocked = !bHasPermission;
					RemoteUserData->UpdatePermission(RequestStatus.bSucceeded, bIsBlocked, bIsBlocked);
				}
				else
				{
					UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete: No remote data found for user %ls", *RemoteUserId);
				}
			}
		}
		else
		{
			UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete: No local data found for user %ls", *LocalUserId);
		}
	}
	else
	{
		UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete: Invalid requesting user");
	}
}

void FMessageSanitizerGDK::OnQueryCrossPlatformPermissionsComplete(const FOnlineError& RequestStatus, const FAnonymousUserCommunicationPermissionResultsMap& Results, const FUniqueNetIdRef RequestingUser, const TArray<FString> OtherPlatformIds, FOnQueryUserBlockedResponse CompletionDelegate)
{
	if (RequestingUser->IsValid())
	{
		const FString LocalUserId = RequestingUser->ToString();

		FBlockedUserData* UserBlockQueries = UserBlockData.Find(LocalUserId);
		if (UserBlockQueries)
		{
			bool bCrossPlatformIsBlocked = false;
			bool bCrossPlatformIsBlockedNonFriends = false;
			for (const TPair<XblAnonymousUserType, bool>& Result : Results)
			{
				XblAnonymousUserType AnonymousUserType = Result.Key;
				bool bHasPermission = Result.Value;

				if (AnonymousUserType == XblAnonymousUserType::CrossNetworkUser)
				{
					bCrossPlatformIsBlocked = !bHasPermission;
				}
				else if (AnonymousUserType == XblAnonymousUserType::CrossNetworkFriend)
				{
					bCrossPlatformIsBlockedNonFriends = !bHasPermission;
				}
			}
			
			for (const FString& OtherPlatformId : OtherPlatformIds)
			{
				FRemoteUserBlockInfo* RemoteUserData = UserBlockQueries->RemoteUserData.Find(OtherPlatformId);
				if (RemoteUserData)
				{
					UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete - other platform user(%ls) blocked:%d blockednonfriend:%d", *OtherPlatformId, bCrossPlatformIsBlocked, bCrossPlatformIsBlockedNonFriends);
					RemoteUserData->UpdatePermission(RequestStatus.bSucceeded, bCrossPlatformIsBlocked, bCrossPlatformIsBlockedNonFriends);
				}
			}
		}
		else
		{
			UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete: No local data found for user %ls", *LocalUserId);
		}
	}
	else
	{
		UE_LOGF(LogOnlineSanitizeGDK, Verbose, "OnQueryBlockedUserComplete: Invalid requesting user");
	}
}

void FMessageSanitizerGDK::ResetBlockedUserCache()
{
	UE_LOGF(LogOnlineSanitizeGDK, Verbose, "ResetBlockedUserCache");
	for (TPair<FString, FBlockedUserData>& LocalUserBlockData : UserBlockData)
	{
		for (TPair<FString, FRemoteUserBlockInfo>& RemoteUserData : LocalUserBlockData.Value.RemoteUserData)
		{
			if (RemoteUserData.Value.State != EOnlineAsyncTaskState::InProgress)
			{
				RemoteUserData.Value.State = EOnlineAsyncTaskState::NotStarted;
			}
		}
	}
}

void FMessageSanitizerGDK::HandleMessageSanitized(bool bSuccess, const FString& SanitizedMessage, FOnMessageProcessed CompletionDelegate, FString UnsanitizedMessage)
{
	// Cache the result only when success, because it can fail to sanitize when xbox live become offline. 
	// When back to online, we need to sanitize it again instead of getting it from cache.
	if (bSuccess && !WordMap.Find(UnsanitizedMessage))
	{
		WordMap.Emplace(MoveTemp(UnsanitizedMessage), SanitizedMessage);
	}
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_HandleMessageSanitized_Delegate);
	CompletionDelegate.ExecuteIfBound(bSuccess, SanitizedMessage);
}

void FMessageSanitizerGDK::HandleMessageArraySanitized(bool bSuccess, const TArray<FString>& SanitizedArrayMessage, FOnMessageArrayProcessed CompletionDelegate, TArray<FString> AlreadyProcessedMessages, TArray<int32> AlreadyProcessedIndex, TArray<FString> UnsanitizedArrayMessage)
{
	if (!bSuccess)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_HandleMessageArraySanitized_Delegate);
		CompletionDelegate.ExecuteIfBound(false, SanitizedArrayMessage);
	}
	else
	{
		if (SanitizedArrayMessage.Num() != UnsanitizedArrayMessage.Num())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_HandleMessageArraySanitized_Delegate);
			CompletionDelegate.ExecuteIfBound(false, SanitizedArrayMessage);
			return;
		}

		check(SanitizedArrayMessage.Num() == UnsanitizedArrayMessage.Num())
		
		TArray<FString> SanitizedStringArray;
		
		for (int32 iMessageIndex = 0; iMessageIndex < UnsanitizedArrayMessage.Num(); iMessageIndex++)
		{
			if (!WordMap.Find(UnsanitizedArrayMessage[iMessageIndex]))
			{
				WordMap.Emplace(MoveTemp(UnsanitizedArrayMessage[iMessageIndex]), SanitizedArrayMessage[iMessageIndex]);
			}
			SanitizedStringArray.Add(SanitizedArrayMessage[iMessageIndex]);
		}

		for (int32 MessageIndex = 0; MessageIndex < AlreadyProcessedMessages.Num(); MessageIndex++)
		{
			SanitizedStringArray.Insert(AlreadyProcessedMessages[MessageIndex], AlreadyProcessedIndex[MessageIndex]);
		}
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerGDK_HandleMessageArraySanitized_Delegate);
		CompletionDelegate.ExecuteIfBound(true, SanitizedStringArray);
	}
}
#endif //WITH_GRDK