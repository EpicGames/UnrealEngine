// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"
#include "Interfaces/IMessageSanitizerInterface.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineUserInterfaceGDK.h"
#include "OnlineSubsystemGDKPackage.h"

class FOnlineSubsystemGDK;

/**
 * Implements the GDK specific interface chat message sanitization
 */
class FMessageSanitizerGDK :
	public IMessageSanitizer
{

public:

	// IMessageSanitizer
	virtual void SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate) override;
	virtual void SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate) override;
	virtual void QueryBlockedUser(int32 LocalUserNum, const FString& FromUserIdStr, const FString& FromPlatform, const FOnQueryUserBlockedResponse& InCompletionDelegate) override;
	virtual void ResetBlockedUserCache() override;
	// FMessageSanitizerGDK

	explicit FMessageSanitizerGDK(FOnlineSubsystemGDK* InGDKSubsystem);
	virtual ~FMessageSanitizerGDK();

private:

	void HandleAppResume();
	void HandleAppReactivated();
	void OnQueryBlockedUserComplete(const FOnlineError& RequestStatus, const FUniqueNetIdRef& RequestingUser, const FCommunicationPermissionResultsMap& Results, FOnQueryUserBlockedResponse CompletionDelegate);
	void OnQueryCrossPlatformPermissionsComplete(const FOnlineError& RequestStatus, const FAnonymousUserCommunicationPermissionResultsMap& Results, const FUniqueNetIdRef RequestingUser, const TArray<FString> OtherPlatformIds, FOnQueryUserBlockedResponse CompletionDelegate);

	FDelegateHandle AppResumeDelegateHandle;
	FDelegateHandle AppReactivatedDelegateHandle;

	/** Reference to the main GDK subsystem */
	FOnlineSubsystemGDK* GDKSubsystem;

	struct FRemoteUserBlockInfo
	{
		FRemoteUserBlockInfo()
			: State(EOnlineAsyncTaskState::NotStarted)
			, bIsBlocked(false)
			, bIsBlockedNonFriends(false)
			, bIsSamePlatform(true)
		{ }

		/** Remote user id */
		FString RemoteUserId;

		/** State of the query */
		EOnlineAsyncTaskState::Type State;

		/** Is this user blocked, valid only if bIsComplete is true */
		bool bIsBlocked;
		/** Is this user blocked only for non-friends, valid only if bIsComplete is true */
		bool bIsBlockedNonFriends;
		/** True if this user is on the same platform. ie. GDK */
		bool bIsSamePlatform;

		/** Delegates to fire when the query is complete, should only be filled while bIsComplete is false */
		TArray<FOnQueryUserBlockedResponse> Delegates;

		/** Handle updated permission and trigger delegates */
		void UpdatePermission(bool bSucceeded, bool bInIsBlocked, bool bIsBlockedNonFriends);
	};

	struct FBlockedUserData
	{
		/** Local user id */
		FString LocalUserId;
		/** Mapping from remote users to their cached info */
		TMap<FString, FRemoteUserBlockInfo> RemoteUserData;
	};

	/** Mapping of all local users to the block data for remote users */
	TMap<FString, FBlockedUserData> UserBlockData;

private:
	/** Handle the sanitized strings and trigger completion delegates */
	void HandleMessageSanitized(bool bSuccess, const FString& SanitizedString, FOnMessageProcessed CompletionDelegate, FString UnsanitazedString);
	void HandleMessageArraySanitized(bool bSuccess, const TArray<FString>& SanitizedArrayMessage, FOnMessageArrayProcessed CompletionDelegate, TArray<FString> AlreadyProcessedMessages, TArray<int32> AlreadyProcessedIndex, TArray<FString> UnsanitizedArrayMessage);

private:
	// Holds a map of sanitized words
	TMap<FString, FString> WordMap;

};

typedef TSharedPtr<FMessageSanitizerGDK, ESPMode::ThreadSafe> FMessageSanitizerGDKPtr;
