// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAchievementsInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineEventsInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKWriteAchievements.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryAchievements.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <xsapi-c/xbox_live_context_settings_c.h>
//#include <httpClient/httpClient.h>
//#include <httpClient/httpProvider.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define TEST_ACHIEVEMENTS			0

FOnlineAchievementsGDK::FOnlineAchievementsGDK(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
	, AchievementMode(EGDKAchievementMode::Default)
{
	GetAchievementModeFromConfig();

#if TEST_ACHIEVEMENTS		// Enable this to test achievements and events
	TestEventsAndAchievements();
#endif
}

FOnlineAchievementsGDK::~FOnlineAchievementsGDK()
{
}

bool FOnlineAchievementsGDK::LoadAndInitFromJsonConfig(const TCHAR* JsonConfigName)
{
	// check legacy path first
	FString JsonConfigFilename = FPaths::ProjectDir() / TEXT("Platforms/GDK/Config/OSS") / JsonConfigName;
	if (!FPaths::FileExists(JsonConfigFilename))
	{
		JsonConfigFilename = FPaths::ProjectDir() / TEXT("Config/Xbl") / JsonConfigName;
	}
	else if (ensureMsgf(!FPaths::FileExists(FPaths::ProjectDir() / TEXT("Config/Xbl") / JsonConfigName), TEXT("%s file found in both deprecated Platforms/GDK/Config/OSS and new Config/Xbl paths - remove the deprecated one"), JsonConfigName))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAchievementsGDK: Platforms/GDK/Config/OSS/ path for %s is deprecated - move it to Config/Xbl/"), JsonConfigName);
	}

	FString JsonText;

	if (!FFileHelper::LoadFileToString(JsonText, *JsonConfigFilename))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAchievementsGDK: Failed to find json OSS achievements config: %s"), *JsonConfigFilename);
		return false;
	}

	if (!AchievementsConfig.FromJson(JsonText))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAchievementsGDK: Failed to parse json OSS achievements config: %s"), *JsonConfigFilename);
		return false;
	}

	return true;
}

void FOnlineAchievementsGDK::TestEventsAndAchievements()
{
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();

	check(Identity.IsValid());

	uint64 PlayerXUID = 0;
	FGDKContextHandle GDKContext;
	{
		bool bSuccess = false;
		FGDKUserHandle TestUser = (Identity->GetCachedUsers())[0];
		if (TestUser.IsValid())
		{
			GDKContext = GDKSubsystem->GetGDKContext(TestUser);
			if (GDKContext.IsValid())
			{
				bSuccess = SUCCEEDED(XUserGetId(TestUser, &PlayerXUID));
			}
		}

		if (!bSuccess)
		{
			return;
		}
	}
	
	// Show service calls from GDK Services on the UI for easy debugging
	
	XblCallRoutedHandler ServiceCallHandler = [](XblServiceCallRoutedArgs args, void* context)
	{
		/*
		const char* HttpMethod = nullptr;
		const char* HttpUrl = nullptr;
		HCHttpCallRequestGetUrl(args.call, &HttpMethod, &HttpUrl);
		
		const char* HttpRequestBody = nullptr;
		HCHttpCallRequestGetRequestBodyString(args.call, &HttpRequestBody);
		
		uint32 HttpStatusCode = 0;
		HCHttpCallResponseGetStatusCode(args.call, &HttpStatusCode);
		
		const char* HttpResponseBody = nullptr;
		HCHttpCallResponseGetResponseString(args.call, &HttpResponseBody);

		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("[URL]: %s %s"), UTF8_TO_TCHAR(HttpMethod), UTF8_TO_TCHAR(HttpUrl));
		if (HttpRequestBody)
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("[RequestBody]: %s"), UTF8_TO_TCHAR(HttpRequestBody));
		}
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT(""));
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("[Response]: %d %s"), HttpStatusCode, UTF8_TO_TCHAR(HttpResponseBody));
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT(""));
		*/
	};

	XblFunctionContext ServiceCallContext = XblAddServiceCallRoutedHandler(ServiceCallHandler, this);

	FOnlineEventsGDK * EventInterface = (FOnlineEventsGDK*)GDKSubsystem->GetEventsInterface().Get();

	FUniqueNetIdGDKRef PlayerId = FUniqueNetIdGDK::Create(PlayerXUID);

	// Start player session
	{
		FOnlineEventParms Parms;

		Parms.Add(TEXT("GameplayModeId"), FVariantData((int32)1));
		Parms.Add(TEXT("DifficultyLevelId"), FVariantData((int32)1));
		Parms.Add(TEXT("MapName"), FVariantData(FString("Highrise")));

		EventInterface->TriggerEvent(*PlayerId, TEXT("PlayerSessionStart"), Parms);
	}

	// Test a stat change event
	{
		FOnlineEventParms Parms;

		Parms.Add(TEXT("SectionId"), FVariantData((int32)0));
		Parms.Add(TEXT("GameplayModeId"), FVariantData((int32)0));
		Parms.Add(TEXT("DifficultyLevelId"), FVariantData((int32)0));
		Parms.Add(TEXT("PlayerRoleId"), FVariantData((int32)0));
		Parms.Add(TEXT("PlayerWeaponId"), FVariantData((int32)0));
		Parms.Add(TEXT("EnemyRoleId"), FVariantData((int32)0));
		Parms.Add(TEXT("KillTypeId"), FVariantData((int32)0));
		Parms.Add(TEXT("LocationX"), FVariantData((float)0));
		Parms.Add(TEXT("LocationY"), FVariantData((float)0));
		Parms.Add(TEXT("LocationZ"), FVariantData((float)0));
		Parms.Add(TEXT("EnemyWeaponId"), FVariantData((int32)0));

		EventInterface->TriggerEvent(*PlayerId, TEXT("KillOponent"), Parms);
	}

	// Give test achievement
	{
		FOnlineEventParms Parms;
		Parms.Add(TEXT("AchievementIndex"), FVariantData((int32)9));
		//Parms.Add(TEXT("AchievementIndex"), FVariantData((uint64)0xFFFFFFFFF));				// This should trigger loss of data error
		//Parms.Add(TEXT("AchievementIndex"), FVariantData(FString(TEXT("Test"))));		// This should trigger conversion error

		EventInterface->TriggerEvent(*PlayerId, TEXT("TempActivateAchiement"), Parms);
	}

	// End player session
	{
		FOnlineEventParms Parms;

		Parms.Add(TEXT("GameplayModeId"), FVariantData((int32)1));
		Parms.Add(TEXT("DifficultyLevelId"), FVariantData((int32)1));
		Parms.Add(TEXT("ExitStatusId"), FVariantData((int32)0));
		Parms.Add(TEXT("MapName"), FVariantData(FString("Highrise")));
		Parms.Add(TEXT("PlayerScore"), FVariantData((int32)0));
		Parms.Add(TEXT("PlayerWon"), FVariantData((bool)false));

		EventInterface->TriggerEvent(*PlayerId, TEXT("PlayerSessionEnd"), Parms);
	}
}

void FOnlineAchievementsGDK::WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate)
{
	FUniqueNetIdGDKRef GDKId = StaticCastSharedRef<const FUniqueNetIdGDK>(PlayerId.AsShared());
	if (!GDKId->IsValid())
	{
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAchievementsGDK_WriteAchievements_Delegate);
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(PlayerId);
	if (!GDKContext.IsValid())
	{
		WriteObject->WriteState = EOnlineAsyncTaskState::Failed;
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	bool bResult = true;

	const IOnlineEventsPtr EventInterface = GDKSubsystem->GetEventsInterface();
	check(EventInterface.IsValid());

	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	FGDKUserHandle GDKUser = Identity->GetUserForUniqueNetId(*GDKId);

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKWriteAchievements>(GDKSubsystem, GDKContext, GDKId, WriteObject, Delegate);
};

void FOnlineAchievementsGDK::QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	check(GDKSubsystem)

	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (!Identity.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAchievementsGDK_QueryAchievements_Delegate);
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(PlayerId);
	FGDKUserHandle GDKUser = Identity->GetUserForUniqueNetId(*GDKUserId);
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryAchievements>(GDKSubsystem, GDKContext, GDKUserId, FOnQueryGDKAchievementCompleteDelegate::CreateThreadSafeSP(this, &FOnlineAchievementsGDK::HandleQueryGDKAchievementComplete, Delegate));
}

void FOnlineAchievementsGDK::HandleQueryGDKAchievementComplete(const FUniqueNetId& PlayerNetId, bool bIsSuccess, const TArray<XblAchievement>& AchievementArray, FOnQueryAchievementsCompleteDelegate Delegate)
{
	FOnlineAchievementsGDKPtr GDKSubsystemAchievements = GDKSubsystem->GetAchievementsInterfaceGDK();
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(PlayerNetId);

	if (GDKSubsystemAchievements.IsValid() && bIsSuccess)
	{
		TArray< FOnlineAchievement > AchievementsForPlayer;

		for (const XblAchievement& GDKAchievement : AchievementArray)
		{
			FOnlineAchievement OnlineAchievement;

			// Copy over id

			OnlineAchievement.Id = GetAchievementNameFromId(FCString::Atoi(UTF8_TO_TCHAR(GDKAchievement.id)));
			OnlineAchievement.Progress = (GDKAchievement.progressState == XblAchievementProgressState::Achieved) ? 100 : 0;
			AchievementsForPlayer.Add(OnlineAchievement);
			FOnlineAchievementDesc Desc;

			// Fill in description
			Desc.Title = FText::FromString(UTF8_TO_TCHAR(GDKAchievement.name));
			Desc.LockedDesc = FText::FromString(UTF8_TO_TCHAR(GDKAchievement.lockedDescription));
			Desc.UnlockedDesc = FText::FromString(UTF8_TO_TCHAR(GDKAchievement.unlockedDescription));
			Desc.bIsHidden = GDKAchievement.isSecret;
			Desc.UnlockTime = FDateTime(1601, 1, 1) + FTimespan((int64)GDKAchievement.progression.timeUnlocked);

			// Should replace any already existing values
			GDKSubsystemAchievements->AchievementDescriptions.Add(OnlineAchievement.Id, Desc);
		}

		// Should replace any already existing values
		GDKSubsystemAchievements->PlayerAchievements.Add(GDKUserId, AchievementsForPlayer);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAchievementsGDK_HandleQueryGDKAchievementComplete_Delegate);
		Delegate.ExecuteIfBound(PlayerNetId, true);
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineAchievementsGDK_HandleQueryGDKAchievementComplete_Delegate);
		Delegate.ExecuteIfBound(PlayerNetId, false);
	}
}

void FOnlineAchievementsGDK::QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	// Just query achievements to get descriptions
	QueryAchievements(PlayerId, Delegate);
}

EOnlineCachedResult::Type FOnlineAchievementsGDK::GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement)
{
	TArray<FOnlineAchievement>* AchievementsPtr = PlayerAchievements.Find(PlayerId.AsShared());

	if (AchievementsPtr == nullptr)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GDK achievements have not been read for player %s"), *PlayerId.ToString());
		return EOnlineCachedResult::NotFound;
	}

	const TArray<FOnlineAchievement>& Achievements = *AchievementsPtr;
	for (const FOnlineAchievement& Achievement : Achievements)
	{
		if (Achievement.Id == AchievementId)
		{
			OutAchievement = Achievement;
			return EOnlineCachedResult::Success;
		}
	}

	return EOnlineCachedResult::NotFound;
};

EOnlineCachedResult::Type FOnlineAchievementsGDK::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements)
{
	TArray<FOnlineAchievement>* AchievementsPtr = PlayerAchievements.Find(PlayerId.AsShared());

	if (AchievementsPtr == nullptr)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GDK achievements have not been read for player %s"), *PlayerId.ToString());
		return EOnlineCachedResult::NotFound;
	}

	OutAchievements = *AchievementsPtr;

	return EOnlineCachedResult::Success;
};

EOnlineCachedResult::Type FOnlineAchievementsGDK::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	FOnlineAchievementDesc* AchievementDesc = AchievementDescriptions.Find(AchievementId);
	
	if (AchievementDesc == nullptr)
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GDK achievements have not been read for id: %s"), *AchievementId);
		return EOnlineCachedResult::NotFound;
	}

	OutAchievementDesc = *AchievementDesc;
	return EOnlineCachedResult::Success;
};

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsGDK::ResetAchievements(const FUniqueNetId& PlayerId)
{
	UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("Resetting achievements not supported on GDK. Use Xbox Player Data Reset tool. "));
	return false;
};
#endif // !UE_BUILD_SHIPPING

void FOnlineAchievementsGDK::GetAchievementModeFromConfig()
{
	// Load our Achievement Mode
	FString AchievementModeString;
	if (GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("AchievementMode"), AchievementModeString, GEngineIni))
	{
		if (AchievementModeString.Contains(TEXT("2013")))
		{
			AchievementMode = EGDKAchievementMode::Mode2013;
		}
		else if (AchievementModeString.Contains(TEXT("2017")))
		{
			AchievementMode = EGDKAchievementMode::Mode2017;
		}
		else
		{
			UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("FOnlineAchievementsGDK: invalid configuration value \"%s\" for AchievementMode."), *AchievementModeString);
		}

		UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("FOnlineAchievementsGDK: AchievementMode set to \"%d\""), static_cast<int32>(AchievementMode));
	}
	else
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("FOnlineAchievementsGDK: No AchievementMode set; defaulting AchievementMode to \"%d\""), static_cast<int32>(AchievementMode));
	}
}

#endif //WITH_GRDK