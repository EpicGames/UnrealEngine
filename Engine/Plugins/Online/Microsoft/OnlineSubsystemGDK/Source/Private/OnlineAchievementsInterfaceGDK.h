// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineEventsInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDKPackage.h"
#include "Serialization/JsonSerializerMacros.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/achievements_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

const int32 INVALID_ACHIEVEMENT_ID = -1;

enum class EGDKAchievementMode : uint16
{
	/** 2013 stats are driven via events */
	Mode2013 = 2013,
	/** 2017 stats are title-managed */
	Mode2017 = 2017,

	/** Default mode to use if not specified */
	Default = Mode2013
};

class FAchievementsConfig : public FJsonSerializable
{
public:
	FString								AchievementEventName;
	FJsonSerializableKeyValueMapInt		AchievementMap;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE( "AchievementEventName", AchievementEventName );
		JSON_SERIALIZE_MAP( "AchievementMap", AchievementMap );
	END_JSON_SERIALIZER
};

/**
* Delegate fired when an achievement has been queried
*/
DECLARE_DELEGATE_ThreeParams(FOnQueryGDKAchievementCompleteDelegate, const FUniqueNetId&, bool, const TArray<XblAchievement>&);

/**
 *	FOnlineAchievementsGDK - Interface class for achievements (GDK implementation)
 */
class FOnlineAchievementsGDK : public IOnlineAchievements, public TSharedFromThis< FOnlineAchievementsGDK, ESPMode::ThreadSafe>
{
private:

	void GetAchievementModeFromConfig();
	bool LoadAndInitFromJsonConfig( const TCHAR* JsonConfigName );
	void TestEventsAndAchievements();

	/** Pointer to owning live subsystem */
	class FOnlineSubsystemGDK *							GDKSubsystem;

	/** Config settings initialized from json file */
	FAchievementsConfig										AchievementsConfig;

	/** Mapping of players to their achievements */
	TUniqueNetIdMap<TArray<FOnlineAchievement>> PlayerAchievements;

	/** Cached achievement descriptions for an Id */
	TMap<FString, FOnlineAchievementDesc>					AchievementDescriptions;

	const FOnAchievementsWrittenDelegate* OnAchievementsWrittenDelegate = nullptr;
	const FOnAchievementsWrittenDelegate* OnAchievementsQueriedDelegate = nullptr;


PACKAGE_SCOPE:

public:

	/** What version of achievements do we use for unlocking achievements? */
	EGDKAchievementMode AchievementMode;


	//~ Begin IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override;
	virtual void QueryAchievementDescriptions( const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate() ) override;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement> & OutAchievements) override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineAchievements Interface

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineAchievementsGDK(class FOnlineSubsystemGDK* InSubsystem);

	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineAchievementsGDK();

	int32 GetAchievementIdFromName(const FString& AchievementName)
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}

		int32* AchievementId = AchievementsConfig.AchievementMap.Find(AchievementName);
		return AchievementId ? *AchievementId : INVALID_ACHIEVEMENT_ID;
	}

	
	const FString& GetAchievementNameFromId(int32 AchievementId)
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}

		static const FString NoAchievment=FString("No Mathing achievement");
		const FString* AchievementName = AchievementsConfig.AchievementMap.FindKey(AchievementId);	
		return AchievementName ? *AchievementName : NoAchievment;
	}

	const FString& GetAchievementEventName()
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}

		return AchievementsConfig.AchievementEventName;
	}

	void HandleQueryGDKAchievementComplete(const FUniqueNetId& PlayerNetId, bool isSuccess, const TArray<XblAchievement>& AchievementArray, FOnQueryAchievementsCompleteDelegate Delegate);

private:
	/**
	 *	Async event that notifies when a Query achievements operation has completed.
	 */
	class FAsyncEventQueryCompleted : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
	{
		/** Hidden on purpose */
		FAsyncEventQueryCompleted()
			: FOnlineAsyncEvent(nullptr)
			, PlayerId(FUniqueNetIdGDK::EmptyId())
		{
		}

		/** The user who made the request */
		FUniqueNetIdGDKRef PlayerId;

		TArray<XblAchievement*> Achievements;

		/** True if the set presence operation succeeded, false if it didn't. */
		bool bWasSuccessful;

		/** Delegate to execute on the game thread to notify it that the operation is complete. */
		FOnQueryAchievementsCompleteDelegate Delegate;

	public:
		/**
		 * Constructor.
		 *
		 * @param InGDKSubsystem The owner of the external UI interface that triggered this event.
		 * @param InUsers The users whose presence was retrieved.
		 * @param InRecord The presence information for the user.
		 * @param InWasSuccessful True if the set presence operation succeeded, false if it didn't.
		 */
		FAsyncEventQueryCompleted(FOnlineSubsystemGDK* InGDKSubsystem, const FUniqueNetIdGDKRef& InPlayerId, const TArray<XblAchievement*>& InAchievements, bool InWasSuccessful, const FOnQueryAchievementsCompleteDelegate& InDelegate)
			: FOnlineAsyncEvent(InGDKSubsystem)
			, PlayerId(InPlayerId)
			, Achievements(InAchievements)
			, bWasSuccessful(InWasSuccessful)
			, Delegate(InDelegate)
		{
		}

		virtual void		Finalize() override;
		virtual FString		ToString() const override;
		virtual void		TriggerDelegates() override;
	};

	/** Process the AchievementsResult and query for the next "page" of results if necessary */
	void ProcessGetAchievementsResults(const TArray<XblAchievement*>& AllAchievements, FUniqueNetIdGDKRef UserGDK, const FOnQueryAchievementsCompleteDelegate Delegate);
};

typedef TSharedPtr<FOnlineAchievementsGDK, ESPMode::ThreadSafe> FOnlineAchievementsGDKPtr;
