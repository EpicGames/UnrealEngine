// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"

FOnlineAsyncTaskManagerGDK::~FOnlineAsyncTaskManagerGDK()
{
	// Cancel any GDK-specific in-flight tasks before base class destructor cleans up the queues
	OnlineTasksQueue.CancelPendingTasksAndDestroyQueue();
}

void FOnlineAsyncTaskManagerGDK::OnlineTick()
{
	check(GDKSubsystem);
	check(FPlatformTLS::GetCurrentThreadId() == OnlineThreadId);
}

FOnlineAsyncTaskGDK::FOnlineAsyncTaskGDK(FOnlineSubsystemGDK* const InGDKSubsystem, const FString& InAsyncTaskName, const int32 InUserIndex /*= -1*/)
	: FOnlineAsyncTaskBasic(InGDKSubsystem)
	, UserIndex(InUserIndex)
	, AsyncTaskName(InAsyncTaskName)
{
	AsyncBlock = CreateAsyncBlock();

	double Timeout = 0.0f;
	if (GConfig->GetDouble(TEXT("GDKTaskTimeouts"), *AsyncTaskName, Timeout, GEngineIni))
	{
		TimeoutInSeconds = Timeout;
	}
}

FGDKAsyncBlockPtr FOnlineAsyncTaskGDK::CreateAsyncBlock(void* UserData, FGDKAsyncBlockDelegate Delegate)
{
	FGDKAsyncBlockPtr LocalAsyncBlock = MakeShared<FGDKAsyncBlock>(UserData, Delegate);
	LocalAsyncBlock->GetInnerBlockForGDKAPI()->queue = Subsystem->GetAsyncTaskManager()->GetGDKTaskQueue();
	AsyncBlocks.Add(LocalAsyncBlock);
	return LocalAsyncBlock;
}


FOnlineAsyncTaskGDK::~FOnlineAsyncTaskGDK()
{
	for (FGDKAsyncBlockPtr AsyncBlockLocal : AsyncBlocks)
	{
		checkf(!AsyncBlockLocal.IsValid() || XAsyncGetStatus(AsyncBlockLocal->GetInnerBlockForGDKAPI(), false) != E_PENDING,
			TEXT("[FOnlineAsyncTaskGDK::~FOnlineAsyncTaskGDK] AsyncBlock still pending on AsyncTask destruction for task %s"), *AsyncTaskName);
	}
}

void FOnlineAsyncTaskGDK::CancelWhenTimeout()
{
	if (TimeoutInSeconds.IsSet() && !bIsComplete && !bCancelled && GetElapsedTime() > TimeoutInSeconds.GetValue())
	{
		UE_LOG_ONLINE(Warning, TEXT("Async task '%s' timed out in %.2f seconds, cancelling it..."), *ToString(), GetElapsedTime());
		bCancelled = true;
		for (FGDKAsyncBlockPtr AsyncBlockPtr : AsyncBlocks)
		{
			XAsyncCancel(*AsyncBlockPtr);
		}
	}
}

#endif //WITH_GRDK