// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDKPackage.h"
#include "OnlineSubsystem.h"
#include "GDKTaskQueueHelpers.h"

#include <type_traits>

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XAsync.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"



class FOnlineSubsystemGDK;

typedef TSharedPtr<FGDKAsyncBlock> FGDKAsyncBlockPtr;

/**
 * Base class that holds a delegate to fire when a given async task is complete
 */
class FOnlineAsyncTaskGDK : public FOnlineAsyncTaskBasic<FOnlineSubsystemGDK>, public TSharedFromThis<FOnlineAsyncTaskGDK, ESPMode::ThreadSafe>
{
PACKAGE_SCOPE:
	/** GDK user index associated with this task **/
	int32 UserIndex;
	TArray<FGDKAsyncBlockPtr> AsyncBlocks;
	FGDKAsyncBlockPtr AsyncBlock;
	FString AsyncTaskName;

public:
	/** Hidden on purpose */
	FOnlineAsyncTaskGDK() = delete;

	explicit FOnlineAsyncTaskGDK(FOnlineSubsystemGDK* const InGDKSubsystem, const FString& InAsyncTaskName, const int32 InUserIndex = -1);

	virtual ~FOnlineAsyncTaskGDK();

	virtual void ProcessResults() {};

	/**
	* Create a new async block for an async GDK call.
	*/
	FGDKAsyncBlockPtr CreateAsyncBlock(void* UserData, FGDKAsyncBlockDelegate Delegate);

	/**
	* Create a new async block for an async GDK call.
	*/
	void RemoveAsyncBlock(FGDKAsyncBlockPtr InAsyncBlock)
	{
		if (InAsyncBlock.IsValid())
		{
			AsyncBlocks.Remove(InAsyncBlock);
		}
	}

	FGDKAsyncBlockPtr FindAsyncBlock(XAsyncBlock* InAsyncBlock)
	{
		for (FGDKAsyncBlockPtr AsyncBlockPtr : AsyncBlocks)
		{
			if (AsyncBlockPtr->GetInnerBlockForGDKAPI() == InAsyncBlock)
			{
				return AsyncBlockPtr;
			}
		}

		return nullptr;
	}

	/**
	* Create a new async block for an async GDK call, with settings set to defaults for OnlineAsyncTaskGDK
	*/
	FGDKAsyncBlockPtr CreateAsyncBlock()
	{
		return CreateAsyncBlock(nullptr, [this](FGDKAsyncBlock* LambdaAsyncBlock) {
			ProcessResults();
		});
	}

	/**
	 * Handle start-up/initialize code here.  This is ran on the Game Thread the tick before the first call to Tick().
	 */
	virtual void Initialize() override
	{
	}

	/**
	 * Handle processing/updating the task here.  This is ran on the Online Thread.
	 */
	virtual void Tick() override
	{
	}

	/**
	 * Cancel when timeout if timeout is set for this task
	 */
	virtual void CancelWhenTimeout() override;

private:
	TOptional<double> TimeoutInSeconds;
	bool bCancelled = false;
};

/**
 *	GDK version of the async task manager to register the various Live callbacks with the engine
 */
class FOnlineAsyncTaskManagerGDK : public FOnlineAsyncTaskManager
{
protected:

	/** Cached reference to the main online subsystem */
	FOnlineSubsystemGDK* GDKSubsystem;

	FGDKAsyncTaskQueue OnlineTasksQueue;

public:

	FOnlineAsyncTaskManagerGDK(FOnlineSubsystemGDK* InOnlineSubsystem)
		: GDKSubsystem(InOnlineSubsystem)
	{
	}

	XTaskQueueHandle GetGDKTaskQueue() { return OnlineTasksQueue.GetQueue(); }

	virtual ~FOnlineAsyncTaskManagerGDK();

	// FOnlineAsyncTaskManager
	virtual void OnlineTick() override;
};
