// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineEventsInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "Serialization/JsonSerializerMacros.h"
#include "GDKThreadCheck.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/events_c.h>
#include <XGameRuntimeFeature.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FOnlineEventsGDK::FOnlineEventsGDK(FOnlineSubsystemGDK* InSubsystem) :
	Subsystem( InSubsystem )
{
}

bool FOnlineEventsGDK::TriggerEvent(const FUniqueNetId& PlayerId, const TCHAR* EventName, const FOnlineEventParms& Parms)
{
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGameEvent) == false)
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineEventsGDK::TriggerEvent: Event System not Available"));
		return false;
	}

	if (!PlayerId.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineEventsGDK::TriggerEvent: Invalid UniqueNetId"));
		return false;
	}

	if (EventName == nullptr)
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineEventsGDK::TriggerEvent: Invalid Event Name"));
		return false;
	}

	FUniqueNetIdGDKRef GDKPlayerId = FUniqueNetIdGDK::Cast(PlayerId);
	FGDKContextHandle GDKContext = Subsystem->GetGDKContext(*GDKPlayerId);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineEventsGDK::TriggerEvent: Cannot find GDK context for player id"));
		return false;
	}

	FGuid PlayerSessionId;
	FGuid* FoundPlayerSessionId = PlayerSessionIds.Find(GDKPlayerId);
	if (FoundPlayerSessionId)
	{
		PlayerSessionId = *FoundPlayerSessionId;
	}

	TSharedRef<FJsonObject> JsonObject(new FJsonObject());
	JsonObject->SetStringField(TEXT("MultiplayerCorrelationId"), PlayerSessionId.ToString(EGuidFormats::DigitsWithHyphens));

	for (const TPair< FString, FVariantData >& Pair : Parms)
	{
		EOnlineKeyValuePairDataType::Type ValueType = Pair.Value.GetType();
		const FString& FieldName = Pair.Key;
		switch (ValueType)
		{
			case EOnlineKeyValuePairDataType::Type::Int32:
			{
				int32 TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, static_cast<double>(TypedValue));
				break;
			}
			case EOnlineKeyValuePairDataType::Type::UInt32:
			{
				uint32 TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, static_cast<double>(TypedValue));
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Int64:
			{
				int64 TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, static_cast<double>(TypedValue));
				break;
			}
			case EOnlineKeyValuePairDataType::Type::UInt64:
			{
				uint64 TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, static_cast<double>(TypedValue));
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Float:
			{
				float TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, static_cast<double>(TypedValue));
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Double:
			{
				double TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetNumberField(FieldName, TypedValue);
				break;
			}
			case EOnlineKeyValuePairDataType::Type::String:
			{
				FString TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetStringField(FieldName, TypedValue);
				break;
			}
			case EOnlineKeyValuePairDataType::Type::Bool:
			{
				bool TypedValue;
				Pair.Value.GetValue(TypedValue);
				JsonObject->SetBoolField(FieldName, TypedValue);
				break;
			}
			default:
			{
				UE_LOG_ONLINE_EVENTS(Warning, TEXT("FOnlineEventsGDK::TriggerEvent: [%s] Unsupported argument type %d for field %s."), EventName, ValueType, *FieldName);
				return false;
			}
		}
	}

	FString DimensionsJson;
	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&DimensionsJson);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XblEventsWriteInGameEvent calls XGameEventWrite which is not safe to call on time-sensitive threads
	HRESULT Result = XblEventsWriteInGameEvent(GDKContext, TCHAR_TO_UTF8(EventName), TCHAR_TO_UTF8(*DimensionsJson), "{}");
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_EVENTS(VeryVerbose, TEXT("FOnlineEventsGDK::TriggerEvent: XblEventsWriteInGameEvent Failed for event: %s"), *FString(EventName));
		return false;
	}
	
	UE_LOG_ONLINE_EVENTS(VeryVerbose, TEXT("FOnlineEventsGDK::TriggerEvent: XblEventsWriteInGameEvent Succeeded for event: %s"), *FString(EventName));
	return true;
}

void FOnlineEventsGDK::SetPlayerSessionId(const FUniqueNetId& PlayerId, const FGuid& PlayerSessionId)
{
	PlayerSessionIds.Add(PlayerId.AsShared(), PlayerSessionId);
}
#endif //WITH_GRDK
