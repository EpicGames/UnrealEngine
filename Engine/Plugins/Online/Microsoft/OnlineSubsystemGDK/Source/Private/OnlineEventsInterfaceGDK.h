// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGDKPackage.h"
#include "SimpleTokenParser.h"
#include "Interfaces/OnlineEventsInterface.h"
#include "OnlineSubsystemGDKTypes.h"

class FOnlineSubsystemGDK;

/**
 *	FOnlineEventsGDK - Interface class for events (GDK implementation)
 */
class FOnlineEventsGDK : public IOnlineEvents
{
public:
	FOnlineEventsGDK(FOnlineSubsystemGDK* InSubsystem);
	virtual ~FOnlineEventsGDK() = default;

	/**
	 * Calls an event
	 *
	 * @param PlayerId	- Id of the player to call the event on
	 * @param Name		- Name of requested event
	 * @param Parms		- Parameters that will be passed directly to the event
	 */
	virtual bool TriggerEvent( const FUniqueNetId& PlayerId, const TCHAR* EventName, const FOnlineEventParms& Parms ) override;

	virtual void SetPlayerSessionId(const FUniqueNetId& PlayerId, const FGuid& PlayerSessionId) override;

private:
	/** The owning subsystem */
	FOnlineSubsystemGDK*				Subsystem;

	TUniqueNetIdMap<FGuid>		PlayerSessionIds;
};

typedef TSharedPtr<FOnlineEventsGDK, ESPMode::ThreadSafe> FOnlineEventsGDKPtr;
