// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineIdentityErrors.h"
#include "OnlineAsyncTaskManagerGDK.h"
#if WITH_ENGINE && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#endif //WITH_ENGINE && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "GDKThreadCheck.h"

#include "AsyncTasks/OnlineAsyncTaskGDKShowLoginUI.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetActivitiesForUsers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKShowSendGameInvitesUI.h"
#include "AsyncTasks/OnlineAsyncTaskGDKShowAchievementsUI.h"
#include "AsyncTasks/OnlineAsyncTaskGDKShowStoreUI.h"
#include "AsyncTasks/OnlineAsyncTaskGDKShowProfileUI.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaSendInvitesWithUI.h"


extern TAutoConsoleVariable<bool> CVarXboxMpaEnabled;


const int32 PEOPLE_PICKER_MAX_SIZE = 100;
#define INVITE_UI_TEXT TEXT("Invite players")

// These need to be their own asyc tasks
bool FOnlineExternalUIGDK::ShowLoginUI(const int32 ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKShowLoginUI>(GDKSubsystem, bAllowGuestLogin, FOnQueryGDKShowLoginUICompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleShowLoginUIComplete, Delegate));

	return true;
}

void FOnlineExternalUIGDK::HandleShowLoginUIComplete(bool bSuccess, HRESULT hResult, FGDKUserHandle GDKUser, FOnLoginUIClosedDelegate Delegate)
{
	GDKSubsystem->ExecuteNextTick([this, hResult, GDKUser, Delegate]()
	{
		// We want to wait 1 extra tick so that the input system has definitely be called before we tell the game to check
		// the user's state.  Without the ExecuteNextTick, this can happen before the input system registers the user/controllers
		GDKSubsystem->CreateAndDispatchAsyncEvent<FAsyncEventAccountPickerClosed>(GDKSubsystem, hResult, GDKUser, Delegate);
	});
}


bool FOnlineExternalUIGDK::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIGDK::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	FGDKUserHandle GDKUser = Identity->GetUserForPlatformUserId(LocalUserNum);

	if (!GDKUser)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Couldn't find GDK user for LocalUserNum %d."), LocalUserNum);
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Couldn't find GDK context for LocalUserNum %d."), LocalUserNum);
		return false;
	}

	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKMpaSendInvitesWithUI>(GDKSubsystem, GDKContext, GDKUser);
		return true;
	}

	TArray<uint64> UserArray;
	uint64 GDKUserId;
	if (FAILED(XUserGetId(GDKUser, &GDKUserId)))
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Couldn't find XUID for LocalUserNum %d."), LocalUserNum);
		return false;
	}

	UserArray.Add(GDKUserId);

	FOnlineSessionGDKPtr GDKSession = StaticCastSharedPtr<FOnlineSessionGDK>(GDKSubsystem->GetSessionInterface());
	FNamedOnlineSessionPtr Session = GDKSession->GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Named session not found for %s session name. Can't send invite."), *SessionName.ToString());
		return false;
	}

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session->SessionInfo);
	if (!GDKInfo.IsValid())
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: FOnlineSessionInfoMpsdGDK not valid for %s. Can't send invite."), *SessionName.ToString());
		return false;
	}

	FGDKMultiplayerSessionHandle GDKSessionHandle  = GDKInfo->GetGDKMultiplayerSession();
	if (!GDKSessionHandle.IsValid())
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: GDKSessionHandle not valid for %s. Can't send invite."), *SessionName.ToString());
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGetActivitiesForUsers>(GDKSubsystem, GDKContext, UserArray, FOnGetActivitiesForUsersCompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleGetActivitiesForUsersComplete, GDKSessionHandle, GDKContext));
	return true;
}
	
void FOnlineExternalUIGDK::HandleGetActivitiesForUsersComplete(const TArray<XblMultiplayerActivityDetails>& ActivityDetails, bool bIsSuccess, FGDKMultiplayerSessionHandle InGDKSession, FGDKContextHandle GDKContext)
{
	if (bIsSuccess && ActivityDetails.Num() > 0)
	{
		bool bFoundSessionMatch = false;
		for (const XblMultiplayerActivityDetails& CurActivityDetail : ActivityDetails)
		{
			const XblMultiplayerSessionReference* GDKSessionReference = XblMultiplayerSessionSessionReference(InGDKSession);
			const XblMultiplayerSessionReference& CurrentSessionRef = CurActivityDetail.SessionReference;
			if (FString(GDKSessionReference->SessionName).Compare(FString(CurrentSessionRef.SessionName), ESearchCase::IgnoreCase) == 0)
			{
				bFoundSessionMatch = true;
				GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKShowSendGameInvitesUI>(GDKSubsystem, GDKContext, InGDKSession, FOnQueryGDKShowSendGameInvitesUICompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleShowSendGameInvitesUIComplete));
				break;
			}	
		}
		if (!bFoundSessionMatch)
		{
			UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: Could not find matching session. Can't send invite."));
			HandleShowSendGameInvitesUIComplete(false);
		}
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowInviteUI: User is not in a session. Can't send invite."));
		HandleShowSendGameInvitesUIComplete(false);
	}
}

void FOnlineExternalUIGDK::HandleShowSendGameInvitesUIComplete(bool bIsSuccess)
{
	GDKSubsystem->ExecuteNextTick([this, bIsSuccess]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_HandleShowSendGameInvitesUIComplete_Delegate);
		TriggerOnExternalUIChangeDelegates(bIsSuccess);
	});
}

bool FOnlineExternalUIGDK::ShowAchievementsUI(int32 LocalUserNum)
{
	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	FGDKUserHandle GDKUser = Identity->GetUserForPlatformUserId(LocalUserNum);

	if (!GDKUser)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowAchievementsUI: Couldn't find Live user for LocalUserNum %d."), LocalUserNum);
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowAchievementsUI: Couldn't find GDK context for LocalUserNum %d."), LocalUserNum);
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKShowAchievementsUI>(GDKSubsystem, GDKContext, GDKUser, FOnQueryGDKShowAchievementsUICompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleShowAchievementsUIComplete));
	return true;
}

void FOnlineExternalUIGDK::HandleShowAchievementsUIComplete(bool bIsSuccess)
{
	if (bIsSuccess)
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("ShowAchievementsUI: Achievement task UI displaying."));
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("ShowAchievementsUI: Achievement task UI failed. Not displaying."));
	}
}

bool FOnlineExternalUIGDK::ShowLeaderboardUI( const FString& LeaderboardName )
{
	return false;
}

bool FOnlineExternalUIGDK::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	if (ShowWebUrlRequest.WaitForProtocolActivationTimeRemaining > 0.0f)
	{
		// If we had one outstanding, consider it cancelled
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("FOnlineExternalUILive::ShowWebURL: Abandoning wait for previous ShowWebUrl protocol activation"));
		FinishShowWebUrl(MoveTemp(ShowWebUrlRequest.WebUrlBeingOpened));
	}
	ShowWebUrlRequest.WebUrlBeingOpened = Url;
	ShowWebUrlRequest.WebUrlClosedDelegate = Delegate;

	FCoreDelegates::ApplicationHasReactivatedDelegate.AddThreadSafeSP(this, &FOnlineExternalUIGDK::HandleApplicationHasReactivated_WebUrl);

	// No async launch URI, do we need to run this in its own thread?
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XLaunchUri is not safe to be called on time-sensitive threads
	HRESULT Result = XLaunchUri(nullptr, TCHAR_TO_UTF8(*Url));
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowWebURL: URL launch result error: 0x%0.8x"), Result);
	}
	else
	{
		UE_LOG_ONLINE_EXTERNALUI(Log, TEXT("ShowWebURL: URL launch completed, success: %d"), Result);
	}

	return true;
}

void FOnlineExternalUIGDK::HandleApplicationHasReactivated_WebUrl()
{
	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
	if (!ShowWebUrlRequest.WebUrlBeingOpened.IsEmpty())
	{
		// Wait for a protocol activation before triggering the complete delegate
		// Not guaranteed to come, will only come if the web browser reactivates the application with a specific uri
		ShowWebUrlRequest.WaitForProtocolActivationTimeRemaining = 3.0f;
	}
}

FOnlineExternalUIGDK::FOnlineExternalUIGDK(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
	, bShouldCallUIDelegate(false)
	, bAllowGuestLogin(true)
{
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bAllowGuestLogin"), bAllowGuestLogin, GEngineIni);
}

bool FOnlineExternalUIGDK::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIGDK::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIGDK::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	FGDKUserHandle GDKUser = Identity->GetUserForPlatformUserId(LocalUserNum);
	if (!GDKUser)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowStoreUI: Couldn't find GDK user for LocalUserNum %d."), LocalUserNum);
		GDKSubsystem->ExecuteNextTick([this, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_ShowStoreUI_Delegate);
			Delegate.ExecuteIfBound(false);
		});
		return false;
	}
	
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("ShowStoreUI: Couldn't find GDK context for LocalUserNum %d."), LocalUserNum);
		GDKSubsystem->ExecuteNextTick([this, Delegate]()
		{
			Delegate.ExecuteIfBound(false);
		});
		return false;
	}	

	StoreUIClosedDelegate = Delegate;
	bShouldCallUIDelegate = true;


	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKShowStoreUI>(GDKSubsystem, GDKContext, ShowParams, FOnQueryGDKShowStoreUICompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleShowStoreUIComplete));
	return true;
}

void FOnlineExternalUIGDK::HandleShowStoreUIComplete(bool wasPurchaseMade)
{
	check(IsInGameThread());
	// if we received a purchase callback that means we must have bought something
	if (bShouldCallUIDelegate)
	{
		GDKSubsystem->CreateAndDispatchAsyncEvent<FAsyncEventStoreUIClosed>(GDKSubsystem, StoreUIClosedDelegate, wasPurchaseMade);
		bShouldCallUIDelegate = false;
	}
}

bool FOnlineExternalUIGDK::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIGDK::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	if(!GDKSubsystem || !GDKSubsystem->GetIdentityGDK().IsValid())
	{
		return false;
	}

	FGDKUserHandle RequestingUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(*FUniqueNetIdGDK::Cast(Requestor));
	const FUniqueNetIdGDKRef GDKRequestee = FUniqueNetIdGDK::Cast(Requestee);
	if (!ensure(RequestingUser.IsValid()))
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("Passed in invalider requester to FOnlineExternalUIGDK::ShowProfileUI"));
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(RequestingUser);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_EXTERNALUI(Warning, TEXT("Cannot find GDK context for FOnlineExternalUIGDK::ShowProfileUI"));
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKShowProfileUI>(GDKSubsystem, GDKContext, RequestingUser, GDKRequestee->ToUint64(), FOnQueryGDKShowProfileUICompleteDelegate::CreateThreadSafeSP(this, &FOnlineExternalUIGDK::HandleShowProfileUIComplete, Delegate));
	return true;
}

void FOnlineExternalUIGDK::HandleShowProfileUIComplete(bool bSuccess, const FOnProfileUIClosedDelegate Delegate)
{
	if (GDKSubsystem)
	{
		GDKSubsystem->CreateAndDispatchAsyncEvent<FAsyncEventProfileCardClosed>(GDKSubsystem, Delegate);
	}
}

FString FOnlineExternalUIGDK::FAsyncEventAccountPickerClosed::ToString() const
{
	return TEXT("Account picker closed.");
}

void FOnlineExternalUIGDK::FAsyncEventAccountPickerClosed::TriggerDelegates()
{
	FOnlineAsyncEvent::TriggerDelegates();

	if (SignedInUser)
	{
		FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Create(SignedInUser);
		int32 ControllerIndex = INDEX_NONE;

		const FOnlineIdentityGDKPtr Identity = Subsystem->GetIdentityGDK();
		if (Identity.IsValid())
		{
			ControllerIndex = Identity->GetPlatformUserIdFromGDKUser(SignedInUser);
			check(ControllerIndex != -1);
		}
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_FAsyncEventAccountPickerClosed_TriggerDelegates_SignedIn);
		Delegate.ExecuteIfBound(UniqueNetId, ControllerIndex, FOnlineError::Success());
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_FAsyncEventAccountPickerClosed_TriggerDelegates_NotSignedIn);
		Delegate.ExecuteIfBound(nullptr, INDEX_NONE, hResult == E_ABORT ? OnlineIdentity::Errors::Canceled() : OnlineIdentity::Errors::RequestFailure());
	}
}

FString FOnlineExternalUIGDK::FAsyncEventProfileCardClosed::ToString() const 
{
	return TEXT("Profile card closed.");
}

void FOnlineExternalUIGDK::FAsyncEventProfileCardClosed::TriggerDelegates()
{
	FOnlineAsyncEvent::TriggerDelegates();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_FAsyncEventProfileCardClosed_TriggerDelegates);
	Delegate.ExecuteIfBound();
}

/**
 * Console commands for testing and debugging
 */
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static void	TestProfileCard( const TArray<FString>& Args, UWorld* InWorld )
{
#if WITH_ENGINE
	if(!InWorld || Args.Num() < 2)
	{
		return;
	}

	IOnlineSubsystem* OnlineSub = IOnlineSubsystem::Get();
	IOnlineIdentityPtr Identity = OnlineSub ? OnlineSub->GetIdentityInterface() : nullptr;
	IOnlineExternalUIPtr ExternalUI = OnlineSub ? OnlineSub->GetExternalUIInterface() : nullptr;

	FUniqueNetIdPtr Requestor;
	FUniqueNetIdPtr Requestee;
	for( auto Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		APlayerController* PlayerController = Iterator->Get();

		if( PlayerController )
		{
			ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player);
			if( LocalPlayer && Identity.IsValid() )
			{
				const int32 ControllerId = LocalPlayer->GetControllerId();
				if(ControllerId == FCString::Atoi(*Args[0]))
				{
					Requestor = Identity->GetUniquePlayerId(ControllerId);
				}

				if(ControllerId == FCString::Atoi(*Args[1]))
				{
					Requestee = Identity->GetUniquePlayerId(ControllerId);
				}
			}
		}
	}

	if(ExternalUI.IsValid() && Requestor.IsValid() && Requestee.IsValid())
	{
		ExternalUI->ShowProfileUI(*Requestor, *Requestee);
	}
#endif //WITH_ENGINE
}

FAutoConsoleCommandWithWorldAndArgs TestProfileCardCommand(
	TEXT("net.TestExternalProfileUI"), 
	TEXT( "Calls IOnlineExternalUI::ShowProfileUI. First parameter is the index of the requestor, second parameter is the index of the requestee." ), 
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(TestProfileCard)
	);
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

FString FOnlineExternalUIGDK::FAsyncEventStoreUIClosed::ToString() const
{
	return TEXT("Store UI Closed");
}

void FOnlineExternalUIGDK::FAsyncEventStoreUIClosed::TriggerDelegates()
{
	FOnlineAsyncEvent::TriggerDelegates();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_FAsyncEventStoreUIClosed_TriggerDelegates);
	Delegate.ExecuteIfBound(bPurchasedProduct);
}

FString FOnlineExternalUIGDK::FAsyncEventWebUrlUIClosed::ToString() const
{
	return TEXT("WebURL closed");
}

void FOnlineExternalUIGDK::FAsyncEventWebUrlUIClosed::TriggerDelegates()
{
	FOnlineAsyncEvent::TriggerDelegates();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineExternalUIGDK_FAsyncEventWebUrlUIClosed_TriggerDelegates);
	Delegate.ExecuteIfBound(WebUrl);
}

void FOnlineExternalUIGDK::Tick(float DeltaTime)
{
	if (ShowWebUrlRequest.WaitForProtocolActivationTimeRemaining > 0.0f)
	{
		ShowWebUrlRequest.WaitForProtocolActivationTimeRemaining -= DeltaTime;
		if (ShowWebUrlRequest.WaitForProtocolActivationTimeRemaining <= 0.0f)
		{
			FinishShowWebUrl(MoveTemp(ShowWebUrlRequest.WebUrlBeingOpened));
		}
	}
}

void FOnlineExternalUIGDK::FinishShowWebUrl(FString&& FinalUrl)
{
	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);

	FAsyncEventWebUrlUIClosed* NewEvent = new FAsyncEventWebUrlUIClosed(GDKSubsystem, MoveTemp(ShowWebUrlRequest.WebUrlClosedDelegate), MoveTemp(FinalUrl));
	ShowWebUrlRequest.Reset();
	GDKSubsystem->GetAsyncTaskManager()->AddToOutQueue(NewEvent);
}

#endif //WITH_GRDK