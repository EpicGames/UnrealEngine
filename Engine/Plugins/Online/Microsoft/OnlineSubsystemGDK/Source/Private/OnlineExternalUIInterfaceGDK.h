// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGDK.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDKPackage.h"

/**
* Delegate fired when activities for user has been queried
*/
DECLARE_DELEGATE_TwoParams(FOnGetActivitiesForUsersCompleteDelegate, const TArray<XblMultiplayerActivityDetails>&, bool);

/**
* Delegate fired when invite UI has been requested
*/
DECLARE_DELEGATE_OneParam(FOnQueryGDKShowSendGameInvitesUICompleteDelegate, bool);

/**
* Delegate fired when achievement UI has been requested
*/
DECLARE_DELEGATE_OneParam(FOnQueryGDKShowAchievementsUICompleteDelegate, bool);

/**
* Delegate fired when profile UI has been requested
*/
DECLARE_DELEGATE_OneParam(FOnQueryGDKShowProfileUICompleteDelegate, bool);

/**
* Delegate fired when store UI has been requested
*/
DECLARE_DELEGATE_OneParam(FOnQueryGDKShowStoreUICompleteDelegate, bool);

/**
* Delegate fired when login UI has been requested
*/
DECLARE_DELEGATE_ThreeParams(FOnQueryGDKShowLoginUICompleteDelegate, bool, HRESULT, FGDKUserHandle);

/**
* Delegate fired when Privilege has been requested, but GDK needs to resolve something with UI
*/
DECLARE_DELEGATE_OneParam(FOnQueryGDKResolvePrivilegeWithUICompleteDelegate, bool);

/** 
 * Implementation for the Live external UIs
 */
class FOnlineExternalUIGDK
	: public IOnlineExternalUI
	, public TSharedFromThis<FOnlineExternalUIGDK, ESPMode::ThreadSafe>
{
private:
	/**
	 *	Async event that notifies when the GDK account picker has been closed
	 */
	class FAsyncEventAccountPickerClosed : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
	{
		/** Hidden on purpose */
		FAsyncEventAccountPickerClosed() :
			FOnlineAsyncEvent(NULL),
			SignedInUser(nullptr)
		{
		}

		/** The result of picking user, to decide if the failure was because user cancelled it or something else */
		HRESULT hResult;

		/** The user that signed in through the account picker. Will be null if no user signed in. */
		FGDKUserHandle SignedInUser;

		/** The delegate to execute when the account picker is closed. */
		FOnLoginUIClosedDelegate Delegate;

	public:

		/**
		 * Constructor.
		 *
		 * @param InGDKSubsystem The owner of the external UI interface that triggered this event.
		 * @param InUser The user that signed in through the account picker, if any. Can be null.
		 * @param InControllerIndex The controller that was used to sign in.
		 * @param InDelegate The delegate to execute on the game thread.
		 */
		FAsyncEventAccountPickerClosed(FOnlineSubsystemGDK* InGDKSubsystem, HRESULT hInResult, FGDKUserHandle InUser, const FOnLoginUIClosedDelegate& InDelegate) :
			FOnlineAsyncEvent(InGDKSubsystem),
			hResult(hInResult),
			SignedInUser(InUser),
			Delegate(InDelegate)
		{
		}

		virtual FString ToString() const override;
		virtual void TriggerDelegates() override;
	};

	/**
	 *	Async event that notifies when the profile card has been closed
	 */
	class FAsyncEventProfileCardClosed : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
	{
		/** Hidden on purpose */
		FAsyncEventProfileCardClosed() :
			FOnlineAsyncEvent(NULL)
		{
		}
		
		/** The delegate to execute when the account picker is closed. */
		FOnProfileUIClosedDelegate Delegate;

	public:

		/**
		 * Constructor.
		 *
		 * @param InGDKSubsystem The owner of the external UI interface that triggered this event.
		 * @param InDelegate The delegate to execute on the game thread.
		 */
		FAsyncEventProfileCardClosed(FOnlineSubsystemGDK* InGDKSubsystem, const FOnProfileUIClosedDelegate& InDelegate) :
			FOnlineAsyncEvent(InGDKSubsystem),
			Delegate(InDelegate)
		{
		}

		virtual FString ToString() const override;
		virtual void TriggerDelegates() override;
	};

	/**
	*	Async event that notifies when the store UI has been closed
	*/
	class FAsyncEventStoreUIClosed : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
	{
		/** Hidden on purpose */
		FAsyncEventStoreUIClosed() :
			FOnlineAsyncEvent(NULL)
		{
		}

		/** The delegate to execute when the store UI is closed. */
		FOnShowStoreUIClosedDelegate Delegate;

		/** Whether a purchase was made */
		bool bPurchasedProduct;

	public:

		/**
		* Constructor.
		*
		* @param InGDKSubsystem The owner of the external UI interface that triggered this event.
		* @param InDelegate The delegate to execute on the game thread.
		* @param PurchasedProduct A flag determining whether a purchase was made.
		*/
		FAsyncEventStoreUIClosed(FOnlineSubsystemGDK* InGDKSubsystem, const FOnShowStoreUIClosedDelegate& InDelegate, bool PurchasedProduct) :
			FOnlineAsyncEvent(InGDKSubsystem),
			Delegate(InDelegate),
			bPurchasedProduct(PurchasedProduct)
		{
		}

		virtual FString ToString() const override;
		virtual void TriggerDelegates() override;
	};

	void HandleApplicationHasReactivated_WebUrl();

	/**
	*	Async event that notifies when the Web Url has closed
	*/
	class FAsyncEventWebUrlUIClosed : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
	{
		/** Hidden on purpose */
		FAsyncEventWebUrlUIClosed() :
			FOnlineAsyncEvent(NULL)
		{
		}

		/** The delegate to execute when the WebUrl UI is closed. */
		FOnShowWebUrlClosedDelegate Delegate;

		FString WebUrl;

	public:

		/**
		 * Constructor.
		 *
		 * @param InGDKSubsystem The owner of the external UI interface that triggered this event.
		 * @param InDelegate The delegate to execute on the game thread.
		 * @param InWebUrl The URL that was opened
		 */
		FAsyncEventWebUrlUIClosed(FOnlineSubsystemGDK* InGDKSubsystem, FOnShowWebUrlClosedDelegate&& InDelegate, FString&& InWebUrl) :
			FOnlineAsyncEvent(InGDKSubsystem),
			Delegate(MoveTemp(InDelegate)),
			WebUrl(MoveTemp(InWebUrl))
		{
		}

		virtual FString ToString() const override;
		virtual void TriggerDelegates() override;
	};

	FCriticalSection CallbackLock;

PACKAGE_SCOPE:

	/** Constructor
	 *
	 * @param InSubsystem The owner of this external UI interface.
	 */
	explicit FOnlineExternalUIGDK(FOnlineSubsystemGDK* InSubsystem);

	/** Reference to the owning subsystem */
	class FOnlineSubsystemGDK* GDKSubsystem;

	/** delegates to hold onto for particular callbacks */
	bool bShouldCallUIDelegate;
	FOnShowStoreUIClosedDelegate StoreUIClosedDelegate;
	
	struct FShowWebUrlRequest
	{
		FShowWebUrlRequest()
		{
			Reset();
		}
		void Reset()
		{
			WebUrlBeingOpened.Empty();
			WebUrlClosedDelegate = FOnShowWebUrlClosedDelegate();
			WaitForProtocolActivationTimeRemaining = 0.0f;
		}
		FString WebUrlBeingOpened;
		FOnShowWebUrlClosedDelegate WebUrlClosedDelegate;
		float WaitForProtocolActivationTimeRemaining;
	};
	FShowWebUrlRequest ShowWebUrlRequest;

	void FinishShowWebUrl(FString&& FinalUrl);

	/* CVar to configure if guest accounts will appear in the account picker */
	bool bAllowGuestLogin;

public:

	/**
	 * Destructor.
	 */
	virtual ~FOnlineExternalUIGDK()
	{
	}

	// IOnlineExternalUI
	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) override { /** NYI */ return false; }
	virtual bool ShowFriendsUI(int32 LocalUserNum) override;
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName = NAME_GameSession) override;
	virtual bool ShowAchievementsUI(int32 LocalUserNum) override;
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) override;
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) override;
	virtual bool CloseWebURL() override;
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) override;
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) override;
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) override;
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;
	
	void HandleShowLoginUIComplete(bool bSuccess, HRESULT hResult, FGDKUserHandle GDKUser, FOnLoginUIClosedDelegate Delegate);
	void HandleGetActivitiesForUsersComplete(const TArray<XblMultiplayerActivityDetails>& ActivityDetails, bool bIsSuccess, FGDKMultiplayerSessionHandle InGDKSession, FGDKContextHandle GDKContext);
	void HandleShowSendGameInvitesUIComplete(bool bIsSuccess);
	void HandleShowAchievementsUIComplete(bool bIsSuccess);
	void HandleShowStoreUIComplete(bool wasPurchaseMade);
	void HandleShowProfileUIComplete(bool bSuccess, const FOnProfileUIClosedDelegate Delegate);

	void Tick(float DeltaTime);
};

typedef TSharedPtr<FOnlineExternalUIGDK, ESPMode::ThreadSafe> FOnlineExternalUIGDKPtr;
