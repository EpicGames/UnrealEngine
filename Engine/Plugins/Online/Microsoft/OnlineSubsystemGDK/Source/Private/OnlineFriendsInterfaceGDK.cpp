// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineFriendsInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSessionGDK.h"
#include "Online/OnlineSessionNames.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaUpdateRecentPlayers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryFriends.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryAvoidList.h"

FOnlineFriendGDK::FOnlineFriendGDK(const XblSocialRelationship& InSocialRelationship)
	: SocialRelationship(InSocialRelationship)
	, UniqueNetIdGDK(FUniqueNetIdGDK::Create(InSocialRelationship.xboxUserId))
{
	// We don't set DisplayName here since we don't know it yet; it gets set later
}

EInviteStatus::Type FOnlineFriendGDK::GetInviteStatus() const
{
	// Friends on GDK are single-directional, meaning there is no approval step
	return EInviteStatus::Accepted;
}

const FOnlineUserPresence& FOnlineFriendGDK::GetPresence() const
{
	return Presence;
}

FUniqueNetIdRef FOnlineFriendGDK::GetUserId() const
{
	return UniqueNetIdGDK;
}

FString FOnlineFriendGDK::GetRealName() const
{
	return DisplayName;
}

FString FOnlineFriendGDK::GetDisplayName(const FString& Platform /*= FString()*/) const
{
	return DisplayName;
}

bool FOnlineFriendGDK::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* FoundAttribute = UserAttributes.Find(AttrName);
	if (FoundAttribute != nullptr)
	{
		OutAttrValue = *FoundAttribute;
	}

	return FoundAttribute != nullptr;
}

bool FOnlineFriendGDK::IsFavorite() const
{
	return SocialRelationship.isFavorite;
}

FOnlineBlockedPlayerGDK::FOnlineBlockedPlayerGDK(uint64 InXUID)
	: UniqueNetIdGDK(FUniqueNetIdGDK::Create(InXUID))
{
}

FUniqueNetIdRef FOnlineBlockedPlayerGDK::GetUserId() const
{
	return UniqueNetIdGDK;
}

FString FOnlineBlockedPlayerGDK::GetRealName() const
{
	// We don't currently query DisplayNames for avoided players
	return TEXT("BlockedPlayer");
}

FString FOnlineBlockedPlayerGDK::GetDisplayName(const FString& Platform /*= FString()*/) const
{
	// We don't currently query DisplayNames for avoided players
	return TEXT("BlockedPlayer");
}

bool FOnlineBlockedPlayerGDK::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	OutAttrValue.Empty();
	return false;
}

bool FOnlineFriendsGDK::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate /*= FOnReadFriendsListComplete()*/)
{
	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!UserGDKContext)
	{
		constexpr bool bWasSuccessful = false;
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsLive::ReadFriendsList failed, could not find user at index %d"), LocalUserNum);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_ReadFriendsList_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, bWasSuccessful, ListName, FString::Printf(TEXT("Could not find user at index %d"), LocalUserNum));
		return false;
	}

	uint64 userXuid;
	XblContextGetXboxUserId(UserGDKContext, &userXuid);
	FUniqueNetIdGDKRef UserNetId = FUniqueNetIdGDK::Create(userXuid);

	// Add our delegate to our global friends read for this user, and then check if we have any other requests in progress
	TArray<FOnReadFriendsListComplete>& QueuedDelegates = FriendsListInProgressDelegates.FindOrAdd(UserNetId);
	QueuedDelegates.Add(Delegate);
	if (QueuedDelegates.Num() > 1)
	{
		// We have another request in progress, so we're done here (other request will call our delegate)
		// We don't really care about the ListName being different, as we don't support the listname properly in this OSS anyway
		return true;
	}

	UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsLive::ReadFriendsList Request started for user %d/%s for friendslist %s"), LocalUserNum, *UserNetId->ToString(), *ListName);
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryFriends>(GDKSubsystem, UserGDKContext, LocalUserNum, UserNetId, ListName);
	return true;
}

bool FOnlineFriendsGDK::DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate /*= FOnDeleteFriendsListComplete()*/)
{
	// Not Implemented
	GDKSubsystem->ExecuteNextTick([LocalUserNum, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsLive::DeleteFriendsList is not currently supported"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_DeleteFriendsList_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, false, ListName, TEXT("FOnlineFriendsLive::DeleteFriendsList is not currently supported"));
	});

	return false;
}

bool FOnlineFriendsGDK::SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate /*= FOnSendInviteComplete()*/) 
{
	FUniqueNetIdGDKRef GDKFriendId = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendId.AsShared());

	if (IsFriend(LocalUserNum, FriendId, ListName))
	{
		GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKFriendId, ListName, Delegate]()
		{
			UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsLive::SendInvite failed; %s are already a friend"), *GDKFriendId->ToString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_SendInvite_Delegate);
			Delegate.ExecuteIfBound(LocalUserNum, false, *GDKFriendId, ListName, TEXT("Selected user is already a friend"));
		});
		return false;
	}

	//Showing the user profile UI is the closest thing we can do to adding/removing friends in GDK
	return ShowProfileUI(LocalUserNum, FriendId, ListName, Delegate);
}

bool FOnlineFriendsGDK::ShowProfileUI(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate /*= FOnSendInviteComplete()*/) {
	
	FUniqueNetIdGDKRef GDKFriendId = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendId.AsShared());

	//First we check if the local user is valid and has a valid context
	FGDKContextHandle ContextHandle = GDKSubsystem->GetGDKContext(LocalUserNum);
	FGDKUserHandle UserHandle;
	XblContextGetUser(ContextHandle, UserHandle.GetInitReference());
	if (!ContextHandle.IsValid() || !UserHandle.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKFriendId, ListName, Delegate]()
		{
			const FString ErrorString(TEXT("Local user is not signed in"));
			UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::AddRemoveFriend failed; %s"), *ErrorString);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_ShowProfileUI_Delegate);
			Delegate.ExecuteIfBound(LocalUserNum, false, *GDKFriendId, ListName, ErrorString);
		});
		return false;
	}

	//Then we attempt to show the user profile UI
	bool StartedUITask = false;

	IOnlineExternalUIPtr ExternalUIInterface = GDKSubsystem->GetExternalUIInterface();
	if (ExternalUIInterface.IsValid())
	{
		uint64 UserId;
		if (SUCCEEDED(XblContextGetXboxUserId(ContextHandle, &UserId)))
		{
			StartedUITask = ExternalUIInterface->ShowProfileUI(
				*FUniqueNetIdGDK::Create(UserId),
				FriendId,
				FOnProfileUIClosedDelegate::CreateThreadSafeSP(this, &FOnlineFriendsGDK::HandleShowProfileUIComplete, LocalUserNum, GDKFriendId, ListName, Delegate));
		}
	}

	if (!StartedUITask)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::ShowProfileUI failed to start"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_ShowProfileUI_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, false, *GDKFriendId, ListName, TEXT("FOnlineFriendsGDK::ShowProfileUI failed to start"));
		return false;
	}

	return true;
}
	
void FOnlineFriendsGDK::HandleShowProfileUIComplete(int32 LocalUserNum, FUniqueNetIdGDKRef FriendId, const FString ListName, const FOnSendInviteComplete Delegate)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_HandleShowProfileUIComplete_Delegate);
	Delegate.ExecuteIfBound(LocalUserNum, true, *FriendId, ListName, FString());
}

bool FOnlineFriendsGDK::AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate /*= FOnAcceptInviteComplete()*/)
{
	// Not Implemented
	FUniqueNetIdGDKRef GDKFriendId = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendId.AsShared());
	GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKFriendId, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::AcceptInvite is currently not implemented"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_AcceptInvite_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, false, *GDKFriendId, ListName, TEXT("FOnlineFriendsGDK::AcceptInvite is currently not implemented"));
	});

	return false;
}

bool FOnlineFriendsGDK::RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	// Not Implemented
	FUniqueNetIdGDKRef GDKFriendId = StaticCastSharedRef<const FUniqueNetIdGDK>(FriendId.AsShared());
	GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKFriendId, ListName, this]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::RejectInvite is currently not implemented"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_RejectInvite_Delegate);
		TriggerOnRejectInviteCompleteDelegates(LocalUserNum, false, *GDKFriendId, ListName, TEXT("FOnlineFriendsGDK::RejectInvite is currently not implemented"));
	});

	return false;
}

void FOnlineFriendsGDK::SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate /*= FOnSetFriendAliasComplete()*/)
{
	// Not Implemented
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	GDKSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::SetFriendAlias is currently not implemented"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_SetFriendAlias_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

void FOnlineFriendsGDK::DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate)
{
	// Not Implemented
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	GDKSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::DeleteFriendAlias is currently not implemented"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_DeleteFriendAlias_Delegate);
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

bool FOnlineFriendsGDK::DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_DeleteFriend_Delegate);
	TriggerOnDeleteFriendCompleteDelegates(LocalUserNum, false, FriendId, ListName, TEXT("FOnlineFriendsGDK::DeleteFriend is currently not implemented"));

	return false;
}

bool FOnlineFriendsGDK::GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends)
{
	FUniqueNetIdGDKPtr GDKUserId = StaticCastSharedPtr<const FUniqueNetIdGDK>(GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(LocalUserNum));
	if (!GDKUserId.IsValid())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Get FriendsList failed, could not find a user for LocalUserNum %d"), LocalUserNum);
		OutFriends.Empty();
		return false;
	}

	const FOnlineFriendsListGDKMap* UserFriendsList = FriendsMap.Find(GDKUserId.ToSharedRef());
	if (UserFriendsList != nullptr)
	{
		OutFriends.Empty(UserFriendsList->Num());
		for (const FOnlineFriendsListGDKMap::ElementType& NetIdFriendPair : *UserFriendsList)
		{
			OutFriends.Emplace(NetIdFriendPair.Value);
		}
		return true;
	}
	else
	{
		OutFriends.Empty();
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Get FriendsList failed, could not find a friendslist for user %s; was this user's friendslist queried?"), *GDKUserId->ToString())
	}
	return false;
}

TSharedPtr<FOnlineFriend> FOnlineFriendsGDK::GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	FUniqueNetIdGDKPtr GDKUserId = StaticCastSharedPtr<const FUniqueNetIdGDK>(GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(LocalUserNum));
	if (!GDKUserId.IsValid())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Get Friend failed, could not find a user for LocalUserNum %d"), LocalUserNum);
		return nullptr;
	}

	const FOnlineFriendsListGDKMap* UserFriendsList = FriendsMap.Find(GDKUserId.ToSharedRef());
	if (UserFriendsList == nullptr)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Get Friend failed, could not find a friendslist for user %s; was this user's friendslist queried?"), *GDKUserId->ToString())
		return nullptr;
	}

	const TSharedRef<FOnlineFriendGDK>* FoundFriendPtr = UserFriendsList->Find(FriendId.AsShared());
	if (FoundFriendPtr == nullptr)
	{
		return nullptr;
	}

	return *FoundFriendPtr;
}

bool FOnlineFriendsGDK::IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	const FUniqueNetIdGDKPtr GDKUserId = StaticCastSharedPtr<const FUniqueNetIdGDK>(GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(LocalUserNum));
	if (!GDKUserId.IsValid())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("IsFriend failed, could not find a user for LocalUserNum %d"), LocalUserNum);
		return false;
	}

	const FOnlineFriendsListGDKMap* const UserFriendsList = FriendsMap.Find(GDKUserId.ToSharedRef());
	if (UserFriendsList == nullptr)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("IsFriend failed, could not find a friendslist for user %s; was this user's friendslist queried?"), *GDKUserId->ToString())
		return false;
	}

	return UserFriendsList->Contains(FriendId.AsShared());
}

void FOnlineFriendsGDK::AddRecentPlayers(const FUniqueNetId& LocalUserId, const TArray<FReportPlayedWithUser>& InRecentPlayers, const FString& ListName, const FOnAddRecentPlayersComplete& InCompletionDelegate)
{
	FOnlineSessionGDKPtr OnlineSessionGDK = GDKSubsystem->GetSessionInterfaceGDK();
	if (!OnlineSessionGDK->IsMpaEnabled())
	{
		InCompletionDelegate.ExecuteIfBound(LocalUserId, FOnlineError::Success());
		return;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserId);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Can't post recent players, invalid GDKContext of player %s"), *LocalUserId.ToDebugString());
		InCompletionDelegate.ExecuteIfBound(LocalUserId, FOnlineError(EOnlineErrorResult::Unknown));
		return;
	}

	FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::FOnComplete UpdateRecentPlayersCompleteDelegate = FOnlineAsyncTaskGDKMpaUpdateRecentPlayers::FOnComplete::CreateLambda([this, LocalUserIdRef = LocalUserId.AsShared(), InCompletionDelegate](bool bWasSuccessful)
	{
		InCompletionDelegate.ExecuteIfBound(*LocalUserIdRef, bWasSuccessful ? FOnlineError::Success() : FOnlineError(EOnlineErrorResult::Unknown));
	});

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKMpaUpdateRecentPlayers>(GDKSubsystem, GDKContext, InRecentPlayers, UpdateRecentPlayersCompleteDelegate);
}

bool FOnlineFriendsGDK::QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace)
{
	// Not Implemented
	FUniqueNetIdGDKRef GDKUserId = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId.AsShared());
	GDKSubsystem->ExecuteNextTick([GDKUserId, Namespace, this]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_QueryRecentPlayers_Delegate);
		TriggerOnQueryRecentPlayersCompleteDelegates(*GDKUserId, Namespace, false, TEXT("FOnlineFriendsGDK::QueryRecentPlayers is currently not implemented"));
	});
	return false;
}

bool FOnlineFriendsGDK::GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers)
{
	// Not Implemented
	UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::GetRecentPlayers is currently not implemented"));
	OutRecentPlayers.Empty();
	return false;
}

void FOnlineFriendsGDK::DumpRecentPlayers() const
{
	// Not Implemented
	UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::DumpRecentPlayers is currently not implemented"));
}

bool FOnlineFriendsGDK::BlockPlayer(int32 LocalUserNum, const FUniqueNetId& UserId)
{
	// Not supported by GDK
	FUniqueNetIdGDKRef GDKUserId = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId.AsShared());
	GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKUserId, this]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::BlockPlayer is not supported"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_BlockPlayer_Delegate);
		TriggerOnBlockedPlayerCompleteDelegates(LocalUserNum, false, *GDKUserId, FString(), TEXT("FOnlineFriendsGDK::BlockPlayer is not supported"));
	});

	return false;
}

bool FOnlineFriendsGDK::UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& UserId)
{
	// Not supported by GDK
	FUniqueNetIdGDKRef GDKUserId = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId.AsShared());
	GDKSubsystem->ExecuteNextTick([LocalUserNum, GDKUserId, this]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::UnblockPlayer is not supported"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_UnblockPlayer_Delegate);
		TriggerOnUnblockedPlayerCompleteDelegates(LocalUserNum, false, *GDKUserId, FString(), TEXT("FOnlineFriendsGDK::UnblockPlayer is not supported"));
	});

	return false;
}

bool FOnlineFriendsGDK::QueryBlockedPlayers(const FUniqueNetId& UserId)
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);

	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(UserId);
	if (!UserGDKContext)
	{
		GDKSubsystem->ExecuteNextTick([this, GDKUserId]()
		{
			constexpr bool bWasSuccessful = false;
			UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsGDK::UnblockPlayer Could not find user %s"), *GDKUserId->ToString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineFriendsGDK_QueryBlockedPlayers_Delegate);
			TriggerOnQueryBlockedPlayersCompleteDelegates(*GDKUserId, bWasSuccessful, FString::Printf(TEXT("Could not find user %s"), *GDKUserId->ToString()));
		});

		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryAvoidList>(GDKSubsystem, UserGDKContext, GDKUserId);
	return true;
}

bool FOnlineFriendsGDK::GetBlockedPlayers(const FUniqueNetId& UserId, TArray<TSharedRef<FOnlineBlockedPlayer>>& OutBlockedPlayers)
{
	TArray<TSharedRef<FOnlineBlockedPlayerGDK>>* UserAvoidList = AvoidListMap.Find(UserId.AsShared());
	if (UserAvoidList == nullptr)
	{
		OutBlockedPlayers.Empty();
		return false;
	}

	OutBlockedPlayers.Empty(UserAvoidList->Num());
	for (const TSharedRef<FOnlineBlockedPlayerGDK>& BlockedPlayerRef : *UserAvoidList)
	{
		OutBlockedPlayers.Emplace(BlockedPlayerRef);
	}

	return true;
}

void FOnlineFriendsGDK::DumpBlockedPlayers() const
{
	for (const auto& GDKUserIdBlockedPlayerPtrPair : AvoidListMap)
	{
		for (const TSharedRef<FOnlineBlockedPlayerGDK>& BlockedPlayer : GDKUserIdBlockedPlayerPtrPair.Value)
		{
			UE_LOG_ONLINE_FRIEND(Log, TEXT("User %s has player %s blocked"), *GDKUserIdBlockedPlayerPtrPair.Key->ToString(), *BlockedPlayer->UniqueNetIdGDK->ToDebugString());
		}
	}
}

void FOnlineFriendsGDK::OnUserPresenceUpdate(const FUniqueNetIdGDK& FriendId, const TSharedRef<FOnlineUserPresenceGDK>& UpdatedPresenceRef)
{
	FOnlineIdentityGDKPtr IdentityPtr = GDKSubsystem->GetIdentityGDK();
	FOnlinePresenceGDKPtr PresencePtr = GDKSubsystem->GetPresenceGDK();

	for (FOnlineUserFriendsListGDKMap::ElementType& UserToFriendListMap : FriendsMap)
	{
		// We are friends with this user if FriendRefPtr is valid
		TSharedRef<FOnlineFriendGDK>* FriendRefPtr = UserToFriendListMap.Value.Find(FriendId.AsShared());
		if (FriendRefPtr)
		{
			TSharedRef<FOnlineFriendGDK>& FriendRef = *FriendRefPtr;

			// Copy our presence
			FriendRef->Presence = *UpdatedPresenceRef;

			// Subscribe to stat updates if they're now online (and playing the same game), unsubscribe otherwise
			if (PresencePtr.IsValid())
			{
				if (UpdatedPresenceRef->bIsOnline && UpdatedPresenceRef->bIsPlayingThisGame)
				{
					PresencePtr->EstablishDefaultPresenceStatSubscriptions(StaticCastSharedRef<const FUniqueNetIdGDK>(UserToFriendListMap.Key),
						StaticCastSharedRef<const FUniqueNetIdGDK>(FriendRef->GetUserId()));
				}
				else
				{
					PresencePtr->ClearStatUpdateSubscriptionsForFriend(StaticCastSharedRef<const FUniqueNetIdGDK>(UserToFriendListMap.Key),
						StaticCastSharedRef<const FUniqueNetIdGDK>(FriendRef->GetUserId()));
				}
			}
		}
	}
}

void FOnlineFriendsGDK::OnUsersSessionPresenceUpdate(TArray<FOnlineSession>& OnlineSessions)
{
	for (const FOnlineSession& Session : OnlineSessions)
	{
		for (FOnlineUserFriendsListGDKMap::ElementType& UserToFriendListMap : FriendsMap)
		{
			if (TSharedRef<FOnlineFriendGDK>* FriendRefPtr = UserToFriendListMap.Value.Find(Session.OwningUserId.ToSharedRef()))
			{
				TSharedRef<FOnlineFriendGDK>& FriendRef = *FriendRefPtr;

				FString ConnectionString;
				Session.SessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

				// Update our presence
				FriendRef->Presence.SessionId = FUniqueNetIdString::Create(ConnectionString, GDK_SUBSYSTEM);
				FriendRef->Presence.bIsJoinable = !ConnectionString.IsEmpty();
			}
		}
	}
}

void FOnlineFriendsGDK::OnUsersSessionPresenceUpdate(const FOnlineGDKActivitiesResultMap& ActivitiesResults)
{
	for (const TPair<FUniqueNetIdRef, FUniqueNetIdPtr>& FriendSessionIdPair : ActivitiesResults)
	{
		for (FOnlineUserFriendsListGDKMap::ElementType& UserToFriendListMap : FriendsMap)
		{
			if (TSharedRef<FOnlineFriendGDK>* FriendRefPtr = UserToFriendListMap.Value.Find(FriendSessionIdPair.Key))
			{
				TSharedRef<FOnlineFriendGDK>& FriendRef = *FriendRefPtr;

				// Update our presence
				FriendRef->Presence.SessionId = FriendSessionIdPair.Value;
				FriendRef->Presence.bIsJoinable = FriendSessionIdPair.Value.IsValid();
			}
		}
	}
}

#endif //WITH_GRDK