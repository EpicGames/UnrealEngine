// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystemGDKPackage.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/social_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class FOnlineSubsystemGDK;
class FOnlineAsyncTaskGDKQueryFriends;
class FOnlineAsyncTaskGDKQueryFriendManagerTask;
class FOnlineAsyncTaskGDKQueryAvoidList;

using FOnlineFriendsListGDKMap = TUniqueNetIdMap<TSharedRef<class FOnlineFriendGDK>>;
using FOnlineUserFriendsListGDKMap = TUniqueNetIdMap<FOnlineFriendsListGDKMap>;
using FOnlineGDKActivitiesResultMap = TUniqueNetIdMap<FUniqueNetIdPtr /*JoinableSessionId*/>;

/**
 * Info associated with an online friend on the GDK service
 */
class FOnlineFriendGDK :
	public FOnlineFriend
{
public:
	// FOnlineFriendGDK
	FOnlineFriendGDK(const XblSocialRelationship& InSocialRelationship);
	virtual ~FOnlineFriendGDK() = default;

	// FOnlineFriend
	virtual EInviteStatus::Type GetInviteStatus() const override;
	virtual const FOnlineUserPresence& GetPresence() const override;

	// FOnlineUser
	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	/** Helper to tell if this person has been been Favourited by the user; Favourite users should come first in a friendslist */
	bool IsFavorite() const;

PACKAGE_SCOPE:
	/** The friend profile data */
	const XblSocialRelationship SocialRelationship;

	/** Unique Live Id for the friend */
	FUniqueNetIdGDKRef UniqueNetIdGDK;

	/** Presence info  */
	FOnlineUserPresenceGDK Presence;

	/** The Name XBox Live tells us to call this user (May be GamerTag, may be Real Name)*/
	FString DisplayName;

	/** Custom attributes store on this user */
	TMap<FString, FString> UserAttributes;
};

class FOnlineBlockedPlayerGDK :
	public FOnlineBlockedPlayer
{
public:
	// FOnlineFriendGDK
	FOnlineBlockedPlayerGDK(uint64 InXUID);
	virtual ~FOnlineBlockedPlayerGDK() = default;

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

PACKAGE_SCOPE:
	FUniqueNetIdGDKRef UniqueNetIdGDK;
};

/**
 * Implements the GDK specific interface for friends
 */
class FOnlineFriendsGDK :
	public IOnlineFriends, public TSharedFromThis<FOnlineFriendsGDK, ESPMode::ThreadSafe>
{
	/** The async task classes require friendship */
	friend FOnlineAsyncTaskGDKQueryFriends;
	friend FOnlineAsyncTaskGDKQueryFriendManagerTask;
	friend FOnlineAsyncTaskGDKQueryAvoidList;

public:
	// IOnlineFriends
	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName,  const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override;
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void AddRecentPlayers(const FUniqueNetId& LocalUserId, const TArray<FReportPlayedWithUser>& InRecentPlayers, const FString& ListName, const FOnAddRecentPlayersComplete& InCompletionDelegate) override;
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual void DumpRecentPlayers() const override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;

	// FOnlineFriendsGDK
	explicit FOnlineFriendsGDK(class FOnlineSubsystemGDK* const InGDKSubsystem)
		: GDKSubsystem(InGDKSubsystem)
	{
		check(GDKSubsystem);
	}

	virtual ~FOnlineFriendsGDK() = default;

PACKAGE_SCOPE:
	void OnUserPresenceUpdate(const FUniqueNetIdGDK& FriendId, const TSharedRef<FOnlineUserPresenceGDK>& UpdatedPresence);
	void OnUsersSessionPresenceUpdate(const FOnlineGDKActivitiesResultMap& ActivitiesResults);
	void OnUsersSessionPresenceUpdate(TArray<FOnlineSession>& OnlineSessions);
	bool ShowProfileUI(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate = FOnSendInviteComplete());
	void HandleShowProfileUIComplete(int32 LocalUserNum, FUniqueNetIdGDKRef FriendId, const FString ListName, const FOnSendInviteComplete Delegate);
private:
	/** Reference to the main GDK subsystem */
	class FOnlineSubsystemGDK* const GDKSubsystem;

	/** Map of local users to map of their friends */
	FOnlineUserFriendsListGDKMap FriendsMap;

	/** These are users we have asked not to play with (similar to a blocklist) */
	TUniqueNetIdMap<TArray<TSharedRef<FOnlineBlockedPlayerGDK>>> AvoidListMap;

	/** Map of players who have friends-reads in progress */
	TUniqueNetIdMap<TArray<FOnReadFriendsListComplete>> FriendsListInProgressDelegates;
};

typedef TSharedPtr<FOnlineFriendsGDK, ESPMode::ThreadSafe> FOnlineFriendsGDKPtr;
