// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineSessionGDK.h"
#include "GDKRuntimeModule.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetXSTSToken.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCheckForPackageUpdate.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetOverallReputation.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserPrivilege.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
//#include "HttpModule.h"
//#include "Interfaces/IHttpRequest.h"
//#include "Interfaces/IHttpResponse.h"
#include "OnlineError.h"
#include "GDKRuntimeModule.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Algo/Count.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

namespace
{
	/** Helper function to get an Unreal login status from a User. */
	ELoginStatus::Type GetLoginStatusForUser(FGDKUserHandle InUser)
	{
		if (!InUser.IsValid())
		{
			return ELoginStatus::NotLoggedIn;
		}
		
		XUserState UserState = XUserState::SignedOut;
		XUserGetState(InUser, &UserState);
		if (UserState == XUserState::SignedIn)
		{
			return ELoginStatus::LoggedIn;
		}
		
		return ELoginStatus::UsingLocalProfile;
	}
}

FOnlineIdentityGDK::FOnlineIdentityGDK(class FOnlineSubsystemGDK* InSubsystem) :
	GDKSubsystem(InSubsystem)
{
	check(GDKSubsystem);

	// Store the Login XSTS endpoint for later XSTS generation
	GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("LoginXSTSEndpoint"), LoginXSTSEndpoint, GEngineIni);
	GConfig->GetFloat(TEXT("OnlineSubsystemGDK"), TEXT("PrivilegeCheckDelayOnResume"), PrivilegeCheckDelayOnResume, GEngineIni);

	HookGDKEvents();
}

FOnlineIdentityGDK::~FOnlineIdentityGDK()
{
	UnhookGDKEvents();
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityGDK::GetUserAccount(const FUniqueNetId& UserId) const
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);
	const TSharedRef<FUserOnlineAccountGDK>* FoundUserPtr = OnlineUsers.Find(GDKUserId);

	return FoundUserPtr ? *FoundUserPtr : TSharedPtr<FUserOnlineAccountGDK>();
}

TArray<TSharedPtr<FUserOnlineAccount>> FOnlineIdentityGDK::GetAllUserAccounts() const 
{
	TArray<TSharedPtr<FUserOnlineAccount>> UserAccounts;
	UserAccounts.Empty(OnlineUsers.Num());

	for (const TPair<FUniqueNetIdRef, TSharedRef<FUserOnlineAccountGDK>>& Pair : OnlineUsers)
	{
		UserAccounts.Emplace(Pair.Value);
	}

	return UserAccounts;
}

bool FOnlineIdentityGDK::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	if (LocalUserNum < 0 || LocalUserNum > MAX_LOCAL_PLAYERS)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Login_Delegate);
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGDK::EmptyId(), FString::Printf(TEXT("Invalid controller index %d"), LocalUserNum));
		return false;
	}

	FUniqueNetIdPtr UserId = FOnlineIdentityGDK::GetUniquePlayerId(LocalUserNum);
	if (!UserId.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Login_Delegate);
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGDK::EmptyId(), FString::Printf(TEXT("No user logged in for controller=%d"), LocalUserNum));
		return false;
	}	
	// If there is no configured Endpoint, we do not need to fetch a XSTS token
	if (LoginXSTSEndpoint.IsEmpty())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Login_Delegate);
		TriggerOnLoginCompleteDelegates(LocalUserNum, true, *UserId, FString());
		return true;
	}

	FGDKUserHandle GDKUser = GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(*UserId));
	if (!GDKUser)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Login_Delegate);
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *UserId, FString::Printf(TEXT("No user logged in for controller=%d"), LocalUserNum));
		return false;
	}

	bool bNewUserAuth = true;
	if (const TSharedRef<FUserOnlineAccountGDK>* OnlineUserGDK = OnlineUsers.Find(FUniqueNetIdGDK::Cast(*UserId)))
	{
		if(!(*OnlineUserGDK)->GetAccessToken().IsEmpty())
		{
			bNewUserAuth = false;
		}
	}
	auto OnLoginCompleteDelegate = FOnXSTSTokenCompleteDelegate::CreateLambda(
	[this,bNewUserAuth](FOnlineError Result, int32 LocalUserNum, const FUniqueNetIdGDKRef& GDKUserId, const FString& ResultSignature, const FString& ResultToken)
	{
		// At this point, if we were successful, our auth token is saved on this user for using in listening delegates
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Login_Delegate);
			TriggerOnLoginCompleteDelegates(LocalUserNum, Result.WasSuccessful(), *GDKUserId, Result.GetErrorMessage().ToString());
		}		
		
		if (TSharedRef<FUserOnlineAccountGDK>* OnlineUserGDK = OnlineUsers.Find(GDKUserId))
		{
			// Find or create an GDK context for this user
			FGDKContextHandle LocalUserGDKContext = GDKSubsystem->GetGDKContext(*GDKUserId);
			if (LocalUserGDKContext)
			{
				if (bNewUserAuth)
				{
					// Clear any stale session state from the service. Could be left over from previous runs / other titles.
					GDKSubsystem->GetSessionInterfaceGDK()->ClearSessionState(LocalUserGDKContext, GDKUserId);
				}

				// Check if we need to query this user's bad reputation state, and do so if needed
				TOptional<bool> BadReputationState((*OnlineUserGDK)->GetIsBadReputation());
				if (!BadReputationState.IsSet())
				{
					// Have this user request their bad reputation state
					GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGetOverallReputation>(
						GDKSubsystem,
						LocalUserGDKContext,
						GDKUserId,
						FOnGetOverallReputationCompleteDelegate::CreateThreadSafeSP(this, &FOnlineIdentityGDK::OnReputationQueryComplete));
				}
			}
		}
	});

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGetXSTSToken>(GDKSubsystem, GDKUser, LocalUserNum, LoginXSTSEndpoint, MoveTemp(OnLoginCompleteDelegate));

	return true;
}

bool FOnlineIdentityGDK::Logout(int32 LocalUserNum)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_Logout_Delegate);
	TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
	return false;
}

bool FOnlineIdentityGDK::AutoLogin(int32 LocalUserNum)
{
	return Login(LocalUserNum, FOnlineAccountCredentials());
}

ELoginStatus::Type FOnlineIdentityGDK::GetLoginStatus(int32 ControllerIndex) const
{
	FGDKUserHandle RequestedUser = GetUserForPlatformUserId(ControllerIndex);
	return GetLoginStatusForUser(RequestedUser);
}

ELoginStatus::Type FOnlineIdentityGDK::GetLoginStatus(const FUniqueNetId& UserId) const
{
	FGDKUserHandle RequestedUser = GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(UserId));
	return GetLoginStatusForUser(RequestedUser);
}

FUniqueNetIdPtr FOnlineIdentityGDK::GetUniquePlayerId(int32 ControllerIndex) const
{
	FGDKUserHandle User = GetUserForPlatformUserId(ControllerIndex);
	if( User.IsValid())
	{
		return FUniqueNetIdGDK::Create(User);
	}

	return nullptr;
}

//WMM TODO: find API to retrieve sponsor from guest account
/*
FUniqueNetIdPtr FOnlineIdentityGDK::GetSponsorUniquePlayerId(int32 ControllerIndex) const
{
	FGDKUserHandle * User = GetUserForPlatformUserId(ControllerIndex);
	if( User != nullptr )
	{
		if( User->Sponsor != nullptr )
		{
			return FUniqueNetIdGDK::Create(User->Sponsor->XboxUserId);
		}
	}

	return nullptr;
}
*/

FUniqueNetIdPtr FOnlineIdentityGDK::CreateUniquePlayerId(uint8* Bytes, int32 Size) 
{
	if (Bytes && Size == sizeof(uint64))
	{
		uint64* RawUniqueId = (uint64*)Bytes;
		return FUniqueNetIdGDK::Create(*RawUniqueId);
		
	}
	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityGDK::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdGDK::Create(Str);
}

FString FOnlineIdentityGDK::GetPlayerNickname(int32 ControllerIndex) const
{
	FString PlayerNickname;
	FGDKUserHandle RequestedUser = GetUserForPlatformUserId(ControllerIndex);
	if( RequestedUser )
	{
		PlayerNickname = GetPlayerNickname(RequestedUser);
	}

	return PlayerNickname;
}

FString FOnlineIdentityGDK::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	FString PlayerNickname;

	FGDKUserHandle RequestedUser = GetUserForUniqueNetId(*FUniqueNetIdGDK::Cast(UserId));
	if (RequestedUser)
	{
		PlayerNickname = GetPlayerNickname(RequestedUser);
	}

	return PlayerNickname;
}

FString FOnlineIdentityGDK::GetPlayerNickname(const FGDKUserHandle RequestedUser) const
{
	FString PlayerNickname;

	uint64 RequestedUserId;
	if (SUCCEEDED(XUserGetId(RequestedUser, &RequestedUserId)))
	{
		PlayerNickname = FOnlineUserInfoGDK::FilterPlayerName(IGDKRuntimeModule::Get().GetGamertag(RequestedUser));

		UE_LOG_ONLINE_IDENTITY(VeryVerbose, TEXT("[FOnlineIdentityGDK LocalPlayer Info] XUID: [%llu] Nickname: [%s]"), RequestedUserId, *PlayerNickname);
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("[FOnlineIdentityGDK::GetPlayerNickname] User was invalid"));
	}

	return PlayerNickname;
}

FString FOnlineIdentityGDK::GetAuthToken(int32 ControllerIndex) const
{
	FString AuthToken;

	FUniqueNetIdPtr UserId = GetUniquePlayerId(ControllerIndex);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccountGDK> UserAccount = StaticCastSharedPtr<FUserOnlineAccountGDK>(GetUserAccount(*UserId));
		if (UserAccount.IsValid())
		{
			AuthToken = UserAccount->GetAccessToken();
		}
	}

	if (AuthToken.IsEmpty())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("There was no AuthToken stored for ControllerId %d"), ControllerIndex);
	}

	return AuthToken;
}

void FOnlineIdentityGDK::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityGDK::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	GDKSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_RevokeAuthToken_Delegate);
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

FGDKUserHandle FOnlineIdentityGDK::GetUserForUniqueNetId(const FUniqueNetIdGDK& UniqueId) const
{
	return IGDKRuntimeModule::Get().GetUserHandleByXUserId(UniqueId.ToUint64());
}

FGDKUserHandle FOnlineIdentityGDK::GetUserForPlatformUserId(int32 PlatformUserId) const
{
	return IGDKRuntimeModule::Get().GetUserHandleByPlatformId(GetPlatformUserIdFromLocalUserNum(PlatformUserId));
}

FPlatformUserId FOnlineIdentityGDK::GetPlatformUserIdFromGDKUser( FGDKUserHandle InUser ) const
{
	if (!InUser)
	{
		return PLATFORMUSERID_NONE;
	}

	return IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(InUser);
}

TArray<FGDKUserHandle >& FOnlineIdentityGDK::GetCachedUsers()
{
	return CachedUsers;
}

bool FOnlineIdentityGDK::AddUserAccount(FGDKUserHandle InUser)
{
	if (InUser)
	{
		uint64 UserId;
		if (SUCCEEDED(XUserGetId(InUser, &UserId)))
		{
			FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Create(UserId);
			if (!OnlineUsers.Contains(GDKUserId))
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("User %s added to OnlineUsers"), *GDKUserId->ToString());
				TSharedRef<FUserOnlineAccountGDK> OnlineUser(MakeShared<FUserOnlineAccountGDK>(InUser, AsShared()));
				OnlineUsers.Add(GDKUserId, MoveTemp(OnlineUser));
				CachedUsers.Add(InUser);
				return true;
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("User %s already exists in OnlineUsers"), *GDKUserId->ToString());
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Log, TEXT("User does not have a xuid"));
		}
	}

	return false;
}

bool FOnlineIdentityGDK::RemoveUserAccount(FGDKUserHandle InUser)
{
	if (InUser)
	{
		uint64 UserId;
		if (SUCCEEDED(XUserGetId(InUser, &UserId)))
		{
			FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Create(UserId);
			if (OnlineUsers.Remove(GDKUserId) > 0 && CachedUsers.Remove(InUser) > 0)
			{
				return true;
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("User %s not found in OnlineUsers"), *GDKUserId->ToString());
			}
		}
		else
		{
			check(!CachedUsers.Contains(InUser));
			UE_LOG_ONLINE_IDENTITY(Log, TEXT("User does not have a xuid"));
		}
	}

	return false;
}

//WMM TODO: This changed in GDK.. We now have to manage our own view of users through XUserChangeEvents
// there is no central list of device associated users.
void FOnlineIdentityGDK::RefreshGamepadsAndUsers()
{
	UE_LOG_ONLINE_IDENTITY(Log, TEXT("RefreshGamepadsAndUsers"));
	// Lock CachedUsers while we access it
	const FScopeLock CachedUsersScopeLock(&CachedUsersLock);

	const TArray<FGDKUserHandle> AllUsers = IGDKRuntimeModule::Get().GetAllUserHandles();
		
	TArray<FUniqueNetIdGDKRef> UserReputationsToQuery;

	// cache the online user account info
	for( const FGDKUserHandle& User : AllUsers )
	{
		if (ensure(User.IsValid()))
		{
			const bool bIsNewUser = AddUserAccount(User);
			if (bIsNewUser)
			{
				uint64 UserId;
				if (SUCCEEDED(XUserGetId(User, &UserId)))
				{
					UserReputationsToQuery.Add(FUniqueNetIdGDK::Create(UserId));
				}
			}
		}
	}

	for (GDKUserAccountMap::TIterator UserIter(OnlineUsers); UserIter; ++UserIter)
	{
		bool bFound = false;
		for( const FGDKUserHandle& User : AllUsers )
		{
			if (ensure(User.IsValid()))
			{
				uint64 UserId;
				if (SUCCEEDED(XUserGetId(User, &UserId)) && *StaticCastSharedRef<const FUniqueNetIdGDK>(UserIter.Key()) == UserId)
				{
					bFound = true;
					break;
				}
			}
		}

		if (!bFound)
		{
			CachedUsers.Remove(UserIter->Value->GetUserHandle());
			UserIter.RemoveCurrent();
		}
	}

	// Query new user's reputation state for clients to read
	if (UserReputationsToQuery.Num() > 0)
	{
		FUniqueNetIdPtr LocalSignedInUser = GetFirstSignedInUser(AsShared());
		if (LocalSignedInUser.IsValid())
		{
			FGDKContextHandle LocalUserGDKContext = GDKSubsystem->GetGDKContext(*LocalSignedInUser);
			if (LocalUserGDKContext)
			{
				GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKGetOverallReputation>(
					GDKSubsystem,
					LocalUserGDKContext,
					MoveTemp(UserReputationsToQuery),
					FOnGetOverallReputationCompleteDelegate::CreateThreadSafeSP(this, &FOnlineIdentityGDK::OnReputationQueryComplete));
			}
		}
	}
}

void FOnlineIdentityGDK::OnReputationQueryComplete(const TUniqueNetIdMap<bool>& UserIsBadMap)
{
	for (const TPair<FUniqueNetIdRef, bool>& Pair : UserIsBadMap)
	{
		if (TSharedRef<FUserOnlineAccountGDK>* GDKUserPtr = OnlineUsers.Find(Pair.Key))
		{
			(*GDKUserPtr)->SetBadReputation(Pair.Value);
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("Setting User %s's BadReputation to %s"), *Pair.Key->ToString(), Pair.Value ? TEXT("1") : TEXT("0"));
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Verbose, TEXT("User %s no longer present, but their BadReputation was %s"), *Pair.Key->ToString(), Pair.Value ? TEXT("1") : TEXT("0"));
		}
	}
}

void FOnlineIdentityGDK::SetUserXSTSToken(FGDKUserHandle User, const FString& AuthToken)
{
	check(User);

	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Create(User);

	const TSharedRef<FUserOnlineAccountGDK>* const FoundUser = OnlineUsers.Find(GDKUserId);
	if (FoundUser == nullptr)
	{
		TSharedRef<FUserOnlineAccountGDK> OnlineUser(MakeShared<FUserOnlineAccountGDK>(User, AsShared()));
		OnlineUser->SetAccessToken(AuthToken);
		OnlineUsers.Add(GDKUserId, OnlineUser);
	}
	else
	{
		(*FoundUser)->SetAccessToken(AuthToken);
	}
}

void FOnlineIdentityGDK::HandleAppResume()
{
	UE_LOG_ONLINE_IDENTITY(Log, TEXT("FOnlineIdentityGDK::HandleAppResume"));
	RefreshGamepadsAndUsers(); //WMM TODO: How does GDK handle subscribed events in this case?
	PendingPrivilegeCheckDelay = PrivilegeCheckDelayOnResume;
}

void FOnlineIdentityGDK::OnEngineInitComplete()
{
	RefreshGamepadsAndUsers();
}

#if WITH_EDITOR
void FOnlineIdentityGDK::OnPostPIEStarted(bool bIsSimulating)
{
	RefreshGamepadsAndUsers();
}
#endif

void FOnlineIdentityGDK::HookGDKEvents()
{
	AppInitComplete = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FOnlineIdentityGDK::OnEngineInitComplete);

#if WITH_EDITOR
	PostPIEStartedHandle = FEditorDelegates::PostPIEStarted.AddRaw(this, &FOnlineIdentityGDK::OnPostPIEStarted);
#endif

	InputDeviceConnectionChanged = IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(this, &FOnlineIdentityGDK::OnInputDeviceConnectionChange);
	InputDevicePairingChanged = IPlatformInputDeviceMapper::Get().GetOnInputDevicePairingChange().AddRaw(this, &FOnlineIdentityGDK::OnInputDevicePairingChange);
	
	FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FOnlineIdentityGDK::HandleAppResume);

}

void FOnlineIdentityGDK::UnhookGDKEvents()
{
	FCoreDelegates::OnFEngineLoopInitComplete.Remove(AppInitComplete);
#if WITH_EDITOR
	FEditorDelegates::PostPIEStarted.Remove(PostPIEStartedHandle);
#endif
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().Remove(InputDeviceConnectionChanged);
	IPlatformInputDeviceMapper::Get().GetOnInputDevicePairingChange().Remove(InputDevicePairingChanged);
}

void FOnlineIdentityGDK::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(UserId);

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGetUserPrivilege>(GDKSubsystem, GDKContext, GDKUserId, Privilege, Delegate, ShowResolveUI);
}

FPlatformUserId FOnlineIdentityGDK::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UniqueNetId);
	FGDKUserHandle UserHandle = GetUserForUniqueNetId(*GDKUserId);
	return IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(UserHandle);
}

FString FOnlineIdentityGDK::GetAuthType() const
{
	return FString(TEXT("xbl"));
}

void FOnlineIdentityGDK::GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& /*TokenType*/, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const
{
	FExternalAuthToken ExternalToken;
	ExternalToken.TokenString = GetAuthToken(LocalUserNum);

	// If the token is initially empty, Login might need to be called, we'll attempt that before returning an error state
	if (!ExternalToken.HasTokenString())
	{
		// We are doing a non-const operation in a const method, but it is warranted in this case
		FOnlineIdentityGDK* MutableThis = const_cast<FOnlineIdentityGDK*>(this);

		TSharedRef<FDelegateHandle> DelegateHandleRef = MakeShared<FDelegateHandle>();
		*DelegateHandleRef = MutableThis->AddOnLoginCompleteDelegate_Handle(LocalUserNum, FOnLoginCompleteDelegate::CreateLambda([this, Delegate, DelegateHandleRef](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& Error)
			{
				if (bWasSuccessful)
				{
					FExternalAuthToken ExternalToken;
					ExternalToken.TokenString = GetAuthToken(LocalUserNum);
					Delegate.ExecuteIfBound(LocalUserNum, ExternalToken.HasTokenString(), ExternalToken);
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[%hs] Login attempt failed for user [%d], unable to retrieve Linked Account Auth Token"), __FUNCTION__, LocalUserNum);
					Delegate.ExecuteIfBound(LocalUserNum, false, FExternalAuthToken());
				}

				FOnlineIdentityGDK* MutableThis = const_cast<FOnlineIdentityGDK*>(this);
				MutableThis->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *DelegateHandleRef);
			}));

		MutableThis->AutoLogin(LocalUserNum);
	}
	else
	{
		Delegate.ExecuteIfBound(LocalUserNum, ExternalToken.HasTokenString(), ExternalToken);
	}
}

void FOnlineIdentityGDK::OnUserAdded(FGDKUserHandle InUserAdded)
{
	AddUserAccount(InUserAdded);

	ELoginStatus::Type LoginStatus = GetLoginStatusForUser(InUserAdded);
	const FPlatformUserId PlatformUserId = GetPlatformUserIdFromGDKUser(InUserAdded);

	FUniqueNetIdGDKRef NewUserAdded = FUniqueNetIdGDK::Create(InUserAdded);

	UE_LOG_ONLINE_IDENTITY(Log, TEXT("UserAdded PlatformUserId %d"), PlatformUserId.GetInternalId());

	/* HACK -- Assume the previous state could not be UsingLocalProfile */
	// BUG: online status change is set per user, so only valid values are typically up to MAX_LOCAL_PLAYERS
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_OnUserAdded_Delegate);
	TriggerOnLoginStatusChangedDelegates(PlatformUserId, (LoginStatus != ELoginStatus::LoggedIn) ? ELoginStatus::LoggedIn : ELoginStatus::NotLoggedIn, LoginStatus, *NewUserAdded);
}

void FOnlineIdentityGDK::OnUserRemoved(FGDKUserHandle InUserRemoved)
{
	RemoveUserAccount(InUserRemoved);

	const FPlatformUserId PlatformUserId = GetPlatformUserIdFromGDKUser(InUserRemoved);

	FUniqueNetIdGDKRef NewUserRemoved = FUniqueNetIdGDK::Create(InUserRemoved);

	UE_LOG_ONLINE_IDENTITY(Log, TEXT("UserRemoved PlatformUserId %d"), PlatformUserId.GetInternalId());

	/* HACK -- Assume the previous state could not be UsingLocalProfile */
	// BUG: online status change is set per user, so only valid values are typically up to MAX_LOCAL_PLAYERS
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_OnUserRemoved_Delegate);
	TriggerOnLoginStatusChangedDelegates(PlatformUserId, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *NewUserRemoved);

	// clear stat subscriptions so they will be readded once the user logs back in
	FOnlinePresenceGDKPtr PresenceInterface(GDKSubsystem->GetPresenceGDK());
	if (PresenceInterface.IsValid())
	{
		PresenceInterface->ClearAllStatUpdateSubscriptionsForUser(NewUserRemoved);
	}
}

void FOnlineIdentityGDK::OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	// only act if this is a disconnect event
	if (NewConnectionState == EInputDeviceConnectionState::Disconnected)
	{
		RefreshGamepadsAndUsers();
	}
}

void FOnlineIdentityGDK::OnInputDevicePairingChange(FInputDeviceId InputDeviceId, FPlatformUserId InNewUserId, FPlatformUserId InOldUserId)
{
	RefreshGamepadsAndUsers();

	TOptional<uint64> PreviousUserXuid = IGDKRuntimeModule::Get().GetXUserIdByPlatformId(InOldUserId);
	TOptional<uint64> NewUserXuid = IGDKRuntimeModule::Get().GetXUserIdByPlatformId(InNewUserId);

	FUniqueNetIdGDKRef PreviousUserId = PreviousUserXuid.IsSet() ? FUniqueNetIdGDK::Create(PreviousUserXuid.GetValue()) : FUniqueNetIdGDK::EmptyId();
	FUniqueNetIdGDKRef NewUserId = NewUserXuid.IsSet() ? FUniqueNetIdGDK::Create(NewUserXuid.GetValue()) : FUniqueNetIdGDK::EmptyId();

	UE_LOG_ONLINE_IDENTITY(Log, TEXT("Triggering OnInputDevicePairingChange with ControllerIndex %d, PreviousUser '%s', NewUser '%s'"),
		InputDeviceId.GetId(), *PreviousUserId->ToString(), *NewUserId->ToString());

	if (InputDeviceId.IsValid())
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

		// look up the number of connected input devices associated with each user
		// do not count the "default" input device... this will be keyboards, mice etc.
		// note that non-controllers will also be counted, including touch (and wheels, arcade sticks etc. but we don't support those yet)
		const FInputDeviceId DefaultDeviceId = DeviceMapper.GetDefaultInputDevice();
		auto IsConnectedDevice = [&DeviceMapper, &DefaultDeviceId]( const FInputDeviceId& DeviceId )
		{
			return (DeviceId != DefaultDeviceId) && (DeviceMapper.GetInputDeviceConnectionState(DeviceId) == EInputDeviceConnectionState::Connected);
		};

		TArray<FInputDeviceId> Dummy;
		if (InOldUserId.IsValid())
		{
			DeviceMapper.GetAllInputDevicesForUser(InOldUserId, Dummy);
		}
		int32 PreviousUserControllerNum = Algo::CountIf( Dummy, IsConnectedDevice );
 
		Dummy.Reset();
		if (InNewUserId.IsValid())
		{
			DeviceMapper.GetAllInputDevicesForUser(InNewUserId, Dummy);
		}
		int32 NewUserControllerNum = Algo::CountIf( Dummy, IsConnectedDevice );


		FControllerPairingChangedUserInfo PreviousUserInfo(*PreviousUserId, PreviousUserControllerNum);
		FControllerPairingChangedUserInfo NewUserInfo(*NewUserId, NewUserControllerNum);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineIdentityGDK_InputDevicePairingChanged_Delegate);
		// TODO: Replace this with FInputDeviceID as well, but requires a refactor of OSS first
		TriggerOnControllerPairingChangedDelegates(InputDeviceId.GetId(), PreviousUserInfo, NewUserInfo);
	}
}

void FOnlineIdentityGDK::OnUserLoginChange(bool bIsSignIn, int32 UserId, int32 UserIndex )
{
	const FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(GetPlatformUserIdFromLocalUserNum(UserId));
	if (UserHandle.IsValid())
	{
		if (bIsSignIn)
		{
			OnUserAdded(UserHandle);
		}
		else
		{
			OnUserRemoved(UserHandle);
		}
	}
}

void FOnlineIdentityGDK::Tick(const float DeltaTime)
{
	if (PendingPrivilegeCheckDelay > 0)
	{
		PendingPrivilegeCheckDelay -= DeltaTime;
	}
}


/** FUserOnlineAccountGDK */

FString FUserOnlineAccountGDK::GetAccessToken() const
{
	return UserXSTSToken;
}

void FUserOnlineAccountGDK::SetAccessToken(const FString& AuthToken)
{
	UserXSTSToken = AuthToken;
}

void FUserOnlineAccountGDK::SetBadReputation(const bool bIsBadReputation)
{
	UserAttributes.Add(BAD_REPUTATION_ATTRIBUTE, bIsBadReputation ? TEXT("1") : TEXT("0"));
}

TOptional<bool> FUserOnlineAccountGDK::GetIsBadReputation() const
{
	TOptional<bool> ReturnValue;

	if (const FString* BadReputationStringPtr = UserAttributes.Find(BAD_REPUTATION_ATTRIBUTE))
	{
		ReturnValue = TOptional<bool>(BadReputationStringPtr->ToBool());
	}

	return ReturnValue;
}

bool FUserOnlineAccountGDK::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetUserAttribute(AttrName, OutAttrValue);
}

bool FUserOnlineAccountGDK::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	// Check if value is already set to this value
	const FString* const ExistingAttrPtr = UserAttributes.Find(AttrName);
	if (ExistingAttrPtr != nullptr)
	{
		if (ExistingAttrPtr->Equals(AttrValue))
		{
			return false;
		}
	}

	// Add or Set value
	UserAttributes.Add(AttrName, AttrValue);
	return true;
}

FUniqueNetIdRef FUserOnlineAccountGDK::GetUserId() const
{
	return UserId.ToSharedRef();
}

FString FUserOnlineAccountGDK::GetRealName() const
{
	return GetDisplayName();
}

FString FUserOnlineAccountGDK::GetDisplayName(const FString& Platform /*= FString()*/) const
{
	FString Result;

	const FOnlineIdentityGDKPtr IdentityInterface = IdentityInterfaceWeakPtr.Pin();
	if (!IdentityInterface.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("[FUserOnlineAccountGDK::GetDisplayName] Couldn't get the identity interface"));
		return Result;
	}

	Result = IdentityInterface->GetPlayerNickname(UserData);

	return Result;
}

bool FUserOnlineAccountGDK::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	const FString* const ExistingAttrPtr = UserAttributes.Find(AttrName);
	if (ExistingAttrPtr != nullptr)
	{
		OutAttrValue = *ExistingAttrPtr;
		return true;
	}

	OutAttrValue.Empty();
	return false;
}

// Debugging commands to get or set the first logged in user's reputation.
#if !UE_BUILD_SHIPPING

/*
static void SetReputationFinished(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{

	if (HttpResponse.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("SetReputation HTTP request complete. bSucceeded = %d. Response code = %d"),
			bSucceeded ? 1 : 0, HttpResponse->GetResponseCode());
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Log, TEXT("GDK SetReputation HTTP request complete. Response is null."));
	}
}
*/
//WMM TODO - analogue for xboxplayerhash?
/*
static void DebugSetReputation(const FString& ReputationJson)
{
	const IOnlineSubsystem* const Subsystem = IOnlineSubsystem::Get(GDK_SUBSYSTEM);

	if (Subsystem == nullptr)
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugSetReputation: couldn't get an online subsystem"));
		return;
	}

	const IOnlineIdentityPtr IdentityInterface = Subsystem->GetIdentityInterface();
	if (!IdentityInterface.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugSetReputation: couldn't get the identity interface"));
		return;
	}
	
	const FUniqueNetIdPtr UserId = GetFirstSignedInUser(IdentityInterface);
	if(!UserId.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugSetReputation: invalid UserId"));
		return;
	}
	
	FString UserHash;
	TSharedPtr<FOnlineIdentityGDK, ESPMode::ThreadSafe> IdGDKPtr = StaticCastSharedPtr<FOnlineIdentityGDK>(IdentityInterface);
	if (IdLivePtr.IsValid())
	{
		FGDKUserHandle GDKUser = IdLivePtr->GetUserForUniqueNetId(*StaticCastSharedPtr<const FUniqueNetIdGDK>(UserId));
		if (GDKUser)
		{
			UserHash = GDKUser->XboxUserHash->Data();
		}
	}

	TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	
	HttpRequest->OnProcessRequestComplete().BindStatic(&SetReputationFinished);
	HttpRequest->SetURL(FString::Printf(TEXT("https://reputation.xboxlive.com/users/me/resetreputation")));
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetHeader(TEXT("x-xbl-contract-version"), TEXT("101"));
	HttpRequest->SetHeader(TEXT("xbl-authz-actor-10"), UserHash);
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetContentAsString(ReputationJson);

	HttpRequest->ProcessRequest();
}

static void DebugSetBadReputation()
{
	DebugSetReputation(TEXT("{ \"fairplayReputation\": 1, \"commsReputation\": 1, \"userContentReputation\": 1 }"));
}

static void DebugSetGoodReputation()
{
	DebugSetReputation(TEXT("{ \"fairplayReputation\": 75, \"commsReputation\": 75, \"userContentReputation\": 75 }"));
}

static FAutoConsoleCommand ConsoleCommandLiveSetBadReputation(
	TEXT("online.LiveSetBadReputation"),
	TEXT("Set a bad reputation for the first logged in user."),
	FConsoleCommandDelegate::CreateStatic(DebugSetBadReputation)
);

static FAutoConsoleCommand ConsoleCommandLiveSetGoodReputation(
	TEXT("online.LiveSetGoodReputation"),
	TEXT("Set a good reputation for the first logged in user."),
	FConsoleCommandDelegate::CreateStatic(DebugSetGoodReputation)
);
*/
static void DebugLogReputation()
{
	FOnlineSubsystemGDK* const GDKSubsystem = static_cast<FOnlineSubsystemGDK*>(IOnlineSubsystem::Get(GDK_SUBSYSTEM));

	if (GDKSubsystem == nullptr)
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugLogReputation: couldn't get an online subsystem"));
		return;
	}

	const IOnlineIdentityPtr IdentityInterface = GDKSubsystem->GetIdentityInterface();
	if (!IdentityInterface.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugLogReputation: couldn't get the identity interface"));
		return;
	}
	
	const FUniqueNetIdPtr UserId = GetFirstSignedInUser(IdentityInterface);
	if(!UserId.IsValid())
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("DebugLogReputation: invalid UserId"));
		return;
	}

	FUniqueNetIdGDKRef UserIdGDKRef = StaticCastSharedRef<const FUniqueNetIdGDK>(UserId.ToSharedRef());

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*UserId);
	TArray<FUniqueNetIdGDKRef> UserIdArray;
	UserIdArray.Add(UserIdGDKRef);
	
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGetOverallReputation>(GDKSubsystem, GDKContext, MoveTemp(UserIdArray),
		FOnGetOverallReputationCompleteDelegate::CreateLambda([IdentityInterface](const UserReputationMap& UsersWithBadReputation)
		{
			for (const UserReputationMap::ElementType& UserPair : UsersWithBadReputation)
			{
				UE_LOG_ONLINE_IDENTITY(Log, TEXT("DebugLogReputation: user %s, OverallReputationIsBad: %s"), 
					*IdentityInterface->GetPlayerNickname(*UserPair.Key),
					UserPair.Value ? TEXT("true") : TEXT("false"));
			}
		}));
}

static FAutoConsoleCommand ConsoleCommandGDKDebugLogReputation(
	TEXT("online.LiveDebugLogReputation"),
	TEXT("Prints the reputation of the first logged in user to the log."),
	FConsoleCommandDelegate::CreateStatic(DebugLogReputation)
);

#endif

#endif //WITH_GRDK