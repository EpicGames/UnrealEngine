// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineLeaderboardInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineEventsInterfaceGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetLeaderboardForUsers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKReadLeaderboards.h"

FOnlineLeaderboardsGDK::FOnlineLeaderboardsGDK(class FOnlineSubsystemGDK* InSubsystem) : GDKSubsystem(InSubsystem)
{
}

bool FOnlineLeaderboardsGDK::ReadLeaderboards(const TArray<FUniqueNetIdRef>& Players, FOnlineLeaderboardReadRef& ReadObject)
{
	if ( !GDKSubsystem )
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT( "ReadLeaderboards failed. GDKSubsystem is null."));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboards_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	FOnlineAsyncTaskManager* AsyncTaskManager = GDKSubsystem->GetAsyncTaskManager();
	if(!AsyncTaskManager)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get AsyncTaskManager from Online Subsystem"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboards_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	//Currently GDK only supports single column leaderboards. If more than one column is requested, fail the request for now.
	//if(ReadObject->ColumnMetadata.Num() > 1)
	//{
	//	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failing Leaderboards request for multiple columns. GDK currently only supports single-column leaderboards."));
	//	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboards_Delegate);
	//	TriggerOnLeaderboardReadCompleteDelegates(false);
	//	return false;
	//}
		
	//Attempt to retrieve an GDK user from the signed in users for making leaderboard requests.
	//This is necessary because the user actively requesting this data is not provided explicitly and not guaranteed to be in the Players list.
	FGDKContextHandle GDKContext;
	
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	TArray<FGDKUserHandle >& Users = Identity->GetCachedUsers();
	for (const FGDKUserHandle& User : Users)
	{
		XUserState UserState = XUserState::SignedOut;
		XUserGetState(User, &UserState);
			
		if (UserState != XUserState::SignedIn)
		{
			continue;
		}

		GDKContext = GDKSubsystem->GetGDKContext(User);
		if (GDKContext.IsValid())
		{
			break;
		}
	}
	
	if(!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain an GDKContext to retrieve leaderboards data."));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboards_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKReadLeaderboards>(
		GDKSubsystem,
		GDKContext,
		TArray<FOnlineLeaderboardReadRef>(),
		ReadObject,
		Players,
		XblSocialGroupType::None,
		ReadObject->SortedColumn,
		false,
		FOnReadLeaderboardsCompleteDelegate::CreateThreadSafeSP(this, &FOnlineLeaderboardsGDK::OnReadLeaderboardsComplete));

	return true;
}

void FOnlineLeaderboardsGDK::OnReadLeaderboardsComplete(bool bSuccess)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_OnReadLeaderboardsComplete_Delegate);
	TriggerOnLeaderboardReadCompleteDelegates(bSuccess);
}

bool FOnlineLeaderboardsGDK::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	if ( !GDKSubsystem )
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT( "ReadLeaderboardsForFriends failed. GDKSubsystem is null."));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsForFriends_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	FOnlineAsyncTaskManager* AsyncTaskManager = GDKSubsystem->GetAsyncTaskManager();
	if(!AsyncTaskManager)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get AsyncTaskManager from Online Subsystem"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsForFriends_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	//Currently GDK only supports single column leaderboards. If more than one column is requested, fail the request for now.
	//if(ReadObject->ColumnMetadata.Num() > 1)
	//{
	//	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failing Leaderboards request for multiple columns. GDK currently only supports single-column leaderboards."));
	//	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsForFriends_Delegate);
	//	TriggerOnLeaderboardReadCompleteDelegates(false);
	//	return false;
	//}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if(!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain an FGDKContextHandle to retrieve leaderboards data."));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsForFriends_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	FString StatName;
	if (ReadObject->ColumnMetadata.Num() > 0)
	{
		StatName = ReadObject->SortedColumn;
	}

	if (StatName.IsEmpty())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failing Leaderboards request. No statistic requested for friends leaderboard."));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsForFriends_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	TArray<FUniqueNetIdRef> Players;
	Players.Add(GDKSubsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum).ToSharedRef());

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKReadLeaderboards>(
		GDKSubsystem,
		GDKContext,
		TArray<FOnlineLeaderboardReadRef>(),
		ReadObject,
		Players,
		XblSocialGroupType::People,
		StatName,
		true,
		FOnReadLeaderboardsCompleteDelegate::CreateThreadSafeSP(this, &FOnlineLeaderboardsGDK::OnReadLeaderboardsForFriendsComplete));

	return true;
}

void FOnlineLeaderboardsGDK::OnReadLeaderboardsForFriendsComplete(bool bSuccess)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_OnReadLeaderboardsForFriendsComplete_Delegate);
	TriggerOnLeaderboardReadCompleteDelegates(bSuccess);
}

bool FOnlineLeaderboardsGDK::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsGDK::ReadLeaderboardsAroundRank is currently not supported."));
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsAroundRank_Delegate);
	TriggerOnLeaderboardReadCompleteDelegates(false);
	return false;
}
bool FOnlineLeaderboardsGDK::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	if (!GDKSubsystem)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("ReadLeaderboards failed. GDKSubsystem is null."));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsAroundUser_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	FOnlineAsyncTaskManager* AsyncTaskManager = GDKSubsystem->GetAsyncTaskManager();
	if (!AsyncTaskManager)
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to get AsyncTaskManager from Online Subsystem"));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsAroundUser_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	//Attempt to retrieve an GDK user from the signed in users for making leaderboard requests.
	//This is necessary because the user actively requesting this data is not provided explicitly and not guaranteed to be in the Players list.
	FGDKContextHandle GDKContext;

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	TArray<FGDKUserHandle >& Users = Identity->GetCachedUsers();
	for (const FGDKUserHandle& User : Users)
	{
		XUserState UserState = XUserState::SignedOut;
		XUserGetState(User, &UserState);

		if (UserState != XUserState::SignedIn)
		{
			continue;
		}

		GDKContext = GDKSubsystem->GetGDKContext(User);
		if (GDKContext.IsValid())
		{
			break;
		}
	}

	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Failed to obtain an GDKContext to retrieve leaderboards data."));
		ReadObject->ReadState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_ReadLeaderboardsAroundUser_Delegate);
		TriggerOnLeaderboardReadCompleteDelegates(false);
		return false;
	}

	TArray<FUniqueNetIdRef> Players;
	Players.Add(Player);

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKReadLeaderboards>(
		GDKSubsystem,
		GDKContext,
		TArray<FOnlineLeaderboardReadRef>(),
		ReadObject,
		Players,
		XblSocialGroupType::None,
		ReadObject->SortedColumn,
		true,
		FOnReadLeaderboardsCompleteDelegate::CreateThreadSafeSP(this, &FOnlineLeaderboardsGDK::OnReadLeaderboardsComplete));

	return true;
}

void FOnlineLeaderboardsGDK::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	UNREFERENCED_PARAMETER(ReadObject);
	// NOOP
}

bool FOnlineLeaderboardsGDK::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	UNREFERENCED_PARAMETER(SessionName);
	UNREFERENCED_PARAMETER(Player);
	UNREFERENCED_PARAMETER(WriteObject);
	return false;
}

bool FOnlineLeaderboardsGDK::FlushLeaderboards(const FName& SessionName)
{
	UNREFERENCED_PARAMETER(SessionName);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineLeaderboardsGDK_FlushLeaderboards_Delegate);
	TriggerOnLeaderboardFlushCompleteDelegates(SessionName, true);
	return false;
}

bool FOnlineLeaderboardsGDK::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	UNREFERENCED_PARAMETER(SessionName);
	UNREFERENCED_PARAMETER(LeaderboardId);
	UNREFERENCED_PARAMETER(PlayerScores);
	return false;
}

#endif //WITH_GRDK