// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineLeaderboardInterface.h"

/**
* Delegate fired when Read Leaderboards is complete
*/
DECLARE_DELEGATE_OneParam(FOnReadLeaderboardsCompleteDelegate, bool);

/**
* Delegate fired when Read Leaderboards is complete
*/
DECLARE_DELEGATE_OneParam(FOnGetLeaderboardCompleteDelegate, bool);

/**
 * Interface definition for the online services leaderboard services 
 */
class FOnlineLeaderboardsGDK : public IOnlineLeaderboards, public TSharedFromThis< FOnlineLeaderboardsGDK, ESPMode::ThreadSafe>
{
private:
	/** Reference to the main Live subsystem */
	class FOnlineSubsystemGDK* GDKSubsystem;
	/** when requesting a friends leaderboard this is the requested sort order for the leaderboard */
	static const TCHAR* SortOrder;

public:

	virtual ~FOnlineLeaderboardsGDK() {}

	// IOnlineLeaderboards
	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;

	void OnReadLeaderboardsComplete(bool bSuccess);
	void OnReadLeaderboardsForFriendsComplete(bool bSuccess);

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	explicit FOnlineLeaderboardsGDK(FOnlineSubsystemGDK* InGDKSubsystem);
};

typedef TSharedPtr<FOnlineLeaderboardsGDK, ESPMode::ThreadSafe> FOnlineLeaderboardsGDKPtr;
