// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineMatchmakingInterfaceGDK.h"	
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystemGDKPrivate.h"

#include "OnlineSubsystemGDK.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineSubsystemSessionSettings.h"

#include "OnlineIdentityInterfaceGDK.h"
#include "SessionMessageRouter.h"
#include "OnlineSessionGDK.h"
#include "Misc/ScopeLock.h"

#include "AsyncTasks/OnlineAsyncTaskGDKCreateSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKJoinSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCreateMatchSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSubmitMatchTicket.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGameSessionReady.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCancelMatchmaking.h"
#include "AsyncTasks/OnlineAsyncTaskGDKDestroySession.h"

//////////////////////////////////////////////////////////////////////////
FOnlineMatchmakingInterfaceGDK::FOnlineMatchmakingInterfaceGDK(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
{
	OnSessionChangedDelegate = FOnSessionChangedDelegate::CreateRaw(this, &FOnlineMatchmakingInterfaceGDK::OnSessionChanged);
}

FOnlineMatchmakingInterfaceGDK::~FOnlineMatchmakingInterfaceGDK()
{
}

void FOnlineMatchmakingInterfaceGDK::OnSessionChanged(const FName SessionName, XblMultiplayerSessionChangeTypes Diff)
{
	UE_LOG_ONLINE(Log, TEXT("FOnlineMatchmakingInterfaceGDK::OnSessionChanged"));

	if ((Diff & XblMultiplayerSessionChangeTypes::MatchmakingStatusChange) == XblMultiplayerSessionChangeTypes::MatchmakingStatusChange)
	{
		OnMatchmakingStatusChanged(SessionName);
	}
	if ((Diff & XblMultiplayerSessionChangeTypes::MemberListChange) == XblMultiplayerSessionChangeTypes::MemberListChange)
	{
		OnMemberListChanged(SessionName);
	}
}

bool FOnlineMatchmakingInterfaceGDK::StartMatchmaking(const TArray<FUniqueNetIdRef>& LocalPlayers, const FName SessionName, const FOnlineSessionSettings& NewSessionSettings, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	if (LocalPlayers.Num() == 0)
	{
		UE_LOG_ONLINE(Error, TEXT("LocalPlayers was empty. At least one player is required for matchmaking."));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineMatchmakingInterfaceGDK_StartMatchmaking_Delegate);
		TriggerOnMatchmakingCompleteDelegates(SessionName, false);
		return false;
	}

	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (!MatchmakingTicket.IsValid())
	{
		MatchmakingTicket = MakeShared<FOnlineMatchTicketInfo>();
		AddMatchmakingTicket(SessionName, MatchmakingTicket);
	}

	MatchmakingTicket->MatchmakingState = EOnlineGDKMatchmakingState::CreatingMatchSession;
	MatchmakingTicket->SessionName = SessionName;
	MatchmakingTicket->SessionSearch = SearchSettings;
	MatchmakingTicket->SessionSettings = NewSessionSettings;

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKCreateMatchSession>(
		GDKSubsystem,
		LocalPlayers,
		SessionName,
		NewSessionSettings,
		SearchSettings
		);

	return true;
}

bool FOnlineMatchmakingInterfaceGDK::CancelMatchmaking(const int32 SearchingPlayerNum, const FName SessionName)
{
	EOnlineGDKMatchmakingState::Type CurrentState = GetMatchmakingState(SessionName);
	switch (CurrentState)
	{
	case EOnlineGDKMatchmakingState::WaitingForGameSession:
		break;
	default:
		UE_LOG_ONLINE(Log, TEXT("Can't cancel GDK Matchmaking in state %d"), CurrentState);
		return false;
	}

	UE_LOG_ONLINE(Log, TEXT("GDK Cancel Matchmaking %s"), *SessionName.ToString());

	FUniqueNetIdPtr UserId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(SearchingPlayerNum);
	if (!UserId.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineMatchmakingInterfaceGDK_CancelMatchmaking_Delegate);
		TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
		return true;
	}

	return CancelMatchmaking(*UserId, SessionName);
}

EOnlineGDKMatchmakingState::Type FOnlineMatchmakingInterfaceGDK::GetMatchmakingState(const FName SessionName) const
{
	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (MatchmakingTicket.IsValid())
	{
		return MatchmakingTicket->MatchmakingState;
	}

	// Fall back to session?
	return EOnlineGDKMatchmakingState::None;
}

void FOnlineMatchmakingInterfaceGDK::SetMatchmakingState(const FName SessionName, const EOnlineGDKMatchmakingState::Type State)
{
	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (MatchmakingTicket.IsValid())
	{
		MatchmakingTicket->MatchmakingState = State;
	}
}

bool FOnlineMatchmakingInterfaceGDK::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, const FName SessionName)
{
	UE_LOG_ONLINE(Log, TEXT("GDK Cancel Matchmaking %s"), *SessionName.ToString());

	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (!MatchmakingTicket.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineMatchmakingInterfaceGDK_CancelMatchmaking_Delegate);
		TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
		return true;
	}

	switch (MatchmakingTicket->MatchmakingState)
	{
		case EOnlineGDKMatchmakingState::None:
		{
			// Session is not in a valid state to be canceled
			RemoveMatchmakingTicket(SessionName);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineMatchmakingInterfaceGDK_CancelMatchmaking_Delegate);
			TriggerOnCancelMatchmakingCompleteDelegates(SessionName, false);
			return true;
		}
		
		case EOnlineGDKMatchmakingState::SubmittingInitialTicket: // Intentional fall-through
		case EOnlineGDKMatchmakingState::CreatingMatchSession:
		{
			RemoveMatchmakingTicket(SessionName);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineMatchmakingInterfaceGDK_CancelMatchmaking_Delegate);
			TriggerOnCancelMatchmakingCompleteDelegates(SessionName, true);
			return true;
		}

		default:
		{
			MatchmakingTicket->MatchmakingState = EOnlineGDKMatchmakingState::UserCancelled;
		}
	}

	FGDKContextHandle UserContext = GDKSubsystem->GetGDKContext(static_cast<const FUniqueNetIdGDK&>(SearchingPlayerId));

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKCancelMatchmaking>(
		GDKSubsystem,
		UserContext,
		SessionName,
		MatchmakingTicket);

	return true;
}

void FOnlineMatchmakingInterfaceGDK::AddMatchmakingTicket(const FName SessionName, const FOnlineMatchTicketInfoPtr InTicketInfo)
{
	FScopeLock ScopeLock(&TicketsLock);
	MatchmakingTickets.Add(SessionName, InTicketInfo);
}

void FOnlineMatchmakingInterfaceGDK::RemoveMatchmakingTicket(const FName SessionName)
{
	FScopeLock ScopeLock(&TicketsLock);

	if (FOnlineMatchTicketInfoPtr* FoundTicket = MatchmakingTickets.Find(SessionName))
	{
		GDKSubsystem->GetSessionMessageRouter()->ClearOnSessionChangedDelegate(OnSessionChangedDelegate, (*FoundTicket)->GetGDKSessionRef());
	}

	MatchmakingTickets.Remove(SessionName);
}

bool FOnlineMatchmakingInterfaceGDK::GetMatchmakingTicket(const FName SessionName, FOnlineMatchTicketInfoPtr& OutTicketInfo) const
{
	FScopeLock ScopeLock(&TicketsLock);

	const FOnlineMatchTicketInfoPtr* FoundTicket = MatchmakingTickets.Find(SessionName);

	if (FoundTicket)
	{
		OutTicketInfo = *FoundTicket;
	}

	return (FoundTicket != nullptr);
}

void FOnlineMatchmakingInterfaceGDK::SetTicketState(const FName SessionName, const EOnlineGDKMatchmakingState::Type State)
{
	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (MatchmakingTicket.IsValid())
	{
		MatchmakingTicket->MatchmakingState = State;
	}
}

void FOnlineMatchmakingInterfaceGDK::OnMatchmakingStatusChanged(const FName SessionName)
{
	check(IsInGameThread());

	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (!MatchmakingTicket.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineMatchmakingInterfaceGDK::OnMatchmakingStatusChanged - ticket doesn't exist or was destroyed before task ran"));
		return;
	}

	FGDKMultiplayerSessionHandle GDKSession = MatchmakingTicket->GetGDKSession();

	const XblMultiplayerMatchmakingServer* MatchmakingServer = XblMultiplayerSessionMatchmakingServer(GDKSession);

	XblMatchmakingStatus MatchStatus = MatchmakingServer->Status;

	switch (MatchStatus)
	{
	case XblMatchmakingStatus::Unknown:
		UE_LOG_ONLINE(Log, TEXT("  MatchStatus = Unknown"));
		break;

	case XblMatchmakingStatus::None:
		UE_LOG_ONLINE(Log, TEXT("  MatchStatus = None"));
		break;

	case XblMatchmakingStatus::Searching:
		UE_LOG_ONLINE(Log, TEXT("  MatchStatus = Searching"));
		break;

	case XblMatchmakingStatus::Expired:
		UE_LOG_ONLINE(Log, TEXT("  MatchStatus = Expired"));

		SubmitMatchingTicket(
			GDKSession,
			SessionName,
			false);
		break;

	case XblMatchmakingStatus::Found:
		{
			UE_LOG_ONLINE(Log, TEXT("  MatchStatus = Found"));

			// Join the target session so we can do QoS. If this isn't a match session (we're
			// advertising an existing game session), we've already joined.
			FOnlineSessionGDKPtr SessionInterface = GDKSubsystem->GetSessionInterfaceGDK();
			FNamedOnlineSessionPtr ExistingNamedSession = SessionInterface->GetNamedSessionPtr(SessionName);
			if (!ExistingNamedSession.IsValid() || SessionName == NAME_PartySession)
			{
				const XblMultiplayerSessionReference* TargetSessionReference = &MatchmakingServer->TargetSessionRef;

				GDKSubsystem->GetSessionMessageRouter()->ClearOnSessionChangedDelegate(OnSessionChangedDelegate, XblMultiplayerSessionSessionReference(GDKSession));
				MatchmakingTicket->RefreshGDKInfo(TargetSessionReference);
				MatchmakingTicket->SetLastDiffedSession(FGDKMultiplayerSessionHandle());
				MatchmakingTicket->MatchmakingState = EOnlineGDKMatchmakingState::JoiningGameSession;

				FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKSession);
				check(GDKContext.IsValid());

				// Create a named session from the search result data
				FNamedOnlineSessionRef NamedSession = SessionInterface->AddNamedSessionRef(NAME_GameSession, MatchmakingTicket->SessionSettings);
				NamedSession->HostingPlayerNum = INDEX_NONE;
				{
					// Party tickets should now follow the Matchmade session for backfilling
					MatchmakingTicket->SessionName = NAME_GameSession;

					FScopeLock ScopeLock(&TicketsLock);
					MatchmakingTickets.Remove(SessionName);
					MatchmakingTickets.Add(NAME_GameSession, MatchmakingTicket);
				}

				// Record the game session URI so invites can pass along the target session.
				if (SessionName == NAME_PartySession && ExistingNamedSession.IsValid())
				{
					ExistingNamedSession->SessionSettings.Set(SETTING_GAME_SESSION_URI, FOnlineSessionMpsdGDK::SessionReferenceToUri(*TargetSessionReference), EOnlineDataAdvertisementType::ViaOnlineService);
					SessionInterface->UpdateSession(SessionName, ExistingNamedSession->SessionSettings, true);
				}
				
				NamedSession->SessionInfo = MakeShared<FOnlineSessionInfoMpsdGDK>(GDKSession);				

				GDKSubsystem->GetSessionMessageRouter()->AddOnSessionChangedDelegate(SessionInterface->GetMpsdImpl()->OnSessionChangedDelegate, TargetSessionReference);

				UE_LOG_ONLINE(Log, TEXT("Session Found: %ls %ls"), UTF8_TO_TCHAR(TargetSessionReference->SessionTemplateName), UTF8_TO_TCHAR(TargetSessionReference->SessionName));

				const bool bSessionIsMatchmakingResult = true;
				const bool bInSetActivity = true;

				GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKJoinSession>(
					GDKSubsystem,
					GDKContext,
					TargetSessionReference,
					NamedSession,
					SessionInterface->MAX_RETRIES,
					bSessionIsMatchmakingResult,
					bInSetActivity,
					TOptional<FString>());
			}
			break;
		}

	case XblMatchmakingStatus::Canceled:
		if (MatchmakingTicket.IsValid() && MatchmakingTicket->MatchmakingState != EOnlineGDKMatchmakingState::UserCancelled)
		{
			SubmitMatchingTicket(
				GDKSession,
				SessionName,
				false);
		}

		UE_LOG_ONLINE(Log, TEXT("  MatchStatus = Canceled"));
		break;

	default:
		UE_LOG_ONLINE(Warning, TEXT("FOnlineMatchmakingInterfaceGDK::OnMatchmakingStatusChanged - Got unexpected MatchmakingStatus: %u"), (uint32)MatchStatus);
		break;
	}
}

void FOnlineMatchmakingInterfaceGDK::SubmitMatchingTicket(
	FGDKMultiplayerSessionHandle InGDKSession,
	const FName SessionName,
	const bool bCancelExistingTicket)
{
	const XblMultiplayerSessionReference* SessionRef = XblMultiplayerSessionSessionReference(InGDKSession);
	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	// Start matchmaking should have been called first...
	if (!MatchmakingTicket.IsValid())
	{
		return;
	}

	GDKSubsystem->GetSessionMessageRouter()->AddOnSessionChangedDelegate(OnSessionChangedDelegate, SessionRef);

	FNamedOnlineSessionPtr NamedSession = GDKSubsystem->GetSessionInterfaceGDK()->GetNamedSessionPtr(SessionName);
	if (NamedSession.IsValid() && !NamedSession->SessionSettings.bShouldAdvertise)
	{
		return;
	}

	// Don't submit if not matchmaking
	if (MatchmakingTicket->MatchmakingState == EOnlineGDKMatchmakingState::None)
	{
		return;
	}

	// When the game is active, only the host should be submitting tickets
	if ((MatchmakingTicket->MatchmakingState == EOnlineGDKMatchmakingState::Active) && NamedSession.IsValid() && NamedSession->OwningUserId.IsValid())
	{
		if (!GDKSubsystem->IsLocalPlayer(*NamedSession->OwningUserId))
		{
			return;
		}
	}

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = NamedSession.IsValid() ? StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo) : nullptr;

	if (GDKInfo.IsValid() && GDKInfo->GetGDKMultiplayerSession().IsValid())
	{
		FGDKMultiplayerSessionHandle SessionHandle = GDKInfo->GetGDKMultiplayerSession();
		uint64 NumMembers = 0;
		const XblMultiplayerSessionMember* Members;
		XblMultiplayerSessionMembers(SessionHandle, &Members, &NumMembers);
		const uint64 CurrentSessionSize = NumMembers;
		if (CurrentSessionSize >= NamedSession->SessionSettings.NumPublicConnections)
		{
			UE_LOG_ONLINE(Log, TEXT("FOnlineSessionGDK::SubmitMatchingTicket: Maximum players reached for this session. No longer matchmaking. Size: %lld"), CurrentSessionSize);
			return;
		}
	}

	TSharedPtr<FOnlineSessionSearch> SearchSettings = MatchmakingTicket->SessionSearch;
	check(SearchSettings.IsValid());

	FString MatchHopperName;
	SearchSettings->QuerySettings.Get(SETTING_MATCHING_HOPPER, MatchHopperName);
	MatchmakingTicket->HopperName = MatchHopperName;

	FString MatchTicketAttributes;
	const FOnlineSessionSetting* AttributesSetting =  MatchmakingTicket->SessionSettings.Settings.Find(SETTING_MATCHING_ATTRIBUTES);
	if (AttributesSetting)
	{
		AttributesSetting->Data.GetValue(MatchTicketAttributes);
	}

	uint64 PlatformTimeout = (uint64)(1000 * SearchSettings->TimeoutInSeconds); // Convert timeout to duration in milli (unit isn't documented, this is an assumption)

	XblPreserveSessionMode Mode = XblPreserveSessionMode::Always;

	bool bCreateNewSession = false;
	if (NamedSession.IsValid() && NamedSession->SessionSettings.Get(SETTING_MATCHING_PRESERVE_SESSION, bCreateNewSession))
	{
		bCreateNewSession = !bCreateNewSession;
	}

	// If we don't have a session we remember, we must have made a match session. Set the ticket to instruct the matchamking service to greate a game session when it finds us a match
	if (!GDKInfo.IsValid() || !GDKInfo->GetGDKMultiplayerSession().IsValid() || bCreateNewSession)
	{
		Mode = XblPreserveSessionMode::Never;
	}

	uint64 HostUserId;
	if (SUCCEEDED(XUserGetId(MatchmakingTicket->GetHostUser(), &HostUserId)))
	{		
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKSubmitMatchTicket>(
			GDKSubsystem,
			HostUserId,
			SessionName,
			InGDKSession,
			MatchHopperName,
			MatchTicketAttributes,
			PlatformTimeout,
			Mode,
			bCancelExistingTicket);
	}
}

void FOnlineMatchmakingInterfaceGDK::OnMemberListChanged(const FName SessionName)
{
	check(IsInGameThread());

	FOnlineMatchTicketInfoPtr MatchmakingTicket;
	GetMatchmakingTicket(SessionName, MatchmakingTicket);

	if (!MatchmakingTicket.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineMatchmakingInterfaceGDK::OnMatchmakingStatusChanged - session doesn't exist or was destroyed before task ran"));
		return;
	}

	if (GDKSubsystem->GetSessionInterfaceGDK()->GetNamedSessionPtr(SessionName).IsValid())
	{
		// If we have a NamedSession, let the Session Interface figure this out for us, since both are listening.
		return;
	}

	// Cancel the current match ticket and re-advertise with the new number of open slots.
	// If the session isn't doing matchmaking, this is a no-op.
	SubmitMatchingTicket(MatchmakingTicket->GetGDKSession(), SessionName, true);
}

FOnlineMatchTicketInfoPtr FOnlineMatchmakingInterfaceGDK::GetMatchTicketForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef)
{
	FScopeLock ScopeLock(&TicketsLock);
	FOnlineMatchTicketInfoPtr FoundTicket;

	for (TPair<FName, FOnlineMatchTicketInfoPtr>& TicketPair : MatchmakingTickets)
	{
		FOnlineMatchTicketInfoPtr& Ticket = TicketPair.Value;
		if (!Ticket.IsValid())
		{
			continue;
		}

		if (const XblMultiplayerSessionReference* TicketSessionRef = Ticket->GetGDKSessionRef())
		{
			if (TicketSessionRef &&
				FOnlineSubsystemGDK::AreSessionReferencesEqual(TicketSessionRef, GDKSessionRef))
			{
				FoundTicket = Ticket;
				break;
			}
		}
	}

	return FoundTicket;
}

#endif //WITH_GRDK