// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SessionMessageRouter.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineDelegateMacros.h"

#include "Interfaces/OnlineSessionInterface.h"

class FOnlineSubsystemGDK;
class FOnlineSessionSettings;
class FOnlineSessionSearch;

class FOnlineMatchmakingInterfaceGDK
{
public:
	FOnlineMatchmakingInterfaceGDK(FOnlineSubsystemGDK* InSubsystem);
	virtual ~FOnlineMatchmakingInterfaceGDK();

PACKAGE_SCOPE:

	bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, const FName SessionName, const FOnlineSessionSettings& NewSessionSettings, const TSharedRef<FOnlineSessionSearch>& SearchSettings);

	bool CancelMatchmaking(const int32 SearchingPlayerNum, const FName SessionName);
	bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, const FName SessionName);
	
	/**
	 * Matchmaking related APIs
	 */
	void AddMatchmakingTicket(const FName SessionName, const FOnlineMatchTicketInfoPtr TicketInfo);
	void RemoveMatchmakingTicket(const FName SessionName );
	bool GetMatchmakingTicket(const FName SessionName, FOnlineMatchTicketInfoPtr& OutTicketInfo) const;
	void SetTicketState(const FName SessionName, const EOnlineGDKMatchmakingState::Type State);

	/** Resubmit a matchmaking ticket if necessary */
	void SubmitMatchingTicket(
		FGDKMultiplayerSessionHandle InGDKSession,
		const FName SessionName,
		const bool bCancelExistingTicket);

	/** Look up the ticket corresponding to a Live session reference */
	FOnlineMatchTicketInfoPtr GetMatchTicketForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef);

	/**
	 * Delegate fired when the cloud matchmaking has completed
	 *
	 * @param SessionName The name of the session that was found via matchmaking
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnMatchmakingComplete, FName, bool);

		/**
	 * Delegate fired when the cloud matchmaking has been canceled
	 *
	 * @param SessionName the name of the session that was canceled
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnCancelMatchmakingComplete, FName, bool);

private:

	EOnlineGDKMatchmakingState::Type GetMatchmakingState(FName SessionName) const;
	void SetMatchmakingState(const FName SessionName, const EOnlineGDKMatchmakingState::Type State);

	void OnSessionChanged(const FName SessionName, XblMultiplayerSessionChangeTypes Diff);
	
	/** Handle changes to matchmaking status */
	void OnMatchmakingStatusChanged(const FName SessionName);
	void OnMemberListChanged(const FName SessionName);

	FOnSessionChangedDelegate OnSessionChangedDelegate;

	typedef TMap<FName, FOnlineMatchTicketInfoPtr> TicketInfoMap;
	mutable FCriticalSection	TicketsLock;
	TicketInfoMap				MatchmakingTickets;

	FOnlineSubsystemGDK* GDKSubsystem;
};

typedef TSharedPtr<FOnlineMatchmakingInterfaceGDK, ESPMode::ThreadSafe> FOnlineMatchmakingInterfaceGDKPtr;
