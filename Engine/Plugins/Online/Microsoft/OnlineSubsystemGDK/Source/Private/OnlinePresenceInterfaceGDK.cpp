// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineFriendsInterfaceGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDK.h"
#include "Async/Async.h"
#include "Online/OnlineSessionNames.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryPresence.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryActivitiesForUsers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaGetActivities.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSetPresence.h"
#include "Misc/ConfigCacheIni.h"

#include "Interfaces/OnlineEventsInterface.h"

extern TAutoConsoleVariable<bool> CVarXboxMpaEnabled;


void SetPresencePropertyFromStat(FPresenceProperties& PresenceProperties, const FString& StatName, const FString& StatType, const FString& StatValue)
{
	if(StatType == TEXT("uint64"))
	{
		int64 Value = 0;
		LexFromString(Value, *StatValue);
		if (Value > TNumericLimits<int32>::Lowest() && Value < TNumericLimits<int32>::Max())
		{
			PresenceProperties.FindOrAdd(StatName).SetValue((int32)Value);
		}
		else
		{
			PresenceProperties.FindOrAdd(StatName).SetValue(Value);
		}
	}
	else if (StatType == TEXT("double"))
	{
		double Value = 0.0;
		LexFromString(Value, *StatValue);
		if (Value > TNumericLimits<float>::Lowest() && Value < TNumericLimits<float>::Max())
		{
			PresenceProperties.FindOrAdd(StatName).SetValue((float)Value);
		}
		else
		{
			PresenceProperties.FindOrAdd(StatName).SetValue(Value);
		}
	}
	else if (StatType == TEXT("string"))
	{
		PresenceProperties.FindOrAdd(StatName).SetValue(StatValue);
	}
	else
	{
		//StatType == TEXT("datetime")
		//StatType == TEXT("othertype")
		UE_LOG_ONLINE_PRESENCE(Log, TEXT("Presence stat %s has unsupported type %s. Adding as a string. Value: %s"),
			*StatName, *StatType, *StatValue);
		PresenceProperties.FindOrAdd(StatName).SetValue(StatValue);

	}
}

void FOnlineUserPresenceGDK::SetStatusPropertiesFromStatistics(const XblUserStatisticsResult& StatsResult)
{
	// Add any stats, if requested and available, to the PresenceProperties
	const ANSICHAR* Scid = nullptr;
	XblGetScid(&Scid);
	FString ServiceConfigurationId = UTF8_TO_TCHAR(Scid);

	for (uint32 i=0; i<StatsResult.serviceConfigStatisticsCount ; ++i)
	{
		const XblServiceConfigurationStatistic& ServiceConfigStat= StatsResult.serviceConfigStatistics[i];
		FString ServiceConfigId = UTF8_TO_TCHAR(ServiceConfigStat.serviceConfigurationId);
		if (ServiceConfigId.Equals(ServiceConfigurationId))
		{
			for (uint32 j=0; j< ServiceConfigStat.statisticsCount; ++j)
			{
				const XblStatistic& Stat = ServiceConfigStat.statistics[j];
				const FString StatName(UTF8_TO_TCHAR(Stat.statisticName));
				const FString StatType(UTF8_TO_TCHAR(Stat.statisticType));
				const FString StatValue(UTF8_TO_TCHAR(Stat.value));
				SetPresencePropertyFromStat(Status.Properties, StatName, StatType, StatValue);
			}
		}
	}	
}

FString FOnlinePresenceGDK::GetPresenceIdString(const FOnlineUserPresenceStatus& Status)
{
	// Only support the default key for now, as a string.
	const FVariantData* PresenceId = Status.Properties.Find(DefaultPresenceKey);
	if (!PresenceId || PresenceId->GetType() != EOnlineKeyValuePairDataType::String)
	{
		PresenceId = nullptr;
	}

	FString PresenceIdString;
	if (PresenceId)
	{
		PresenceId->GetValue(PresenceIdString);
	}
	else
	{
		PresenceIdString = Status.StatusStr;
	}

	return PresenceIdString;
}

FOnlinePresenceGDK::FOnlinePresenceGDK(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
{
	check(GDKSubsystem);
	GConfig->GetFloat(TEXT("OnlineSubsystemGDK"), TEXT("SessionPresencePollDelay"), SessionPresencePollDelay, GEngineIni);
}

void FOnlinePresenceGDK::SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef UserGDK = FUniqueNetIdGDK::Cast(User);

	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (!Identity.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("SetPresence failed. Bad Identity interface."));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_SetPresence_Delegate);
			Delegate.ExecuteIfBound(*UserGDK, false);
		});
		return;
	}

	// Find the FGDKUserHandle associated with this unique net id.
	FGDKUserHandle PresenceUser = FGDKUserHandle(Identity->GetUserForUniqueNetId(*UserGDK));
	if (!PresenceUser)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("SetPresence failed. No local user found."));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_SetPresence_Delegate);
			Delegate.ExecuteIfBound(*UserGDK, false);
		});
		return;
	}

	const FString PresenceIdString = GetPresenceIdString(Status);

	// if we have no current status to set, return
	if (PresenceIdString.IsEmpty())
	{
		GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("SetPresence failed. No presence value was set."));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_SetPresence_Delegate);
			Delegate.ExecuteIfBound(*UserGDK, false);
		});
		return;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(PresenceUser);
	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("SetPresence failed. No GDK Context for local user."));
			Delegate.ExecuteIfBound(*UserGDK, false);
		});
		return;
	}

	// Get the cached presence status so we can skip updates if it hasn't changed
	const FOnlineUserPresenceStatus* const CurrentUsersPresencePtr = LocalUserPresenceCache.Find(UserGDK);

	FOnlineEventParms AllTriggeredParams;

	// before setting the presence queue up the stat events to trigger
	IOnlineEventsPtr EventsInterface = GDKSubsystem->GetEventsInterface();
	if (EventsInterface.IsValid())
	{
		for (const FPresenceProperties::ElementType& PropertyPair : Status.Properties)
		{
			static const FString EventPrefix(TEXT("Event_"));
			if (PropertyPair.Key.StartsWith(EventPrefix, ESearchCase::IgnoreCase))
			{
				// Skip this event if the property value hasn't changed.
				if (CurrentUsersPresencePtr != nullptr)
				{
					const FVariantData* const FoundProperty = CurrentUsersPresencePtr->Properties.Find(PropertyPair.Key);
					if ((FoundProperty != nullptr) && (*FoundProperty == PropertyPair.Value))
					{
						UE_LOG_ONLINE_PRESENCE(VeryVerbose, TEXT("Skipping updating presence stat %s, as it has the same value"), *PropertyPair.Key);
						continue;
					}
				}

				FOnlineEventParms Params;
				Params.Emplace(TEXT("Value"), PropertyPair.Value);
				AllTriggeredParams.Emplace(*PropertyPair.Key, PropertyPair.Value);

				UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Updating presence stat %s to value %s"), *PropertyPair.Key, *PropertyPair.Value.ToString());

				EventsInterface->TriggerEvent(*UserGDK, *PropertyPair.Key, Params);
			}
		}
	}

	// Ensure we're not posting our current presence again
	FString CurrentPresenceIdString;
	if (CurrentUsersPresencePtr != nullptr)
	{
		CurrentPresenceIdString = GetPresenceIdString(*CurrentUsersPresencePtr);
	}

	// Always save the latest presence status so that properties are up to date
	LocalUserPresenceCache.Add(UserGDK, Status);

	if (CurrentPresenceIdString.Equals(PresenceIdString))
	{
		GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
		{
			constexpr const bool bWasSuccessful = true;
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_SetPresence_Delegate);
			Delegate.ExecuteIfBound(*UserGDK, bWasSuccessful);
		});
		return;
	}

	UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Updating presence to %s"), *PresenceIdString);
	if (UE_LOG_ACTIVE(LogOnlinePresence, VeryVerbose))
	{
		for (const FOnlineEventParms::ElementType& Pair : AllTriggeredParams)
		{
			UE_LOG_ONLINE_PRESENCE(VeryVerbose, TEXT("Set Presence Key '%s' to '%s'"), *Pair.Key, *Pair.Value.ToString());
		}
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKSetPresence>(
		GDKSubsystem,
		GDKContext,
		Status,
		PresenceIdString,
		FOnSetGDKPresenceCompleteDelegate::CreateThreadSafeSP(this, &FOnlinePresenceGDK::HandleSetGDKPresenceComplete, UserGDK, CurrentPresenceIdString, Delegate));
}

void FOnlinePresenceGDK::HandleSetGDKPresenceComplete(bool bSuccessful, const FOnlineUserPresenceStatus& Status, const FUniqueNetIdGDKRef User, const FString CurrentPresenceIdString, const FOnPresenceTaskCompleteDelegate Delegate)
{
	if(bSuccessful)
	{
		// Save our new presence into our cache
		GDKSubsystem->ExecuteNextTick([this, User, Status, CurrentPresenceIdString, Delegate]()
		{
			TSharedRef<FOnlineUserPresenceGDK> Presence = MakeShared<FOnlineUserPresenceGDK>();
			Presence->bIsOnline = true;
			Presence->bIsPlaying = true;
			Presence->bIsPlayingThisGame = true;
			Presence->Status = Status;
			PresenceCache.Add(User, Presence);

			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_HandleSetGDKPresenceComplete_Delegate);
			Delegate.ExecuteIfBound(*User, true);
		});
		UE_LOG_ONLINE_PRESENCE(Display, TEXT("SetPresenceAsync succeeded."));
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([User, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_HandleSetGDKPresenceComplete_Delegate);
			Delegate.ExecuteIfBound(*User, false);
		});
		UE_LOG_ONLINE_PRESENCE(Warning, TEXT("SetPresenceAsync failed."));				
	}
}

void FOnlinePresenceGDK::QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef UserGDK = FUniqueNetIdGDK::Cast(User);

	// This interface is wrong and doesn't specify the user to query from, or the users to query, either way.
	// For now we'll treat the User as the person to query, and later there'll be an array of users to query
	// and we'll query them FROM user instead.

	// Try to find a user to query presence from
	FGDKContextHandle GDKContext;
	{
		for (int32 ControllerIndex = 0; ControllerIndex < MAX_LOCAL_PLAYERS; ++ControllerIndex)
		{
			GDKContext = GDKSubsystem->GetGDKContext(ControllerIndex);
			if (GDKContext.IsValid())
			{
				break;
			}
		}

		if (!GDKContext)
		{
			GDKSubsystem->ExecuteNextTick([Delegate, UserGDK]()
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Could not query presence, could not find an GDK context to use"));
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_QueryPresence_Delegate);
				Delegate.ExecuteIfBound(*UserGDK, false);
			});
			return;
		}
	}

	// Subscribe to presence updates
	if (!IsSubscribedToPresenceUpdates(UserGDK))
	{
		// The following call may be blocking so we'll run it out of the game thread
		FOnlinePresenceGDKPtr PresencePtr = GDKSubsystem->GetPresenceGDK();
		AsyncTask(ENamedThreads::AnyThread, [PresencePtr, GDKContext, UserGDK]()
		{
			if (GDKContext.IsValid())
			{
				TArray<uint64> UserQueryXuids;
				UserQueryXuids.Add(UserGDK->ToUint64());

				HRESULT Result = XblPresenceTrackUsers(GDKContext, UserQueryXuids.GetData(), UserQueryXuids.Num());
				if (SUCCEEDED(Result))
				{
					TArray<FUniqueNetIdGDKRef> GDKUsers;
					GDKUsers.Add(UserGDK);

					PresencePtr->AddPresenceUpdateSubscriptions(GDKUsers);
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::QueryPresence] Error from XblPresenceTrackUsers, with code 0x%08X."), Result);
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::QueryPresence] GDKContext was invalid."));
			}
		});

		// We don't need to wait for the presence tracking to be active to query the presence
	}

	// Launch the task that will do the actual query
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKQueryPresence>(GDKSubsystem, GDKContext, UserGDK, Delegate);
}

EOnlineCachedResult::Type FOnlinePresenceGDK::GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	TSharedRef<FOnlineUserPresenceGDK>* Presence = PresenceCache.Find(FUniqueNetIdGDK::Cast(User));
	if (!Presence)
	{
		return EOnlineCachedResult::NotFound;
	}

	OutPresence = *Presence;
	return EOnlineCachedResult::Success;
}

EOnlineCachedResult::Type FOnlinePresenceGDK::GetCachedPresenceForApp(const FUniqueNetId& /*LocalUserId*/, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	EOnlineCachedResult::Type Result = EOnlineCachedResult::NotFound;

	if (GDKSubsystem->GetAppId() == AppId)
	{
		Result = GetCachedPresence(User, OutPresence);
	}

	return Result;
}

void FOnlinePresenceGDK::Tick(const float DeltaTime)
{
	static float TimeWaited = 0.0f;	
	TimeWaited += DeltaTime;
	if (TimeWaited >= SessionPresencePollDelay)
	{
		// Doesn't matter who makes the queries, so just grab first user we can find
		FGDKContextHandle GDKContext;
		for (int32 ControllerIndex = 0; ControllerIndex < MAX_LOCAL_PLAYERS; ++ControllerIndex)
		{
			GDKContext = GDKSubsystem->GetGDKContext(ControllerIndex);
			if (GDKContext.IsValid())
			{
				break;
			}
		}
		if (GDKContext)
		{
			TArray<uint64> UsersToQuery;
			for (auto UserToQuery : PresenceCache)
			{
				const FUniqueNetIdGDKRef& GDKUserToQuery = StaticCastSharedRef<const FUniqueNetIdGDK>(UserToQuery.Key);
				UsersToQuery.Add(GDKUserToQuery->ToUint64());
			}
			if (UsersToQuery.Num() != 0)
			{
				if (CVarXboxMpaEnabled.GetValueOnAnyThread())
				{
					GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKMpaGetActivities>(
						GDKSubsystem,
						GDKContext,
						UsersToQuery,
						FOnlineAsyncTaskGDKMpaGetActivities::FOnComplete::CreateThreadSafeSP(this, &FOnlinePresenceGDK::OnFriendSessionUpdated));
				}
				else
				{
					GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryActivitiesForUsers>(
						GDKSubsystem,
						GDKContext,
						UsersToQuery,
						FOnGDKQueryActivitiesForUsersComplete::CreateThreadSafeSP(this, &FOnlinePresenceGDK::OnFriendSessionUpdated));
				}
			}
		}			

		TimeWaited = 0.0f;
	}	
}

TArray<FString> FOnlinePresenceGDK::GetAutoSubscribePresenceStatNames()
{
	TArray<FString> AutoSubscribePresenceStats;
	GConfig->GetArray(TEXT("OnlineSubsystemGDK"), TEXT("AutoSubscribePresenceStats"), AutoSubscribePresenceStats, GEngineIni);

	return AutoSubscribePresenceStats;
}

void FOnlinePresenceGDK::ProcessStatUpdate(const FUniqueNetIdGDKRef& LocalUserId, const FUniqueNetIdGDKRef& UpdatedUserId, const FString& StatName, const FString& StatValue, const FString& StatType)
{
	if (IsSubscribedToStatUpdates(LocalUserId, UpdatedUserId, StatName))
	{
		// A standard auto-subscribed presence stat changed, so update the changed user's presence property accordingly
		if (TSharedRef<FOnlineUserPresenceGDK>* CachedUserPresence = PresenceCache.Find(UpdatedUserId))
		{
			UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Processing update to stat [%s=%s] for LocalPlayer: %s Friend: %s"), *StatName, *StatValue, *LocalUserId->ToString(), *UpdatedUserId->ToString());

			SetPresencePropertyFromStat((*CachedUserPresence)->Status.Properties, StatName, StatType, StatValue);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_ProcessStatUpdate_Delegate);
			TriggerOnPresenceReceivedDelegates(*UpdatedUserId, *CachedUserPresence);
		}
		else
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Received update to stat [%s=%s] for LocalPlayer: %s Friend: %s, but we have no cached presence to update for the friend"), *StatName, *StatValue, *LocalUserId->ToString(), *UpdatedUserId->ToString());
		}		
	}
	else
	{
		UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Ingoring updated stat [%s=%s] for LocalPlayer: %s Friend: %s"), *StatName, *StatValue, *LocalUserId->ToString(), *UpdatedUserId->ToString());
	}
}

void FOnlinePresenceGDK::OnPresenceDeviceChanged(const FUniqueNetIdGDK& UserGDK, XblPresenceDeviceType DeviceType, bool bIsUserLoggedOnDevice)
{
	TSharedRef<FOnlineUserPresenceGDK>* UserPresencePtr = PresenceCache.Find(UserGDK.AsShared());
	if (!UserPresencePtr)
	{
		return;
	}

	// Update Presence values
	TSharedRef<FOnlineUserPresenceGDK>& Presence = *UserPresencePtr;
	Presence->bIsOnline = bIsUserLoggedOnDevice;
	Presence->Status.State = bIsUserLoggedOnDevice ? EOnlinePresenceState::Online : EOnlinePresenceState::Offline;

	// Attempt to update presence for anyone we have on our friends list
	FOnlineFriendsGDKPtr FriendsInt = GDKSubsystem->GetFriendsGDK();
	if (FriendsInt.IsValid())
	{
		FriendsInt->OnUserPresenceUpdate(UserGDK, Presence);
	}

	// Trigger presence delegate
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_OnPresenceDeviceChanged_Delegate);
	TriggerOnPresenceReceivedDelegates(UserGDK, Presence);
}

void FOnlinePresenceGDK::OnPresenceTitleChanged(const FUniqueNetIdGDK& UserGDK, uint32 TitleId, XblPresenceTitleState TitleState)
{
	TSharedRef<FOnlineUserPresenceGDK>* UserPresencePtr = PresenceCache.Find(UserGDK.AsShared());
	if (!UserPresencePtr)
	{
		return;
	}

	// Update Presence values
	TSharedRef<FOnlineUserPresenceGDK>& Presence = *UserPresencePtr;
	Presence->bIsPlaying = (TitleState == XblPresenceTitleState::Started);
	Presence->bIsPlayingThisGame = Presence->bIsPlaying && (TitleId == GDKSubsystem->TitleId);

	// Attempt to update presence for anyone we have on our friends list
	FOnlineFriendsGDKPtr FriendsInt = GDKSubsystem->GetFriendsGDK();
	if (FriendsInt.IsValid())
	{
		FriendsInt->OnUserPresenceUpdate(UserGDK, Presence);
	}

	// Trigger presence delegate
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_OnPresenceTitleChanged_Delegate);
	TriggerOnPresenceReceivedDelegates(UserGDK, Presence);
}
void FOnlinePresenceGDK::OnFriendSessionUpdated(bool bWasSuccessful, TArray<FOnlineSession> OnlineSessions)
{
	UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("OnFriendSessionUpdated bWasSuccessful: %d UserCount: %d"), bWasSuccessful, OnlineSessions.Num());

	if (!bWasSuccessful)
	{
		return;
	}

	for (const FOnlineSession& Session : OnlineSessions)
	{
		// Update Presence values
		TSharedRef<FOnlineUserPresenceGDK>* UserPresencePtr = PresenceCache.Find(Session.OwningUserId.ToSharedRef());
		if (UserPresencePtr)
		{
			TSharedRef<FOnlineUserPresenceGDK>& Presence = *UserPresencePtr;

			FString ConnectionString;
			Session.SessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

			Presence->bIsJoinable = !ConnectionString.IsEmpty();
			Presence->SessionId = FUniqueNetIdString::Create(ConnectionString, GDK_SUBSYSTEM);

			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_OnFriendSessionUpdated_Delegate);
			TriggerOnPresenceReceivedDelegates(*Session.OwningUserId, Presence);
		}
	}

	// Attempt to update presence for anyone we have on our friends list
	FOnlineFriendsGDKPtr FriendsInt = GDKSubsystem->GetFriendsGDK();
	if (FriendsInt.IsValid())
	{
		FriendsInt->OnUsersSessionPresenceUpdate(OnlineSessions);
	}

}
void FOnlinePresenceGDK::OnFriendSessionUpdated(const FOnlineError& ErrorResult, const FOnlineGDKActivitiesResultMap& ActivitiesResults)
{
	UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("OnFriendSessionUpdated bWasSuccessful: %d UserCount: %d"), ErrorResult.bSucceeded, ActivitiesResults.Num());

	if (!ErrorResult.bSucceeded)
	{
		return;
	}

	for (const TPair<FUniqueNetIdRef, FUniqueNetIdPtr >& ActivityResult : ActivitiesResults)
	{
		FUniqueNetIdPtr SessionId = ActivityResult.Value;

		// Update Presence values
		TSharedRef<FOnlineUserPresenceGDK>* UserPresencePtr = PresenceCache.Find(ActivityResult.Key);
		if (UserPresencePtr)
		{
			TSharedRef<FOnlineUserPresenceGDK>& Presence = *UserPresencePtr;

			Presence->bIsJoinable = SessionId.IsValid();
			Presence->SessionId = SessionId;

			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePresenceGDK_OnFriendSessionUpdated_Delegate);
			TriggerOnPresenceReceivedDelegates(*ActivityResult.Key, Presence);
		}
	}

	// Attempt to update presence for anyone we have on our friends list
	FOnlineFriendsGDKPtr FriendsInt = GDKSubsystem->GetFriendsGDK();
	if (FriendsInt.IsValid())
	{
		FriendsInt->OnUsersSessionPresenceUpdate(ActivitiesResults);
	}
}

void FOnlinePresenceGDK::UnsubscribeFromAllPresenceUpdatesForUser(FGDKContextHandle GDKContext)
{
	if (GDKContext.IsValid())
	{
		// We need to convert the Ids to GDK type before the API call
		TArray<FUniqueNetIdGDKRef> GDKUsers;
		for (const FUniqueNetIdRef& UserId : PresenceSubscriptionSet)
		{
			GDKUsers.Add(FUniqueNetIdGDK::Cast(*UserId));
		}

		// The following call may be blocking so we'll run it out of the game thread
		FOnlinePresenceGDKPtr PresencePtr = GDKSubsystem->GetPresenceGDK();
		AsyncTask(ENamedThreads::AnyThread, [PresencePtr, GDKContext, GDKUsers]()
		{
			if (GDKContext.IsValid())
			{
				TArray<uint64> UserQueryXuids;
				for (const FUniqueNetIdGDKRef& GDKUser : GDKUsers)
				{
					UserQueryXuids.Add(GDKUser->ToUint64());
				}

				HRESULT Result = XblPresenceStopTrackingUsers(GDKContext, UserQueryXuids.GetData(), UserQueryXuids.Num());
				if (SUCCEEDED(Result))
				{
					PresencePtr->RemovePresenceUpdateSubscriptions(GDKUsers);
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::UnsubscribeFromAllPresenceUpdatesForUser] Error from XblPresenceTrackUsers, with code 0x%08X."), Result);
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::UnsubscribeFromAllPresenceUpdatesForUser] GDKContext was invalid."));
			}
		});
	}
}

bool FOnlinePresenceGDK::IsSubscribedToPresenceUpdates(const FUniqueNetIdGDKRef& GDKUserId) const
{
	return PresenceSubscriptionSet.Contains(GDKUserId);
}

void FOnlinePresenceGDK::AddPresenceUpdateSubscriptions(const TArray<FUniqueNetIdGDKRef>& GDKUserIds)
{
	// This method should not be called from the game thread, since it's a continuation to potentially blocking API calls
	check(!IsInGameThread());

	AsyncTask(ENamedThreads::GameThread, [this, GDKUserIds]()
		{
			for (const FUniqueNetIdGDKRef& GDKUserId : GDKUserIds)
			{
				PresenceSubscriptionSet.Emplace(GDKUserId);
			}
		});
}

void FOnlinePresenceGDK::RemovePresenceUpdateSubscriptions(const TArray<FUniqueNetIdGDKRef>& GDKUserIds)
{
	// This method should not be called from the game thread, since it's a continuation to potentially blocking API calls
	check(!IsInGameThread());

	AsyncTask(ENamedThreads::GameThread, [this, GDKUserIds]()
		{
			for (const FUniqueNetIdGDKRef& GDKUserId : GDKUserIds)
			{
				PresenceSubscriptionSet.Remove(GDKUserId);
			}
		});
}

void FOnlinePresenceGDK::AddStatSubscriptionIfNeeded(FGDKContextHandle GDKContext, const FUniqueNetIdGDKRef& UserId, FStatsSubscriptionSetRef& StatSubscriptions, const FString& StatName)
{
	if (!StatSubscriptions->Contains(StatName) && !StatName.IsEmpty())
	{
		// The following call may be blocking so we'll run it out of the game thread
		AsyncTask(ENamedThreads::AnyThread, [this, GDKContext, UserId, StatSubscriptions, StatName]()
			{
				TArray<uint64> UserQueryXuids;
				UserQueryXuids.Add(UserId->ToUint64());

				const ANSICHAR* Scid = nullptr;
				XblGetScid(&Scid);

				const FTCHARToUTF8 StatNameUtf8(*StatName);
				TArray<const char*> StatNamesUtf8;
				StatNamesUtf8.Add(StatNameUtf8.Get());

				HRESULT Result = XblUserStatisticsTrackStatistics(GDKContext, UserQueryXuids.GetData(), UserQueryXuids.Num(), Scid, StatNamesUtf8.GetData(), StatNamesUtf8.Num());
				if (SUCCEEDED(Result))
				{
					AsyncTask(ENamedThreads::GameThread, [this, StatSubscriptions, StatName]()
						{
							StatSubscriptions->Add(StatName);
						});
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::AddStatSubscriptionIfNeeded] Error from XblUserStatisticsTrackStatistics, with code 0x%08X."), Result);
				}
			});
	}
}

void FOnlinePresenceGDK::EstablishDefaultPresenceStatSubscriptions(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId)
{
	if (FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*RequestingPlayerId))
	{
		const TArray<FString> AutoSubscribePresenceStats = GetAutoSubscribePresenceStatNames();
		FStatsSubscriptionSetRef StatSubscriptions = MakeShared<FStatsSubscriptionSet, ESPMode::ThreadSafe>(FStatsSubscriptionSet());
		StatSubscriptions = AllStatSubscriptionsByLocalUserId.FindOrAdd(RequestingPlayerId).FindOrAdd(PlayerId, StatSubscriptions);

		for (const FString& AutoSubscriptionStatName : AutoSubscribePresenceStats)
		{
			AddStatSubscriptionIfNeeded(GDKContext, PlayerId, StatSubscriptions, AutoSubscriptionStatName);
		}
	}
}

void FOnlinePresenceGDK::SubscribeToStatUpdates(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId, const FString& StatName)
{
	if (FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*RequestingPlayerId))
	{
		FStatsSubscriptionSetRef StatSubscriptions = MakeShared<FStatsSubscriptionSet, ESPMode::ThreadSafe>(FStatsSubscriptionSet());
		AllStatSubscriptionsByLocalUserId.FindOrAdd(RequestingPlayerId).FindOrAdd(PlayerId, StatSubscriptions);

		AddStatSubscriptionIfNeeded(GDKContext, PlayerId, StatSubscriptions, StatName);
	}
}

bool FOnlinePresenceGDK::IsSubscribedToStatUpdates(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId, const FString& StatName) const
{
	if (const TUniqueNetIdMap<FStatsSubscriptionSetRef>* UserSubscriptions = AllStatSubscriptionsByLocalUserId.Find(RequestingPlayerId))
	{
		if (const FStatsSubscriptionSetRef* SubscribedStats = UserSubscriptions->Find(PlayerId))
		{
			return SubscribedStats->Get().Contains(StatName);
		}
	}
	return false;
}

void FOnlinePresenceGDK::ClearAllStatUpdateSubscriptionsForUser(const FUniqueNetIdGDKRef& RequestingPlayerId)
{
	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		FGDKUserHandle PresenceUser = FGDKUserHandle(Identity->GetUserForUniqueNetId(*RequestingPlayerId));
		FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(PresenceUser);
		if (GDKContext.IsValid())
		{
			TUniqueNetIdMap<FStatsSubscriptionSetRef> StatSubscriptionsForAllUsers;
			AllStatSubscriptionsByLocalUserId.RemoveAndCopyValue(RequestingPlayerId, StatSubscriptionsForAllUsers);

			for (TUniqueNetIdMap<FStatsSubscriptionSetRef>::TIterator StatSubscriptionsIter(StatSubscriptionsForAllUsers); StatSubscriptionsIter; ++StatSubscriptionsIter)
			{
				// The following call may be blocking so we'll run it out of the game thread
				FOnlinePresenceGDKPtr PresencePtr = GDKSubsystem->GetPresenceGDK();
				AsyncTask(ENamedThreads::AnyThread, [PresencePtr, GDKContext, UserId = FUniqueNetIdGDK::Create(*StatSubscriptionsIter.Key())->ToUint64(), StatNames = StatSubscriptionsIter.Value()->Array()]()
				{
					if (GDKContext.IsValid())
					{
						TArray<uint64> UserQueryXuids;
						UserQueryXuids.Add(UserId);

						const ANSICHAR* Scid = nullptr;
						XblGetScid(&Scid);

						// We have to keep this array to ensure the lifetime of the char* variables
						TArray<FTCHARToUTF8> StatNamesConverter;
						TArray<const char*> StatNamesUtf8;
						StatNamesConverter.Reserve(StatNames.Num());
						StatNamesUtf8.Reserve(StatNames.Num());
						for (const FString& StatName : StatNames)
						{
							FTCHARToUTF8& StatNameConverted = StatNamesConverter.Emplace_GetRef(*StatName);
							StatNamesUtf8.Emplace(StatNameConverted.Get());
						}

						HRESULT Result = XblUserStatisticsStopTrackingStatistics(GDKContext, UserQueryXuids.GetData(), UserQueryXuids.Num(), Scid, StatNamesUtf8.GetData(), StatNamesUtf8.Num());
						if (FAILED(Result))
						{
							UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::ClearAllStatUpdateSubscriptionsForUser] Error from XblUserStatisticsStopTrackingStatistics, with code 0x%08X."), Result);
						}
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::ClearAllStatUpdateSubscriptionsForUser] GDKContext was invalid."));
					}
				});
			}
		}
	}
}

void FOnlinePresenceGDK::ClearStatUpdateSubscriptionsForFriend(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& TargetPlayerId)
{
	const FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		FGDKUserHandle PresenceUser = FGDKUserHandle(Identity->GetUserForUniqueNetId(*RequestingPlayerId));
		FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(PresenceUser);
		if (GDKContext.IsValid())
		{
			if (TUniqueNetIdMap<FStatsSubscriptionSetRef>* UserSubscriptions = AllStatSubscriptionsByLocalUserId.Find(RequestingPlayerId))
			{
				FStatsSubscriptionSetRef StatSubscriptionsForFriend = MakeShared<FStatsSubscriptionSet, ESPMode::ThreadSafe>(FStatsSubscriptionSet());
				if(UserSubscriptions->RemoveAndCopyValue(TargetPlayerId, StatSubscriptionsForFriend))
				{
					// The following call may be blocking so we'll run it out of the game thread
					FOnlinePresenceGDKPtr PresencePtr = GDKSubsystem->GetPresenceGDK();
					AsyncTask(ENamedThreads::AnyThread, [PresencePtr, GDKContext, TargetPlayerXuid = TargetPlayerId->ToUint64(), StatNames = StatSubscriptionsForFriend->Array()]()
					{
						if (GDKContext.IsValid())
						{
							TArray<uint64> UserQueryXuids;
							UserQueryXuids.Add(TargetPlayerXuid);

							const ANSICHAR* Scid = nullptr;
							XblGetScid(&Scid);

							// We have to keep this array to ensure the lifetime of the char* variables
							TArray<FTCHARToUTF8> StatNamesConverter;
							TArray<const char*> StatNamesUtf8;
							StatNamesConverter.Reserve(StatNames.Num());
							StatNamesUtf8.Reserve(StatNames.Num());
							for (const FString& StatName : StatNames)
							{
								FTCHARToUTF8& StatNameConverted = StatNamesConverter.Emplace_GetRef(*StatName);
								StatNamesUtf8.Emplace(StatNameConverted.Get());
							}

							HRESULT Result = XblUserStatisticsStopTrackingStatistics(GDKContext, UserQueryXuids.GetData(), UserQueryXuids.Num(), Scid, StatNamesUtf8.GetData(), StatNamesUtf8.Num());
							if (FAILED(Result))
							{
								UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::ClearStatUpdateSubscriptionsForFriend] Error from XblUserStatisticsStopTrackingStatistics, with code 0x%08X."), Result);
							}
						}
						else
						{
							UE_LOG_ONLINE(Warning, TEXT("[FOnlinePresenceGDK::ClearStatUpdateSubscriptionsForFriend] GDKContext was invalid."));
						}
					});
				}
			}
		}
	}
}

#endif //WITH_GRDK