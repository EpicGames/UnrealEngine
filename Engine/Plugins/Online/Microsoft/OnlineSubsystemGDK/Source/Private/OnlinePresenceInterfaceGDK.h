// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePresenceInterface.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDKPackage.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "GDKThreadCheck.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/presence_c.h>
#include <xsapi-c/user_statistics_c.h>
#include <XGame.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#include "OnlineError.h"
#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.presence"

namespace OnlinePresenceGDK
{
#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError HResultError(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), InCode)); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

class FOnlineSubsystemGDK;
struct FOnlineError;
class FOnlineAsyncTaskGDKQueryFriendManagerTask;
class FOnlineAsyncTaskGDKQueryPresence;

using FOnlineGDKActivitiesResultMap = TUniqueNetIdMap<FUniqueNetIdPtr /*JoinableSessionId*/>;

typedef TSet<FString> FStatsSubscriptionSet;
typedef TSharedRef<FStatsSubscriptionSet, ESPMode::ThreadSafe> FStatsSubscriptionSetRef;
/**
* Delegate fired when SetPresence task has completed
*/
DECLARE_DELEGATE_TwoParams(FOnSetGDKPresenceCompleteDelegate, bool /*bSuccessful*/, const FOnlineUserPresenceStatus& /*Status*/);

/**
 * GDK Presence record to handle constructing from a GDK PresenceRecord
 */
class FOnlineUserPresenceGDK
	: public FOnlineUserPresence
{
PACKAGE_SCOPE:
	/**
	 * Default constructor
	 */
	FOnlineUserPresenceGDK() = default;
	virtual ~FOnlineUserPresenceGDK() = default;
	FOnlineUserPresenceGDK(const FOnlineUserPresenceGDK& Other) = default;
	FOnlineUserPresenceGDK(FOnlineUserPresenceGDK&& Other) = default;
	FOnlineUserPresenceGDK& operator=(const FOnlineUserPresenceGDK& Other) = default;
	FOnlineUserPresenceGDK& operator=(FOnlineUserPresenceGDK&& Other) = default;

	/**
	 * Construct our values from an XBox presence record
	 */
	explicit FOnlineUserPresenceGDK(FGDKPresenceRecordHandle Record)
	{
		SetPresenceFromPresenceRecord(Record);
	}

	void SetPresenceFromPresenceRecord(FGDKPresenceRecordHandle Record)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // XGameGetXboxTitleId is not safe to call on time-sensitive threads

		XblPresenceUserState UserState = XblPresenceUserState::Unknown;
		HRESULT Result = XblPresenceRecordGetUserState(Record, &UserState);
		switch (UserState)
		{
		case XblPresenceUserState::Online:
			Status.State = EOnlinePresenceState::Online;
			bIsOnline = true;
			break;
		case XblPresenceUserState::Away:
			Status.State = EOnlinePresenceState::Away;
			bIsOnline = true;
			break;
		case XblPresenceUserState::Offline:
		case XblPresenceUserState::Unknown:
		default:
			Status.State = EOnlinePresenceState::Offline;
			break;
		}

		const XblPresenceDeviceRecord* PresenceDeviceRecords;
		uint64 NumPresenceDeviceRecords = 0;
		Result = XblPresenceRecordGetDeviceRecords(Record, &PresenceDeviceRecords, &NumPresenceDeviceRecords);
		if (SUCCEEDED(Result))
		{			
			uint32 TitleId = 0;
			XGameGetXboxTitleId(&TitleId);

			// We have to enumerate through all of the users devices, and then all of the user's titles on those devices to find our own title
			for (uint64 DeviceRecordIndex = 0; DeviceRecordIndex < NumPresenceDeviceRecords; ++DeviceRecordIndex)
			{
				const XblPresenceDeviceRecord& DeviceRecord = PresenceDeviceRecords[DeviceRecordIndex];

				const uint64 TitleRecordSize = DeviceRecord.titleRecordsCount;
				for (uint64 TitleRecordIndex = 0; TitleRecordIndex < TitleRecordSize; ++TitleRecordIndex)
				{
					const XblPresenceTitleRecord& TitleRecord = DeviceRecord.titleRecords[TitleRecordIndex];

					if (TitleRecord.titleActive)
					{
						bIsPlaying = true;
					}

					if (TitleId == TitleRecord.titleId)
					{
						uint64 GDKUserId;
						if (SUCCEEDED(XblPresenceRecordGetXuid(Record, &GDKUserId)))
						{
							FString PresenceString(UTF8_TO_TCHAR(TitleRecord.richPresenceString));
							if (!PresenceString.IsEmpty())
							{
								UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Presence for user %lld is: %s"), GDKUserId, *PresenceString);
							}
							else
							{
								UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("Presence for user %lld is not set"), GDKUserId);
							}

							bIsPlayingThisGame = true;

							//@todo DanH: Once we have a reliable method of automatically keeping this string relatively up-to-date, store it here on the presence as well
							//Status.StatusStr = PresenceString;
							Status.Properties.Add(DefaultPresenceKey, MoveTemp(PresenceString));
						}
						else
						{
							UE_LOG_ONLINE_PRESENCE(Error, TEXT("Failed to get xuid for Presence"));
						}
					}
				}
			}
		}
	}

	/**
	 * Add status key/value properties based on GDK statistics.
	 */
	void SetStatusPropertiesFromStatistics(const XblUserStatisticsResult& StatsResult);
};

/**
 * Implementation for the GDK rich presence interface
 */
class FOnlinePresenceGDK
	: public IOnlinePresence
	, public TSharedFromThis<FOnlinePresenceGDK, ESPMode::ThreadSafe>
{
PACKAGE_SCOPE:
	/** Constructor
	 *
	 * @param InSubsystem The owner of this external UI interface.
	 */
	explicit FOnlinePresenceGDK(class FOnlineSubsystemGDK* InSubsystem);

public:
	// IOnlinePresence
	virtual void SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual void QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual EOnlineCachedResult::Type GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	virtual EOnlineCachedResult::Type GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence) override;

	void Tick(const float DeltaTime);

	static TArray<FString> GetAutoSubscribePresenceStatNames();

PACKAGE_SCOPE:

	struct FPresenceSubscriptionHandles
	{
		XblRealTimeActivitySubscriptionHandle DeviceChangeSubscriptionHandle = nullptr;
		XblRealTimeActivitySubscriptionHandle TitleChangeSubscriptionHandle = nullptr;
	};

	void OnPresenceDeviceChanged(const FUniqueNetIdGDK& UserGDK, XblPresenceDeviceType DeviceType, bool bIsUserLoggedOnDevice);
	void OnPresenceTitleChanged(const FUniqueNetIdGDK& UserGDK, uint32 TitleId, XblPresenceTitleState TitleState);

	void UnsubscribeFromAllPresenceUpdatesForUser(FGDKContextHandle GDKContext);
	bool IsSubscribedToPresenceUpdates(const FUniqueNetIdGDKRef& GDKUserId) const;
	void AddPresenceUpdateSubscriptions(const TArray<FUniqueNetIdGDKRef>& GDKUserIds);
	void RemovePresenceUpdateSubscriptions(const TArray<FUniqueNetIdGDKRef>& GDKUserIds);

	void OnFriendSessionUpdated(const FOnlineError& ErrorResult, const FOnlineGDKActivitiesResultMap& ActivitiesResults);
	void OnFriendSessionUpdated(bool bWasSuccessful, TArray<FOnlineSession> OnlineSessions);

	void ProcessStatUpdate(const FUniqueNetIdGDKRef& LocalUserId, const FUniqueNetIdGDKRef& UpdatedUserId, const FString& StatName, const FString& StatValue, const FString& StatType);

	void EstablishDefaultPresenceStatSubscriptions(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId);
	void SubscribeToStatUpdates(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId, const FString& StatName);
	bool IsSubscribedToStatUpdates(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& PlayerId, const FString& StatName) const;
	void ClearAllStatUpdateSubscriptionsForUser(const FUniqueNetIdGDKRef& RequestingPlayerId);
	void ClearStatUpdateSubscriptionsForFriend(const FUniqueNetIdGDKRef& RequestingPlayerId, const FUniqueNetIdGDKRef& TargetPlayerId);

	void HandleSetGDKPresenceComplete(bool bSuccessful, const FOnlineUserPresenceStatus& Status, const FUniqueNetIdGDKRef User, const FString CurrentPresenceIdString, const FOnPresenceTaskCompleteDelegate Delegate);

PACKAGE_SCOPE:
	/** Reference to the owning subsystem */
	FOnlineSubsystemGDK* GDKSubsystem;

private:
	friend FOnlineAsyncTaskGDKQueryFriendManagerTask;
	friend FOnlineAsyncTaskGDKQueryPresence;

	TArray<FDelegateHandle> FriendSessionDelegateHandles;

	/** Cache of local user's last saved presence to prevent duplicate stores */
	TUniqueNetIdMap<FOnlineUserPresenceStatus> LocalUserPresenceCache;

	/** Cache of presence data. Stores only the most recent results of QueryPresence. */
	TUniqueNetIdMap<TSharedRef<FOnlineUserPresenceGDK>> PresenceCache;

	/** Stat updates we are subscribed to for each player on behalf of a given local player */
	TUniqueNetIdMap<TUniqueNetIdMap<FStatsSubscriptionSetRef>> AllStatSubscriptionsByLocalUserId;

	/** Set of users we're subscribed to for presence updates */
	FUniqueNetIdSet PresenceSubscriptionSet;

	/** Helper function to get the presence string from presence status */
	static FString GetPresenceIdString(const FOnlineUserPresenceStatus& Status);

	/** Helper function to launch the async task to track user statistics changes */
	void AddStatSubscriptionIfNeeded(FGDKContextHandle GDKContext, const FUniqueNetIdGDKRef& UserId, FStatsSubscriptionSetRef& StatSubscriptions, const FString& StatName);

	float SessionPresencePollDelay = 60.0f;
};

typedef TSharedPtr<FOnlinePresenceGDK, ESPMode::ThreadSafe> FOnlinePresenceGDKPtr;

