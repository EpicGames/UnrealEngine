// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineStoreInterfaceGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserCollectionsId.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserPurchaseId.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryReceipts.h"
#include "AsyncTasks/OnlineAsyncTaskGDKPurchaseOffer.h"
#include "AsyncTasks/OnlineAsyncTaskGDKRedeemCode.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"

#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.purchase"

void LexFromString(EPurchaseValidationMethod& Value, const TCHAR* String)
{
	if (FCString::Stricmp(String, TEXT("XSTSToken")) == 0)
	{
		Value = EPurchaseValidationMethod::XSTSToken;
	}
	else if (FCString::Stricmp(String, TEXT("UserCollectionsId")) == 0)
	{
		Value = EPurchaseValidationMethod::UserCollectionsId;
	}
	else if (FCString::Stricmp(String, TEXT("UserPurchaseId")) == 0)
	{
		Value = EPurchaseValidationMethod::UserPurchaseId;
	}
	else
	{
		Value = EPurchaseValidationMethod::None;
	}
}

FOnlinePurchaseGDK::FOnlinePurchaseGDK(FOnlineSubsystemGDK* InGDKSubsystem)
	: GDKSubsystem(InGDKSubsystem)
	, bIsCurrentlyInCheckout(false)
	, bSoftFailReceiptFailures(false)
	, ValidationMethod_Consumable(EPurchaseValidationMethod::XSTSToken)
	, ValidationMethod_Pass(EPurchaseValidationMethod::XSTSToken)
{
	check(GDKSubsystem);
	GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("ReceiptsXSTSEndpoint"), ReceiptsXSTSEndpoint, GEngineIni);
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bSoftFailReceiptFailures"), bSoftFailReceiptFailures, GEngineIni);

	FString PurchaseValidationMethodString;
	if (GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("ValidationMethod_Consumable"), PurchaseValidationMethodString, GEngineIni))
	{
		LexFromString(ValidationMethod_Consumable, *PurchaseValidationMethodString);
	}

	if (GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("ValidationMethod_Pass"), PurchaseValidationMethodString, GEngineIni))
	{
		LexFromString(ValidationMethod_Pass, *PurchaseValidationMethodString);
	}
}

FOnlinePurchaseGDK::~FOnlinePurchaseGDK()
{
}

bool FOnlinePurchaseGDK::IsAllowedToPurchase(const FUniqueNetId& UserId)
{
	FOnlineError Temp;
	return IsAllowedToPurchase(UserId, Temp);
}

bool FOnlinePurchaseGDK::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{
	FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(UserId));
	if (!GDKUser)
	{	
		Error = ONLINE_ERROR(EOnlineErrorResult::InvalidUser);
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGDK::IsAllowedToPurchase failed with due to known user %s."), *UserId.ToDebugString());
		return false;
	}

	if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		Error = ONLINE_ERROR(EOnlineErrorResult::MismatchedUser);
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGDK::IsAllowedToPurchase There is mismatch between the local user and the store user "));
		return false;
	}

	//WMM TODO: Remove this when the enumeration supports it:
	const XalPrivilege XalPrivilegePurchaseContent = static_cast<XalPrivilege>(245);

	bool bHasPermission = false;

	HRESULT Result = XalUserCheckPrivilege(GDKUser, XalPrivilegePurchaseContent, &bHasPermission, nullptr);
	if (Result != S_OK)
	{
		Error = ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), Result));
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGDK::IsAllowedToPurchase failed with code 0x%0.8X."), Result);
		return false;
	}

	if (!bHasPermission)
	{
		Error = ONLINE_ERROR(EOnlineErrorResult::InvalidCreds);
	}
	else
	{
		Error = ONLINE_ERROR(EOnlineErrorResult::Success);
	}
	return bHasPermission;
}

void FOnlinePurchaseGDK::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	// Lambda to wrap calling our delegate with an error and logging the message
	auto CallDelegateError = [this, &Delegate](const FString& ErrorMessage, EOnlineErrorResult Result = EOnlineErrorResult::Unknown)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, ErrorMessage, Result]
		{
			UE_LOG_ONLINE_PURCHASE(Error, TEXT("%s"), *ErrorMessage);

			const TSharedRef<FPurchaseReceipt> PurchaseReceipt = MakeShared<FPurchaseReceipt>();
			PurchaseReceipt->TransactionState = EPurchaseTransactionState::Failed;

			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_Checkout_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(Result), PurchaseReceipt);
		});
	};

	if (bIsCurrentlyInCheckout)
	{
		CallDelegateError(TEXT("FOnlinePurchaseGDK::Checkout failed, another purchase already in progress."));
		return;
	}

	FGDKUserHandle GDKUser;
	FGDKContextHandle UserGDKContext;
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);
	if (!GetGDKUserInfoOrCallDelegateOnError(GDKUserId, GDKUser, UserGDKContext, CallDelegateError))
	{
		// The delegate is already called
		return;
	}

	if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		CallDelegateError(TEXT("FOnlinePurchaseGDK::Checkout There is mismatch between the local user and the store user."), EOnlineErrorResult::MismatchedUser);
		return;
	}

	TOptional<FUniqueOfferId> StoreGDKOfferId = GetStoreGDKOfferIdOrCallDelegateOnError(CheckoutRequest, CallDelegateError);
	if (!StoreGDKOfferId.IsSet())
	{
		// The delegate is already called
		return;
	}

	bIsCurrentlyInCheckout = true;
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKPurchaseOffer>(GDKSubsystem, StoreGDKOfferId.GetValue(), GDKUser, UserGDKContext, GDKUserId, Delegate);
}

void FOnlinePurchaseGDK::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	// Lambda to wrap calling our delegate with an error and logging the message
	auto CallDelegateError = [this, &Delegate](const FString& ErrorMessage, EOnlineErrorResult Result = EOnlineErrorResult::Unknown)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, ErrorMessage, Result]
			{
				UE_LOG_ONLINE_PURCHASE(Error, TEXT("%s"), *ErrorMessage);

				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_Checkout_Delegate);
				Delegate.ExecuteIfBound(FOnlineError(Result));
			});
	};

	if (bIsCurrentlyInCheckout)
	{
		CallDelegateError(TEXT("FOnlinePurchaseGDK::Checkout failed, another purchase already in progress."));
		return;
	}

	FGDKUserHandle GDKUser;
	FGDKContextHandle UserGDKContext;
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);
	if (!GetGDKUserInfoOrCallDelegateOnError(GDKUserId, GDKUser, UserGDKContext, CallDelegateError))
	{
		// The delegate is already called
		return;
	}

	if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		CallDelegateError(TEXT("FOnlinePurchaseGDK::Checkout There is mismatch between the local user and the store user."), EOnlineErrorResult::MismatchedUser);
		return;
	}

	TOptional<FUniqueOfferId> StoreGDKOfferId = GetStoreGDKOfferIdOrCallDelegateOnError(CheckoutRequest, CallDelegateError);
	if (!StoreGDKOfferId.IsSet())
	{
		// The delegate is already called
		return;
	}

	bIsCurrentlyInCheckout = true;
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKPurchaseOfferNoEntitlements>(GDKSubsystem, StoreGDKOfferId.GetValue(), GDKUser, UserGDKContext, GDKUserId, Delegate);
}

bool FOnlinePurchaseGDK::GetGDKUserInfoOrCallDelegateOnError(FUniqueNetIdGDKRef GDKUserId, FGDKUserHandle& OutGDKUser, FGDKContextHandle& OutUserGDKContext, const TCallDelegateOnError& CallDelegateOnError)
{
	OutGDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(*GDKUserId);
	if (!OutGDKUser)
	{
		CallDelegateOnError(FString::Printf(TEXT("FOnlinePurchaseGDK::Checkout failed due to unknown user %s."), *(GDKUserId.Get().ToDebugString())));
		return false;
	}

	OutUserGDKContext = GDKSubsystem->GetGDKContext(OutGDKUser);
	if (!OutUserGDKContext)
	{
		CallDelegateOnError(FString::Printf(TEXT("FOnlinePurchaseGDK::Checkout failed due to missing context for user %s."), *(GDKUserId.Get().ToDebugString())));
		return false;
	}

	return true;
}

TOptional<FUniqueOfferId> FOnlinePurchaseGDK::GetStoreGDKOfferIdOrCallDelegateOnError(const FPurchaseCheckoutRequest& CheckoutRequest,  const TCallDelegateOnError& CallDelegateOnError)
{
	TOptional<FUniqueOfferId> StoreGDKOfferId;
	if (CheckoutRequest.PurchaseOffers.Num() == 0)
	{
		CallDelegateOnError(TEXT("FOnlinePurchaseGDK::Checkout failed, there were no entries passed to purchase"));
		return StoreGDKOfferId;
	}
	if (CheckoutRequest.PurchaseOffers.Num() != 1)
	{
		CallDelegateOnError(TEXT("FOnlinePurchaseGDK::Checkout failed, there were more than one entry passed to purchase. GDK currently only supports one."));
		return StoreGDKOfferId;
	}
	check(CheckoutRequest.PurchaseOffers.IsValidIndex(0));
	const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Entry = CheckoutRequest.PurchaseOffers[0];

	if (Entry.Quantity != 1)
	{
		CallDelegateOnError(TEXT("FOnlinePurchaseGDK::Checkout failed, purchase quantity not set to one. GDK currently only supports one."));
		return StoreGDKOfferId;
	}

	if (Entry.OfferId.IsEmpty())
	{
		CallDelegateOnError(TEXT("FOnlinePurchaseGDK::Checkout failed, OfferId is blank."));
		return StoreGDKOfferId;
	}

	TSharedPtr<FOnlineStoreOfferGDK> GDKOffer = StaticCastSharedPtr<FOnlineStoreOfferGDK>(GDKSubsystem->GetStoreGDK()->GetOffer(Entry.OfferId));
	if (!GDKOffer.IsValid())
	{
		CallDelegateOnError(TEXT("FOnlinePurchaseGDK::Checkout failed, Could not find corresponding offer."));
		return StoreGDKOfferId;
	}

	StoreGDKOfferId = GDKOffer->OfferId;
	return StoreGDKOfferId;
}


void FOnlinePurchaseGDK::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	// No-Op
	UNREFERENCED_PARAMETER(UserId);
	UNREFERENCED_PARAMETER(ReceiptId);
}

void FOnlinePurchaseGDK::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);

	FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(*GDKUserId);
	if (!GDKUser)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("No LocalUser found for user %s"), *GDKUserId->ToDebugString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_RedeemCode_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(TEXT("NoLocalUserFound")), MakeShared<FPurchaseReceipt>());
		});
		return;
	}

	if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGDK::RedeemCode There is mismatch between the local user and the store user."));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_RedeemCode_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::MismatchedUser), MakeShared<FPurchaseReceipt>());
		});
		return;
	}

	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	if (!UserGDKContext)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("No XboxGDKContext found for user %s"), *GDKUserId->ToDebugString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_RedeemCode_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(TEXT("NoGDKContextFound")), MakeShared<FPurchaseReceipt>());
		});
		return;
	}

	bIsCurrentlyInCheckout = true;
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKRedeemCode>(GDKSubsystem, GDKUser, UserGDKContext, GDKUserId, RedeemCodeRequest, Delegate);
}

void FOnlinePurchaseGDK::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	UNREFERENCED_PARAMETER(bRestoreReceipts);

	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);
	FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(*GDKUserId);
	if (!GDKUser)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("No LocalUser found for user %s"), *GDKUserId->ToDebugString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_QueryReceipts_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(TEXT("NoLocalUserFound")));
		});
		return;
	}

	if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGDK::QueryReceipts There is mismatch between the local user and the store user."));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_QueryReceipts_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::MismatchedUser));
		});
		return;
	}

	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(GDKUser);
	if (!UserGDKContext)
	{
		GDKSubsystem->ExecuteNextTick([Delegate, GDKUserId]
		{
			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("No XboxGDKContext found for user %s"), *GDKUserId->ToDebugString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_QueryReceipts_Delegate);
			Delegate.ExecuteIfBound(FOnlineError(TEXT("NoGDKContextFound")));
		});
		return;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryReceipts>(GDKSubsystem, GDKUserId, UserGDKContext, Delegate);
}

void FOnlinePurchaseGDK::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);

	const TArray<FPurchaseReceipt>* FoundReceipts = UserCachedReceipts.Find(GDKUserId);
	if (FoundReceipts == nullptr)
	{
		// The user is getting receipts when we haven't queried for this user (or the query hasn't finished yet)
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("No cached receipts found for player %s"), *GDKUserId->ToDebugString());
		OutReceipts.Empty();
	}
	else
	{
		OutReceipts = *FoundReceipts;
	}
}

void FOnlinePurchaseGDK::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	FString ReceiptValidationInfo;
	FString ServiceTicketAndPublisherUserId;

	if (InReceiptValidationInfo.Split(TEXT(":ServiceTicket:"), &ReceiptValidationInfo, &ServiceTicketAndPublisherUserId))
	{
		EOnlineErrorResult Error = EOnlineErrorResult::Unknown;
		FString ErrorStr;
		FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(UserId);
		if (!UserGDKContext.IsValid())
		{
			Error = EOnlineErrorResult::InvalidUser;
			ErrorStr = TEXT("Could not map requested user to a GDKContext");
		}
		else if (ServiceTicketAndPublisherUserId.IsEmpty())
		{
			Error = EOnlineErrorResult::InvalidParams;
			ErrorStr = TEXT("ServiceTicket is empty");
		}
		FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(UserId));
		if (GDKSubsystem->GetStoreGDK()->BlockMismatchedStoreUser(GDKUser))
		{
			ErrorStr = TEXT("Title/Store ID Mismatch");
			Error = EOnlineErrorResult::MismatchedUser;
		}

		FString ServiceTicket;
		FString PublisherUserId;
		if (!ServiceTicketAndPublisherUserId.Split(TEXT(":PublisherUserId:"), &ServiceTicket, &PublisherUserId))
		{
			// PublisherUserId is optional. In this case, the ServiceTicket is the entire string
			ServiceTicket = ServiceTicketAndPublisherUserId;
		}

		if (ErrorStr.IsEmpty())
		{
			if (ReceiptValidationInfo.EndsWith(TEXT(":UserCollectionsId:")))
			{
				GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGetUserCollectionsId>(GDKSubsystem, UserGDKContext, ServiceTicket, PublisherUserId,
					FOnlineAsyncTaskGetUserCollectionsId::FOnCompleteDelegate::CreateLambda(
						[Delegate, ReceiptValidationInfo](const FString& StoreId, const FOnlineError& Result)
						{
							FString FinalValidationInfo = Result.WasSuccessful() ? ReceiptValidationInfo + StoreId : FString();
							QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_FinalizeReceiptValidationInfo_Delegate);
							Delegate.ExecuteIfBound(Result, FinalValidationInfo);
						}));
				return;
			}
			else if (ReceiptValidationInfo.EndsWith(TEXT(":UserPurchaseId:")))
			{
				GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGetUserPurchaseId>(GDKSubsystem, UserGDKContext, ServiceTicket, PublisherUserId,
					FOnlineAsyncTaskGetUserCollectionsId::FOnCompleteDelegate::CreateLambda(
						[Delegate, ReceiptValidationInfo](const FString& StoreId, const FOnlineError& Result)
						{
							FString FinalValidationInfo = Result.WasSuccessful() ? ReceiptValidationInfo + StoreId : FString();
							QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_FinalizeReceiptValidationInfo_Delegate);
							Delegate.ExecuteIfBound(Result, FinalValidationInfo);
						}));
				return;
			}
		}
		else
		{
			GDKSubsystem->ExecuteNextTick([ErrorStr, Delegate, Error]()
			{
				FOnlineError Result = ONLINE_ERROR(Error, ErrorStr);
				Result.bSucceeded = false;
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_FinalizeReceiptValidationInfo_Delegate);
				Delegate.ExecuteIfBound(Result, FString());
			});
			return;
		}
	}

	GDKSubsystem->ExecuteNextTick([InReceiptValidationInfo, Delegate]()
	{
		FOnlineError DefaultSuccess(true);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlinePurchaseGDK_FinalizeReceiptValidationInfo_Delegate);
		Delegate.ExecuteIfBound(DefaultSuccess, InReceiptValidationInfo);
	});
}


EPurchaseValidationMethod FOnlinePurchaseGDK::GetPurchaseValidationMethod(XStoreProductKind ProductKind) const
{
	switch (ProductKind)
	{
	case XStoreProductKind::Consumable:
		return ValidationMethod_Consumable;

	case XStoreProductKind::Pass:
		return ValidationMethod_Pass;

	default:
		return EPurchaseValidationMethod::None;
	}
}
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK