// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGDKPrivate.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemGDKTypes.h"

#include "OnlineError.h"
#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.purchase"

namespace OnlinePurchaseGDK
{
#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError PurchaseCanceled() { return ONLINE_ERROR(EOnlineErrorResult::Canceled, TEXT("purchase_cancelled"), LOCTEXT("PurchaseCancelled", "Your purchase was cancelled.")); }
		inline FOnlineError PurchaseFailureStart(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("purchase_failure_start[0x%08X]"), InCode), LOCTEXT("PurchaseFailedStart", "We were unable to initiate your purchase at this time.")); }
		inline FOnlineError PurchaseFailure(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("purchase_failure[0x%08X]"), InCode), LOCTEXT("PurchaseFailed", "We were unable to complete your purchase at this time.")); }
		inline FOnlineError HResultError(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), InCode)); }
		inline FOnlineError HResultError(int32 InCode, const FText& InText) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), InCode), InText); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

enum class XStoreProductKind : uint32_t;

class FOnlineSubsystemGDK;

/** Used to specify the contents of ValidationInfo in receipt line items */
enum class EPurchaseValidationMethod
{
	// empty ValidationInfo
	None,

	// <UserId>:<XSTSToken> ValidationInfo
	XSTSToken,

	// <UserId>:UserCollectionsId::ServiceTicket: ValidationInfo. Call FinalizeReceiptValidationInfo with an AzureAD token appended to this
	// Optionally, you can append :PublisherUserId:<PublisherUserId> to associate that user id with the UserStoreId
	// See "Collections: Requesting a User Store ID for service-to-service authentication" in the GDK documentation for AzureAD token
	// FinalizeReceiptValidation will return <UserId>:UserCollectionsId:<UserCollectionsId>
	UserCollectionsId,

	// <UserId>:UserPurchaseId::ServiceTicket: ValidationInfo. Call FinalizeReceiptValidationInfo with an AzureAD token appended to this
	// Optionally, you can append :PublisherUserId:<PublisherUserId> to associate that user id with the UserStoreId
	// See "Collections: Requesting a User Store ID for service-to-service authentication" in the GDK documentation for AzureAD token
	// FinalizeReceiptValidation will return <UserId>:UserPurchaseId:<UserPurchaseId>
	UserPurchaseId
};

class FOnlinePurchaseGDK
	: public IOnlinePurchase
	, public TSharedFromThis<FOnlinePurchaseGDK, ESPMode::ThreadSafe>
{
public:
	FOnlinePurchaseGDK(FOnlineSubsystemGDK* InGDKSubsystem);
	virtual ~FOnlinePurchaseGDK();

public:
	//~ Begin IOnlinePurchase Interface
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	//~ End IOnlinePurchase Interface

PACKAGE_SCOPE:
	void RegisterLivePurchaseHooks();
	void UnregisterLivePurchaseHooks();

	EPurchaseValidationMethod GetPurchaseValidationMethod(XStoreProductKind ProductKind) const;

private:
	typedef TFunction<void(const FString& /*ErrorCode*/)> TCallDelegateOnError;
	bool GetGDKUserInfoOrCallDelegateOnError(FUniqueNetIdGDKRef OutGDKUserId, FGDKUserHandle& OutGDKUser, FGDKContextHandle& OutUserGDKContext, const TCallDelegateOnError& CallDelegateOnError);
	TOptional<FUniqueOfferId> GetStoreGDKOfferIdOrCallDelegateOnError(const FPurchaseCheckoutRequest& CheckoutRequest, const TCallDelegateOnError& CallDelegateOnError);

PACKAGE_SCOPE:
	/** Pointer back to our parent subsystem */
	FOnlineSubsystemGDK* GDKSubsystem;
	/** Cached receipts/inventory per user */
	TUniqueNetIdMap<TArray<FPurchaseReceipt> > UserCachedReceipts;
	/** Endpoint to pass during XSTS generation for claiming inventory items */
	FString ReceiptsXSTSEndpoint;
	/** Is there a purchase in progress currently? */
	bool bIsCurrentlyInCheckout;
	/** Soft-Fail Receipt-Checks (return successful but with an Error Message/Code) if any entitlement succeeds */
	bool bSoftFailReceiptFailures;

	EPurchaseValidationMethod ValidationMethod_Consumable;
	EPurchaseValidationMethod ValidationMethod_Pass;

private:
	/** Delegate handle for the "User Purchased Something" delegate */
	XTaskQueueRegistrationToken GDKEventProductPurchaseHandle;
};

typedef TSharedPtr<FOnlinePurchaseGDK, ESPMode::ThreadSafe> FOnlinePurchaseGDKPtr;
