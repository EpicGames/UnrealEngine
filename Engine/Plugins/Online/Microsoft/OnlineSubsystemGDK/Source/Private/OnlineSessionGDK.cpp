// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlineSessionInterfaceMpaGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "GDKRuntimeModule.h"
#include "Misc/ConfigCacheIni.h"

THIRD_PARTY_INCLUDES_START
#include <XGameInvite.h>
#include <XGameRuntimeFeature.h>
THIRD_PARTY_INCLUDES_END


TAutoConsoleVariable<bool> CVarXboxMpaEnabled(
	TEXT("xb.mpaEnabled"),
	true,
	TEXT("Enable MPA instead of MPSD")
);

FOnlineSessionGDK::FOnlineSessionGDK(FOnlineSubsystemGDK* InSubsystem)
	: OnlineSessionMpsdGDK(MakeShared<FOnlineSessionMpsdGDK>(InSubsystem))
	, OnlineSessionMpaGDK(MakeShared<FOnlineSessionMpaGDK>(InSubsystem))
	, GDKSubsystem(InSubsystem)
{
	// Sign up for Activated events, which we use to detect accepted invites while the game is running.
	// This delegate will be called on protocol activation as well, to process session join after startup.
	InviteAcceptedHandler = IGDKRuntimeModule::Get().RegisterGameInviteReceivedDelegate( FGDKOnGameInviteReceivedDelegate::CreateRaw( this, &FOnlineSessionGDK::SaveInviteFromActivation ));
}

FOnlineSessionGDK::~FOnlineSessionGDK()
{
	if (IGDKRuntimeModule* GDKRuntimeModule = IGDKRuntimeModule::TryGet())
	{
		GDKRuntimeModule->UnregisterGameInviteReceivedDelegate(InviteAcceptedHandler);
	}
}

bool FOnlineSessionGDK::IsMpaEnabled() const
{
	return CVarXboxMpaEnabled.GetValueOnAnyThread();
}

void FOnlineSessionGDK::ClearSessionState(FGDKContextHandle GDKContext, FUniqueNetIdGDKRef NetID)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->ClearSessionState(GDKContext, NetID);
	}
}

void FOnlineSessionGDK::SaveInviteFromActivation(const FString& ActivationUri)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->SaveInviteFromActivation(ActivationUri);
	}
	else
	{
		OnlineSessionMpsdGDK->SaveInviteFromActivation(ActivationUri);
	}
}

const FOnlineSessionMpsdGDKPtr& FOnlineSessionGDK::GetMpsdImpl() const 
{ 
	check(!CVarXboxMpaEnabled.GetValueOnAnyThread());
	return OnlineSessionMpsdGDK; 
}

const FOnlineSessionMpaGDKPtr& FOnlineSessionGDK::GetMpaImpl() const 
{ 
	check(CVarXboxMpaEnabled.GetValueOnAnyThread());
	return OnlineSessionMpaGDK; 
}

FUniqueNetIdPtr FOnlineSessionGDK::CreateSessionIdFromString(const FString& SessionIdStr)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CreateSessionIdFromString(SessionIdStr);
	}
	else
	{
		return OnlineSessionMpsdGDK->CreateSessionIdFromString(SessionIdStr);
	}
}

FNamedOnlineSessionPtr FOnlineSessionGDK::GetNamedSessionPtr(FName SessionName) const
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetNamedSessionPtr(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetNamedSessionPtr(SessionName);
	}
}

FNamedOnlineSession* FOnlineSessionGDK::GetNamedSession(FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetNamedSession(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetNamedSession(SessionName);
	}
}

void FOnlineSessionGDK::RemoveNamedSession(FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->RemoveNamedSession(SessionName);
	}
	else
	{
		OnlineSessionMpsdGDK->RemoveNamedSession(SessionName);
	}
}

bool FOnlineSessionGDK::HasPresenceSession()
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->HasPresenceSession();
	}
	else
	{
		return OnlineSessionMpsdGDK->HasPresenceSession();
	}
}

EOnlineSessionState::Type FOnlineSessionGDK::GetSessionState(FName SessionName) const
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetSessionState(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetSessionState(SessionName);
	}
}

bool FOnlineSessionGDK::CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CreateSession(HostingPlayerControllerIndex, SessionName, NewSessionSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->CreateSession(HostingPlayerControllerIndex, SessionName, NewSessionSettings);
	}
}

bool FOnlineSessionGDK::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CreateSession(HostingPlayerId, SessionName, NewSessionSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->CreateSession(HostingPlayerId, SessionName, NewSessionSettings);
	}
}

bool FOnlineSessionGDK::StartSession(FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->StartSession(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->StartSession(SessionName);
	}
}

bool FOnlineSessionGDK::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->UpdateSession(SessionName, UpdatedSessionSettings, bShouldRefreshOnlineData);
	}
	else
	{
		return OnlineSessionMpsdGDK->UpdateSession(SessionName, UpdatedSessionSettings, bShouldRefreshOnlineData);
	}
}

bool FOnlineSessionGDK::EndSession(FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->EndSession(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->EndSession(SessionName);
	}
}

bool FOnlineSessionGDK::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->DestroySession(SessionName, CompletionDelegate);
	}
	else
	{
		return OnlineSessionMpsdGDK->DestroySession(SessionName, CompletionDelegate);
	}
}

bool FOnlineSessionGDK::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->IsPlayerInSession(SessionName, UniqueId);
	}
	else
	{
		return OnlineSessionMpsdGDK->IsPlayerInSession(SessionName, UniqueId);
	}
}

FDelegateHandle FOnlineSessionGDK::AddOnMatchmakingCompleteDelegate_Handle(const FOnMatchmakingCompleteDelegate& Delegate)
{
	return GDKSubsystem->GetMatchmakingInterfaceGDK()->AddOnMatchmakingCompleteDelegate_Handle(Delegate);
}

void FOnlineSessionGDK::ClearOnMatchmakingCompleteDelegate_Handle(FDelegateHandle& DelegateHandle)
{
	GDKSubsystem->GetMatchmakingInterfaceGDK()->ClearOnMatchmakingCompleteDelegate_Handle(DelegateHandle);
}

void FOnlineSessionGDK::TriggerOnMatchmakingCompleteDelegates(FName SessionName, bool bWasSuccessful)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_OnMatchmakingComplete);
	GDKSubsystem->GetMatchmakingInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
}

FDelegateHandle FOnlineSessionGDK::AddOnCancelMatchmakingCompleteDelegate_Handle(const FOnCancelMatchmakingCompleteDelegate& Delegate)
{
	return GDKSubsystem->GetMatchmakingInterfaceGDK()->AddOnCancelMatchmakingCompleteDelegate_Handle(Delegate);
}

void FOnlineSessionGDK::ClearOnCancelMatchmakingCompleteDelegate_Handle(FDelegateHandle& DelegateHandle)
{
	GDKSubsystem->GetMatchmakingInterfaceGDK()->ClearOnCancelMatchmakingCompleteDelegate_Handle(DelegateHandle);
}

void FOnlineSessionGDK::TriggerOnCancelMatchmakingCompleteDelegates(FName SessionName, bool bWasSuccessful)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_OnCancelMatchmakingComplete);
	GDKSubsystem->GetMatchmakingInterfaceGDK()->TriggerOnCancelMatchmakingCompleteDelegates(SessionName, bWasSuccessful);
}

bool FOnlineSessionGDK::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->StartMatchmaking(LocalPlayers, SessionName, NewSessionSettings, SearchSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->StartMatchmaking(LocalPlayers, SessionName, NewSessionSettings, SearchSettings);
	}
}

bool FOnlineSessionGDK::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CancelMatchmaking(SearchingPlayerNum, SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->CancelMatchmaking(SearchingPlayerNum, SessionName);
	}
}

bool FOnlineSessionGDK::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CancelMatchmaking(SearchingPlayerId, SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->CancelMatchmaking(SearchingPlayerId, SessionName);
	}
}

bool FOnlineSessionGDK::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindSessions(SearchingPlayerNum, SearchSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindSessions(SearchingPlayerNum, SearchSettings);
	}
}

bool FOnlineSessionGDK::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindSessions(SearchingPlayerId, SearchSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindSessions(SearchingPlayerId, SearchSettings);
	}
}

bool FOnlineSessionGDK::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindSessionById(SearchingUserId, SessionId, FriendId, CompletionDelegate);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindSessionById(SearchingUserId, SessionId, FriendId, CompletionDelegate);
	}
}

bool FOnlineSessionGDK::CancelFindSessions()
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->CancelFindSessions();
	}
	else
	{
		return OnlineSessionMpsdGDK->CancelFindSessions();
	}
}

bool FOnlineSessionGDK::PingSearchResults(const FOnlineSessionSearchResult& SearchResult)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->PingSearchResults(SearchResult);
	}
	else
	{
		return OnlineSessionMpsdGDK->PingSearchResults(SearchResult);
	}
}

bool FOnlineSessionGDK::JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->JoinSession(ControllerIndex, SessionName, DesiredSession);
	}
	else
	{
		return OnlineSessionMpsdGDK->JoinSession(ControllerIndex, SessionName, DesiredSession);
	}
}

bool FOnlineSessionGDK::JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->JoinSession(UserId, SessionName, DesiredSession);
	}
	else
	{
		return OnlineSessionMpsdGDK->JoinSession(UserId, SessionName, DesiredSession);
	}
}

bool FOnlineSessionGDK::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindFriendSession(LocalUserNum, Friend);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindFriendSession(LocalUserNum, Friend);
	}
}

bool FOnlineSessionGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindFriendSession(LocalUserId, Friend);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindFriendSession(LocalUserId, Friend);
	}
}

bool FOnlineSessionGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->FindFriendSession(LocalUserId, FriendList);
	}
	else
	{
		return OnlineSessionMpsdGDK->FindFriendSession(LocalUserId, FriendList);
	}
}

bool FOnlineSessionGDK::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->SendSessionInviteToFriend(LocalUserNum, SessionName, Friend);
	}
	else
	{
		return OnlineSessionMpsdGDK->SendSessionInviteToFriend(LocalUserNum, SessionName, Friend);
	}
}

bool FOnlineSessionGDK::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->SendSessionInviteToFriend(LocalUserId, SessionName, Friend);
	}
	else
	{
		return OnlineSessionMpsdGDK->SendSessionInviteToFriend(LocalUserId, SessionName, Friend);
	}
}

bool FOnlineSessionGDK::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->SendSessionInviteToFriends(LocalUserNum, SessionName, Friends);
	}
	else
	{
		return OnlineSessionMpsdGDK->SendSessionInviteToFriends(LocalUserNum, SessionName, Friends);
	}
}

bool FOnlineSessionGDK::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->SendSessionInviteToFriends(LocalUserId, SessionName, Friends);
	}
	else
	{
		return OnlineSessionMpsdGDK->SendSessionInviteToFriends(LocalUserId, SessionName, Friends);
	}
}

bool FOnlineSessionGDK::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetResolvedConnectString(SessionName, ConnectInfo, PortType);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetResolvedConnectString(SessionName, ConnectInfo, PortType);
	}
}

bool FOnlineSessionGDK::GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetResolvedConnectString(SearchResult, PortType, ConnectInfo);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetResolvedConnectString(SearchResult, PortType, ConnectInfo);
	}
}

FOnlineSessionSettings* FOnlineSessionGDK::GetSessionSettings(FName SessionName)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetSessionSettings(SessionName);
	}
	else
	{
		return OnlineSessionMpsdGDK->GetSessionSettings(SessionName);
	}
}

bool FOnlineSessionGDK::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->RegisterPlayer(SessionName, PlayerId, bWasInvited);
	}
	else
	{
		return OnlineSessionMpsdGDK->RegisterPlayer(SessionName, PlayerId, bWasInvited);
	}
}

bool FOnlineSessionGDK::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->RegisterPlayers(SessionName, Players, bWasInvited);
	}
	else
	{
		return OnlineSessionMpsdGDK->RegisterPlayers(SessionName, Players, bWasInvited);
	}
}

bool FOnlineSessionGDK::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->UnregisterPlayer(SessionName, PlayerId);
	}
	else
	{
		return OnlineSessionMpsdGDK->UnregisterPlayer(SessionName, PlayerId);
	}
}

bool FOnlineSessionGDK::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) 
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->UnregisterPlayers(SessionName, Players);
	}
	else
	{
		return OnlineSessionMpsdGDK->UnregisterPlayers(SessionName, Players);
	}
}

void FOnlineSessionGDK::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->RegisterLocalPlayer(PlayerId, SessionName, Delegate);
	}
	else
	{
		OnlineSessionMpsdGDK->RegisterLocalPlayer(PlayerId, SessionName, Delegate);
	}
}

void FOnlineSessionGDK::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->UnregisterLocalPlayer(PlayerId, SessionName, Delegate);
	}
	else
	{
		OnlineSessionMpsdGDK->UnregisterLocalPlayer(PlayerId, SessionName, Delegate);
	}
}

int32 FOnlineSessionGDK::GetNumSessions()
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->GetNumSessions();
	}
	else
	{
		return OnlineSessionMpsdGDK->GetNumSessions();
	}
}

void FOnlineSessionGDK::DumpSessionState()
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->DumpSessionState();
	}
	else
	{
		OnlineSessionMpsdGDK->DumpSessionState();
	}
}

void FOnlineSessionGDK::Tick(float DeltaTime)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		OnlineSessionMpaGDK->Tick(DeltaTime);
	}
	else
	{
		OnlineSessionMpsdGDK->Tick(DeltaTime);
	}
}

FNamedOnlineSessionRef FOnlineSessionGDK::AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->AddNamedSessionRef(SessionName, SessionSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->AddNamedSessionRef(SessionName, SessionSettings);
	}
}

FNamedOnlineSessionRef FOnlineSessionGDK::AddNamedSessionRef(FName SessionName, const FOnlineSession& Session)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->AddNamedSessionRef(SessionName, Session);
	}
	else
	{
		return OnlineSessionMpsdGDK->AddNamedSessionRef(SessionName, Session);
	}
}

FNamedOnlineSession* FOnlineSessionGDK::AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->AddNamedSession(SessionName, SessionSettings);
	}
	else
	{
		return OnlineSessionMpsdGDK->AddNamedSession(SessionName, SessionSettings);
	}
}

FNamedOnlineSession* FOnlineSessionGDK::AddNamedSession(FName SessionName, const FOnlineSession& Session)
{
	if (CVarXboxMpaEnabled.GetValueOnAnyThread())
	{
		return OnlineSessionMpaGDK->AddNamedSession(SessionName, Session);
	}
	else
	{
		return OnlineSessionMpsdGDK->AddNamedSession(SessionName, Session);
	}
}


#endif //WITH_GRDK