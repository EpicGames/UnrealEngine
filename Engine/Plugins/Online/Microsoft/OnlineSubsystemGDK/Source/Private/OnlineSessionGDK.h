// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDKPackage.h"

class FOnlineSessionMpsdGDK;
class FOnlineSessionMpaGDK;

class FOnlineSessionGDK : public IOnlineSession, public TSharedFromThis<FOnlineSessionGDK, ESPMode::ThreadSafe>
{
public:
	~FOnlineSessionGDK();

	FNamedOnlineSessionPtr GetNamedSessionPtr(FName SessionName) const;

	// IOnlineSession interface
	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;
	virtual FNamedOnlineSession* GetNamedSession(FName SessionName) override;
	virtual void RemoveNamedSession(FName SessionName) override;
	virtual bool HasPresenceSession() override;
	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override;
	virtual bool CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool StartSession(FName SessionName) override;
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)  override;
	virtual bool EndSession(FName SessionName) override;
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;
	virtual bool CancelFindSessions() override;
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;
	virtual bool JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)  override;
	virtual bool GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)  override;
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)  override;
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false)  override;
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)  override;
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)  override;
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual int32 GetNumSessions() override;
	virtual void DumpSessionState() override;

	virtual FDelegateHandle AddOnMatchmakingCompleteDelegate_Handle(const FOnMatchmakingCompleteDelegate& Delegate) override;
	virtual void ClearOnMatchmakingCompleteDelegate_Handle(FDelegateHandle& DelegateHandle) override;
	virtual void TriggerOnMatchmakingCompleteDelegates(FName SessionName, bool bWasSuccessful) override;

	virtual FDelegateHandle AddOnCancelMatchmakingCompleteDelegate_Handle(const FOnCancelMatchmakingCompleteDelegate& Delegate) override;
	virtual void ClearOnCancelMatchmakingCompleteDelegate_Handle(FDelegateHandle& DelegateHandle) override;
	virtual void TriggerOnCancelMatchmakingCompleteDelegates(FName SessionName, bool bWasSuccessful) override;

PACKAGE_SCOPE:
	FOnlineSessionGDK(class FOnlineSubsystemGDK* InSubsystem);

	bool IsMpaEnabled() const;

	const TSharedPtr<FOnlineSessionMpsdGDK>& GetMpsdImpl() const;
	const TSharedPtr<FOnlineSessionMpaGDK>& GetMpaImpl() const;

	void Tick(float DeltaTime);

	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSession& Session);
	virtual FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override;
	virtual FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override;

	void ClearSessionState(FGDKContextHandle GDKContext, FUniqueNetIdGDKRef NetID);

	static const int MAX_RETRIES = 20;

protected:
	void SaveInviteFromActivation(const FString& ActivationUri);

	TSharedPtr<FOnlineSessionMpsdGDK> OnlineSessionMpsdGDK;
	TSharedPtr<FOnlineSessionMpaGDK> OnlineSessionMpaGDK;

	FDelegateHandle InviteAcceptedHandler;

	class FOnlineSubsystemGDK* GDKSubsystem;
};

typedef TSharedPtr<FOnlineSessionGDK, ESPMode::ThreadSafe> FOnlineSessionGDKPtr;
