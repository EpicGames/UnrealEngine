// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineSessionInterfaceMpaGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaDeleteActivity.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaGetActivities.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaSetActivity.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMpaSendInvites.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserPrivilege.h"
#include "CoreGlobals.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
THIRD_PARTY_INCLUDES_END

#ifdef UE_PLAYFAB_MATCHMAKING
#include "PlayFabInterface.h"

#define PLAYER_ENTITY "title_player_account"
#endif

#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#endif //WITH_ENGINE
#include "GDKRuntimeModule.h"
#include "Interfaces/VoiceInterface.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineSessionGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SocketSubsystem.h"
#include "PlatformHttp.h"

THIRD_PARTY_INCLUDES_START
#include <grdk.h>
THIRD_PARTY_INCLUDES_END

namespace Private
{

static TAutoConsoleVariable<bool> CVarGdkMpaFindSessionByIdEnabled(
	TEXT("gdk.MpaFindSessionByIdEnabled"),
	true,
	TEXT("When enabled, FindSessionById will get MPA activity as session find result."));

}

/** 
 * GDK Session information wrapper, as well as convenient place to store Host Addr as well as map of xuid-FOnlineAssociateGDK
 */
class FOnlineSessionInfoMpaGDK : public FOnlineSessionInfo
{
public:
	FOnlineSessionInfoMpaGDK()
		: IsReady(false)
		, OwnerId(FUniqueNetIdGDK::EmptyId())
		, SessionId(FUniqueNetIdString::Create(FString::FromInt(IDIncrement), GDK_SUBSYSTEM))
	{
		IDIncrement++;
	}

	virtual const uint8* GetBytes() const override
	{
		check(false);  // NOTIMPL for GDK
		return NULL;
	}

	virtual int32 GetSize() const override
	{
		return sizeof(FOnlineSessionInfoMpaGDK);
	}

	virtual bool IsValid() const override
	{
		return true;
	}

	virtual FString ToString() const override
	{
		return "";
	}

	virtual FString ToDebugString() const override
	{
		return "";
	}

	virtual const FUniqueNetId& GetSessionId() const override
	{
		return *SessionId;
	}

	void SetOwnerId(FUniqueNetIdRef InOwnerId)
	{
		OwnerId = InOwnerId;
	}

	const FUniqueNetIdRef& GetOwnerId() const
	{
		return OwnerId;
	}

	TSharedPtr<class FInternetAddr> GetHostAddr() const
	{
		return HostAddr;
	}

	void SetHostAddr(TSharedPtr<class FInternetAddr> InAddr)
	{
		HostAddr = InAddr;
	}

	bool IsSessionReady() const
	{
		return IsReady;
	}

	void SetSessionReady()
	{
		IsReady = true;
	}

	bool ContainsLocalPlayer(uint64 Xuid) const
	{
		return RegisteredLocalXuids.Contains(Xuid);
	}

	void RegisterLocalPlayer(uint64 Xuid)
	{
		RegisteredLocalXuids.Add(Xuid);
	}

	void UnregisterLocalPlayer(uint64 Xuid)
	{
		RegisteredLocalXuids.Remove(Xuid);
	}

private:
	bool IsReady;

	TSharedPtr<class FInternetAddr> HostAddr;

	TSet<uint64> RegisteredLocalXuids;

	FUniqueNetIdRef OwnerId;

	/** Placeholder */
	FUniqueNetIdStringRef SessionId;

	static uint64 IDIncrement;
};

uint64 FOnlineSessionInfoMpaGDK::IDIncrement = 0;

typedef TSharedPtr<FOnlineSessionInfoMpaGDK> FOnlineSessionInfoMpaGDKPtr;


FOnlineSessionMpaGDK::FOnlineSessionMpaGDK(class FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
{
}

void FOnlineSessionMpaGDK::ClearSessionState(FGDKContextHandle GDKContext, FUniqueNetIdGDKRef NetID)
{
	// Defensive activity clear on login, blank callback because we don't care about the result.
	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKMpaDeleteActivity>(GDKSubsystem, GDKContext, NetID, FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete::CreateLambda([](bool Success) {}));
}

FUniqueNetIdPtr FOnlineSessionMpaGDK::CreateSessionIdFromString(const FString& SessionIdStr)
{
	FUniqueNetIdPtr SessionId;
	if (!SessionIdStr.IsEmpty())
	{
		SessionId = FUniqueNetIdString::Create(SessionIdStr, GDK_SUBSYSTEM);
	}
	return SessionId;
}

FNamedOnlineSession* FOnlineSessionMpaGDK::GetNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return &Session.Get();
		}
	}
	return nullptr;
}

FNamedOnlineSessionPtr FOnlineSessionMpaGDK::GetNamedSessionPtr(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return Session;
		}
	}
	return nullptr;
}

void FOnlineSessionMpaGDK::RemoveNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex]->SessionName == SessionName)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Removing Session %s"), *SessionName.ToString());
			Sessions.RemoveAtSwap(SearchIndex);
			return;
		}
	}
}

bool FOnlineSessionMpaGDK::HasPresenceSession() const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionSettings.bUsesPresence)
		{
			return true;
		}
	}

	return false;
}

EOnlineSessionState::Type FOnlineSessionMpaGDK::GetSessionState(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return Session->SessionState;
		}
	}

	return EOnlineSessionState::NoSession;
}

bool FOnlineSessionMpaGDK::CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	FUniqueNetIdPtr UniqueId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(HostingPlayerControllerIndex);
	if (!UniqueId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("Couldn't find unique id for HostingPlayerNum %d"), HostingPlayerControllerIndex);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	return CreateSession(*UniqueId, SessionName, NewSessionSettings);
}

bool FOnlineSessionMpaGDK::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// Check for an existing session
	if (GetNamedSessionPtr(SessionName).IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	FUniqueNetIdGDKRef HostGDKId = FUniqueNetIdGDK::Cast(HostingPlayerId);

	FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete CreateSessionCompleteDelegate = FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName, FUniqueNetIdGDKRef HostGDKId, FOnlineSessionSettings NewSessionSettings)
	{
		if (!bWasSuccessful)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
			return;
		}

		FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
		if (!NamedSession.IsValid())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
			return;
		}

		NamedSession->bHosting = true;
		NamedSession->HostingPlayerNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(*HostGDKId);

		FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = MakeShared<FOnlineSessionInfoMpaGDK>();
		GDKSessionInfo->SetSessionReady();
		GDKSessionInfo->SetOwnerId(HostGDKId);
		NamedSession->SessionInfo  = GDKSessionInfo;
		NamedSession->SessionState = EOnlineSessionState::Pending;

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
	}, HostGDKId, NewSessionSettings);

	// Create a new session and deep copy the game settings
	FNamedOnlineSessionRef Session = AddNamedSessionRef(SessionName, NewSessionSettings);
	Session->SessionState = EOnlineSessionState::Creating;
	Session->bHosting = true;
	Session->LocalOwnerId = HostGDKId;

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(HostingPlayerId);
	uint64 GDKUserId;

	if (!GDKContext || FAILED(XblContextGetXboxUserId(GDKContext, &GDKUserId)))
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("Failed to create async create session operation"));
		RemoveNamedSession(SessionName);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	FString ConnectionString;
	NewSessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaSetActivity>(
		GDKSubsystem,
		GDKContext,
		FUniqueNetIdGDK::Create(GDKUserId),
		SessionName,
		ConnectionString,
		"",
		1, // The owner
		true,
		NewSessionSettings,
		CreateSessionCompleteDelegate);

	return true;
}

bool FOnlineSessionMpaGDK::StartSession(FName SessionName)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_StartSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, false);
		return false;
	}

	// Can't start a match multiple times
	if (Session->SessionState != EOnlineSessionState::Pending &&
		Session->SessionState != EOnlineSessionState::Ended)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online session (%s) in state %s"),
			*SessionName.ToString(),
			EOnlineSessionState::ToString(Session->SessionState));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_StartSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, false);
		return false;
	}

	// TODO: Handle join in progress vs. not join in progress.
	Session->SessionState = EOnlineSessionState::InProgress;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_StartSession_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, true);
	return true;
}

#ifdef UE_PLAYFAB_MATCHMAKING
void FOnlineSessionMpaGDK::ShareMatchmakingString(FString& ConnectString)
{
	if (MMState.MMLocalPlayers.Num() == 0)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpaGDK::ShareMatchmakingString No local players"));
		return;
	}
	FUniqueNetIdGDKRef FirstUserNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(MMState.MMLocalPlayers[0]);

	if (HaveEntityIdForXuid(FirstUserNetId->ToUint64()))
	{
		const auto ConnectStringConv = StringCast<UTF8CHAR>(*ConnectString);
		const char* const ConnectStringPtr = reinterpret_cast<const char*>(ConnectStringConv.Get());
		const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
		const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());

		PFLobbyDataUpdate LobbyUpdate{nullptr,nullptr,nullptr,nullptr,0,nullptr,nullptr,1,&KeyPtr,&ConnectStringPtr};

		PFEntityKey LocalUserEntity;
		LocalUserEntity.id = GetEntityIdForXuid(FirstUserNetId->ToUint64());
		LocalUserEntity.type = PLAYER_ENTITY;

		HRESULT Result = PFLobbyPostUpdate(MMState.MMLobby,&LocalUserEntity,&LobbyUpdate,nullptr,nullptr);

		if(FAILED(Result))
		{
			CancelMatchmaking(FirstUserNetId.Get(), MMState.MMSessionName);
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpaGDK::ShareMatchmakingString Lobby update Failed"));
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpaGDK::ShareMatchmakingString Success"));
		}
	}
	else
	{
		CancelMatchmaking(FirstUserNetId.Get(), MMState.MMSessionName);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpaGDK::ShareMatchmakingString No Playfab Entity found for user"));
	}
}

#endif // UE_PLAYFAB_MATCHMAKING

bool FOnlineSessionMpaGDK::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpaGDK::UpdateSession %s"), *SessionName.ToString());
	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to update session %s, it does not exist"), *SessionName.ToString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	EOnlineSessionState::Type SessionState = NamedSession->SessionState;
	if (SessionState <= EOnlineSessionState::Creating || SessionState >= EOnlineSessionState::Destroying)
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName, SessionState]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to update session %s, it is state %s, which may not be updated"), *SessionName.ToString(), EOnlineSessionState::ToString(SessionState));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	if (!bShouldRefreshOnlineData)
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName, SessionState]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, true);
		});
		return true;
	}

	NamedSession->SessionSettings = UpdatedSessionSettings;

	FGDKContextHandle GDKContext;
	const FUniqueNetIdPtr& HostNetId(NamedSession->OwningUserId);
	if (HostNetId.IsValid())
	{
		GDKContext = GDKSubsystem->GetGDKContext(*HostNetId);
	}

	if (!GDKContext.IsValid())
	{
		const FUniqueNetIdPtr& LocalOwnerId(NamedSession->LocalOwnerId);
		if (LocalOwnerId.IsValid())
		{
			GDKContext = GDKSubsystem->GetGDKContext(*LocalOwnerId);
		}
	}

	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	uint64 GDKUserId;
	if (FAILED(XblContextGetXboxUserId(GDKContext, &GDKUserId)))
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	FUniqueNetIdGDKRef HostGDKId = FUniqueNetIdGDK::Cast(*NamedSession->LocalOwnerId);

	FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete UpdateSessionCompleteDelegate = FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName, FUniqueNetIdGDKRef HostGDKId, FOnlineSessionSettings NewSessionSettings)
	{
		if (!bWasSuccessful)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
			return;
		}

		FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
		if (!NamedSession.IsValid())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
			return;
		}

		FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(NamedSession->SessionInfo);
		GDKSessionInfo->SetSessionReady();
		GDKSessionInfo->SetOwnerId(HostGDKId);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_UpdateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, true);
	}, HostGDKId, UpdatedSessionSettings);

	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(NamedSession->SessionInfo);

	FString ConnectionString;
	UpdatedSessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

#ifdef UE_PLAYFAB_MATCHMAKING
	if(!ConnectionString.IsEmpty() && MMState.MMSessionName == SessionName)
	{
		ShareMatchmakingString(ConnectionString);
	}
#endif // UE_PLAYFAB_MATCHMAKING

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaSetActivity>(
		GDKSubsystem,
		GDKContext,
		FUniqueNetIdGDK::Create(GDKUserId),
		SessionName,
		ConnectionString,
		"",
		NamedSession->RegisteredPlayers.Num(),
		true,
		UpdatedSessionSettings,
		UpdateSessionCompleteDelegate);

	return true;
}

bool FOnlineSessionMpaGDK::EndSession(FName SessionName)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_EndSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, false);
		return false;
	}

	if (Session->SessionState != EOnlineSessionState::InProgress)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
			*SessionName.ToString(),
			EOnlineSessionState::ToString(Session->SessionState));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_EndSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, false);
		return false;
	}

	Session->SessionState = EOnlineSessionState::Ended;
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_EndSession_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, true);
	return true;
}

bool FOnlineSessionMpaGDK::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	UE_LOG_ONLINE_SESSION(Log, TEXT("Attempting to destroy session %s"), *SessionName.ToString());

	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_NamedSessionInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, false);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_NamedSessionInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			}
		});
		return false;
	}

	if (Session->SessionState == EOnlineSessionState::Destroying)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Already in process of destroying session (%s)"), *SessionName.ToString());
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_SessionStateDestroying_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, false);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_SessionStateDestroying_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			}
		});
		return false;
	}

	Session->SessionState = EOnlineSessionState::Destroying;


	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(Session->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Destroying an online session (%s) will null GDK info. No writes to the MPA will occur."), *SessionName.ToString());
		RemoveNamedSession(SessionName);
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_SessionInfoInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, true);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_SessionInfoInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, true);
			}
		});
		return true;
	}

	if (!GDKSessionInfo->IsSessionReady())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Destroying a session with a null GDK MultiplayerSession (%s)"), *SessionName.ToString());
		RemoveNamedSession(SessionName);
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_GDKMultiplayerSessionHandleInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, true);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_GDKMultiplayerSessionHandleInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, true);
			}
		});
		return true;
	}

	FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete DestroySessionCompleteDelegate = FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName, FOnDestroySessionCompleteDelegate CompletionDelegate)
	{
		RemoveNamedSession(SessionName);

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_CompleteDelegate);
			CompletionDelegate.ExecuteIfBound(SessionName, bWasSuccessful);
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_Delegates);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, bWasSuccessful);
		}
	}, SessionName, CompletionDelegate);

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*GDKSessionInfo->GetOwnerId());

	uint64 GDKUserId;
	if (FAILED(XblContextGetXboxUserId(GDKContext, &GDKUserId)))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Destroying a session with a null GDKUserId (%s)"), *SessionName.ToString());
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
			{
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_GDKUserIdInvalid_CompletionDelegate);
					CompletionDelegate.ExecuteIfBound(SessionName, true);
				}
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_DestroySession_GDKUserIdInvalid_CompletionDelegate);
					GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, true);
				}
			});
		return true;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaDeleteActivity>(GDKSubsystem, GDKContext, FUniqueNetIdGDK::Create(GDKUserId), DestroySessionCompleteDelegate);

	return true;
}

bool FOnlineSessionMpaGDK::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(GDKSubsystem->GetSessionInterface().Get(), SessionName, UniqueId);
}

bool FOnlineSessionMpaGDK::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{

#ifdef UE_PLAYFAB_MATCHMAKING
	HRESULT Result = S_OK;

	if (!PlayfabHandle) // Lazy initialize the PFMultiplayer library.
	{
		FString PlayFabAppId;
		if (GConfig->GetString(TEXT("PlayFab"), TEXT("AppId"), PlayFabAppId, GEngineIni))
		{
#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
			Result = PFMultiplayerInitialize(TCHAR_TO_UTF8(*PlayFabAppId), &PlayfabHandle);
#else
			auto ConvertedAppId = StringCast<ANSICHAR>(*PlayFabAppId);
			const MultiplayerInitializationConfiguration Config = 
			{
				.titleId = ConvertedAppId.Get(),
			};
			Result = PFMultiplayerInitialize(&Config, &PlayfabHandle);

#endif
			if (FAILED(Result))
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Playfab Multiplyer failed to initialize. Matchmaking unvailable"));
				return false;
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Playfab Multiplyer is not configured. No AppidId"));
			return false;
		}
	}

	//Put first player in control before making the ticket.
	FUniqueNetIdGDKRef FirstUserNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(LocalPlayers[0]);
	if (HaveEntityIdForXuid(FirstUserNetId->ToUint64()))
	{
		PFEntityKey LocalUserEntity;

		LocalUserEntity.id = GetEntityIdForXuid(FirstUserNetId->ToUint64());
		LocalUserEntity.type = PLAYER_ENTITY;
		Result = PFMultiplayerSetEntityToken(PlayfabHandle, &LocalUserEntity, GetEntityTokenForXuid(FirstUserNetId->ToUint64()));
		if (FAILED(Result))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Playfab Failed to set entity token"));
			return false;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No Playfab entity ID for local user"));
		return false;
	}

	return CreateMatchmakingTicket(LocalPlayers, SessionName, NewSessionSettings);
	
#else // UE_PLAYFAB_MATCHMAKING 
	UE_LOG_ONLINE_SESSION(Warning, TEXT("This MPA implementation is supposed be used with external matchmaking. Set EnablePlayfabMatchmaking=true in XSX.ini or use Playfab OSS plugin instead for platform matchmaking if needed: https://github.com/PlayFab/PlayFabMultiplayerUnreal"));
	return false;
#endif // UE_PLAYFAB_MATCHMAKING 

}
#ifdef UE_PLAYFAB_MATCHMAKING
bool FOnlineSessionMpaGDK::CreateMatchmakingTicket(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	FString MatchHopperName;
	const FOnlineSessionSetting* HopperSetting = NewSessionSettings.Settings.Find(SETTING_MATCHING_HOPPER);
	if (HopperSetting)
	{
		HopperSetting->Data.GetValue(MatchHopperName);
	}
	const auto QueueName = StringCast<UTF8CHAR>(*MatchHopperName);

	PFMatchmakingTicketConfiguration Configuration{};
	Configuration.queueName = (const char*)QueueName.Get();
	Configuration.timeoutInSeconds = 120;
	TArray<PFEntityKey> Keys;
	TArray<char*> Attributes;
	char Placeholder[] = "";
	uint64 Xuid = 0;
	for (int32 i = 0; i < LocalPlayers.Num(); ++i)
	{
		Xuid = StaticCastSharedRef<const FUniqueNetIdGDK>(LocalPlayers[i])->ToUint64();
		if (HaveEntityIdForXuid(Xuid))
		{
			Keys.Emplace(GetEntityIdForXuid(Xuid), "title_player_account");
			Attributes.Add(Placeholder); // We don't use this, but it can't be null.
		}
	}	
	Attributes.SetNum(Keys.Num());
	MMState.Reset();

	HRESULT Result = PFMultiplayerCreateMatchmakingTicket(
		PlayfabHandle,
		static_cast<uint32_t>(Keys.Num()),
		Keys.GetData(),
		Attributes.GetData(),
		&Configuration,
		nullptr,
		&MMState.MMticket);

	if(FAILED(Result))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to create matchmaking ticket"));
		return false;
	}

	
	MMState.MMLocalPlayers = LocalPlayers;
	MMState.MMSessionSettings = NewSessionSettings;  
	MMState.MMSessionName = SessionName;
	return true;
}
#endif // UE_PLAYFAB_MATCHMAKING

#ifdef UE_PLAYFAB_MATCHMAKING
bool FOnlineSessionMpaGDK::CleanupMatchmaking()
{
	if (MMState.MMticket)
	{
		HRESULT Result = PFMultiplayerDestroyMatchmakingTicket(PlayfabHandle, MMState.MMticket);
		bool bSuccess = SUCCEEDED(Result);
		UE_CLOG_ONLINE_SESSION(!bSuccess, Warning, TEXT("Failed to cleanup Playfab Matchamking ticket. Result = (0x% 0.8X)"), Result);
		if (bSuccess)
		{
			MMState.Reset();
		}
		return bSuccess;
	}
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to cleanup Playfab Matchamking. No Matchmaking in progress."));
	return false;
}
#endif // UE_PLAYFAB_MATCHMAKING

bool FOnlineSessionMpaGDK::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
#ifdef UE_PLAYFAB_MATCHMAKING
	bool bSuccess = CleanupMatchmaking();
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCancelMatchmakingCompleteDelegates(MMState.MMSessionName, bSuccess);
	return bSuccess;
#else
	UE_LOG_ONLINE_SESSION(Warning, TEXT("This MPA implementation is supposed be used with game customized matchmaking. Use Playfab OSS plugin instead for platform matchmaking if needed: https://github.com/PlayFab/PlayFabMultiplayerUnreal"));
	return false;
#endif // UE_PLAYFAB_MATCHMAKING
}

bool FOnlineSessionMpaGDK::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
#ifdef UE_PLAYFAB_MATCHMAKING
	bool bSuccess = CleanupMatchmaking();
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCancelMatchmakingCompleteDelegates(MMState.MMSessionName, bSuccess);
	return bSuccess;
#else
	UE_LOG_ONLINE_SESSION(Warning, TEXT("This MPA implementation is supposed be used with game customized matchmaking. Use Playfab OSS plugin instead for platform matchmaking if needed: https://github.com/PlayFab/PlayFabMultiplayerUnreal"));
	return false;
#endif // UE_PLAYFAB_MATCHMAKING
}

bool FOnlineSessionMpaGDK::FindSessions(int32 SearchingPlayerControllerIndex, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// Unsupported
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring GDK Session Search request, no such feature when use MPA."));
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindSessionsCompleteDelegates(false);
	return false;
}

bool FOnlineSessionMpaGDK::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// Unsupported
	UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring GDK Session Search request, no such feature when use MPA."));
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindSessionsCompleteDelegates(false);
	return false;
}

bool FOnlineSessionMpaGDK::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	if (!Private::CVarGdkMpaFindSessionByIdEnabled.GetValueOnAnyThread())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::FindSessionById] Ignoring GDK Session find request, no such feature when use MPA"));
		GDKSubsystem->ExecuteNextTick([this, CompletionDelegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_FindSessionById_Delegate);
			CompletionDelegate.ExecuteIfBound(0, false, FOnlineSessionSearchResult());
		});
		return false;
	}

	int32 LocalUserNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(SearchingUserId);

	if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::FindSessionById] Failed to find LocalUserNum"));
		CompletionDelegate.ExecuteIfBound(0, false/*bWasSuccessful*/, FOnlineSessionSearchResult());
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::FindSessionById] Failed to find GDKContext"));
		CompletionDelegate.ExecuteIfBound(LocalUserNum, false/*bWasSuccessful*/, FOnlineSessionSearchResult());
		return false;
	}

	TArray<uint64> FriendListXuids;
	FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(FriendId);
	FriendListXuids.Add(GDKFriend->ToUint64());

	FOnlineAsyncTaskGDKMpaGetActivities::FOnComplete GetActivitiesCompleteDelegate = FOnlineAsyncTaskGDKMpaGetActivities::FOnComplete::CreateLambda([this, LocalUserNum, CompletionDelegate](bool bWasSuccessful, TArray<FOnlineSession>&& OnlineSessions) {
		if (!bWasSuccessful || OnlineSessions.IsEmpty())
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::FindSessionById] Failed to get MPA activity"));
			CompletionDelegate.ExecuteIfBound(LocalUserNum, false/*bWasSuccessful*/, FOnlineSessionSearchResult());
			return;
		}

		UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionMpaGDK::FindSessionById] Retrieved latest MPA activity"));
		FOnlineSession OnlineSession = MoveTemp(OnlineSessions[0]);
		OnlineSession.SessionInfo = MakeShared<FOnlineSessionInfoMpaGDK>();

		FOnlineSessionSearchResult SearchResult;
		SearchResult.Session = MoveTemp(OnlineSession);
		CompletionDelegate.ExecuteIfBound(LocalUserNum, true/*bWasSuccessful*/, SearchResult);
	});

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaGetActivities>(GDKSubsystem, GDKContext, FriendListXuids, GetActivitiesCompleteDelegate);
	return true;
}

bool FOnlineSessionMpaGDK::CancelFindSessions()
{
	// Unsupported
	GDKSubsystem->ExecuteNextTick([this]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_CancelFindSessions_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCancelFindSessionsCompleteDelegates(false);
	});

	return false;
}

bool FOnlineSessionMpaGDK::JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	FUniqueNetIdPtr UniqueId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(ControllerIndex);
	if (!UniqueId.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([SessionName, ControllerIndex, this]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("JoinSession failed; unable to find player at index %d"), ControllerIndex);
			#ifdef UE_PLAYFAB_MATCHMAKING
			if (MMState.MMticket != nullptr)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, false);
				CleanupMatchmaking();
			}
			else
			#endif// UE_PLAYFAB_MATCHMAKING
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
			}
		});
		return false;
	}

	return JoinSession(*UniqueId, SessionName, DesiredSession);
}

bool FOnlineSessionMpaGDK::JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	UE_LOG_ONLINE_SESSION(Log, TEXT("User %s Attempting to join session %s"), *UserId.ToDebugString(), *SessionName.ToString());

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(UserId);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session %s failed, user %s has no GDK context"), *SessionName.ToString(), *UserId.ToDebugString());
		GDKSubsystem->ExecuteNextTick([SessionName, this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
		});
		return false;
	}

	// work out if we're already in the session of this name or not
	FNamedOnlineSessionPtr NamedSessionCheck = GetNamedSessionPtr(SessionName);
	if (NamedSessionCheck.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session failed; already in session of type %s, you must leave session before joining"), *SessionName.ToString());

		GDKSubsystem->ExecuteNextTick([SessionName, this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::AlreadyInSession);
		});
		return false;
	}

	const FOnlineSessionSettings& SessionSettings = DesiredSession.Session.SessionSettings;

	FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete JoinSessionCompleteDelegate = FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName, FUniqueNetIdGDKRef HostGDKId, FOnlineSessionSettings NewSessionSettings)
	{
		if (!bWasSuccessful)
		{
#ifdef UE_PLAYFAB_MATCHMAKING
			if (MMState.MMticket != nullptr)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, false);
				CleanupMatchmaking();
			}
			else
#endif // UE_PLAYFAB_MATCHMAKING
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_JoinSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);				
			}
			return;
		}

		FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
		if (!NamedSession.IsValid())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::SessionDoesNotExist);
			return;
		}

		FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(NamedSession->SessionInfo);
		if (!GDKSessionInfo.IsValid() || !GDKSessionInfo->IsValid())
		{
			GDKSessionInfo = MakeShared<FOnlineSessionInfoMpaGDK>();
		}

		GDKSessionInfo->SetSessionReady();
		GDKSessionInfo->SetOwnerId(HostGDKId);
		NamedSession->SessionState = EOnlineSessionState::Pending;
#ifdef UE_PLAYFAB_MATCHMAKING
		if (MMState.MMticket != nullptr)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, true);
			CleanupMatchmaking();
		}
		else
#endif // UE_PLAYFAB_MATCHMAKING
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::Success);
		}
	}, FUniqueNetIdGDK::Create(UserId), SessionSettings);

	// Check for a Join from URI (Join In Progress from Matchmade Sessions)
	FString SessionURI;
	if (SessionSettings.Get(SETTING_GAME_SESSION_URI, SessionURI))
	{
		FNamedOnlineSessionRef NamedSession = AddNamedSessionRef(SessionName, SessionSettings);

		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaSetActivity>(
			GDKSubsystem,
			GDKContext,
			FUniqueNetIdGDK::Cast(UserId),
			SessionName,
			SessionURI,
			"",
			1,
			true,
			SessionSettings,
			JoinSessionCompleteDelegate);
		return true;
	}

	// Create a named session from the search result data
	FNamedOnlineSessionRef NamedSession = AddNamedSessionRef(SessionName, DesiredSession.Session);
	NamedSession->HostingPlayerNum = INDEX_NONE;
	NamedSession->LocalOwnerId = FUniqueNetIdGDK::Cast(UserId);

	FString ConnectionString;
	SessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaSetActivity>(
		GDKSubsystem,
		GDKContext,
		FUniqueNetIdGDK::Cast(UserId),
		SessionName,
		ConnectionString,
		"",
		1,
		true,
		SessionSettings,
		JoinSessionCompleteDelegate);

	return true;
}

bool FOnlineSessionMpaGDK::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	TArray<FUniqueNetIdRef> FriendList;
	FriendList.Add(Friend.AsShared());
	return FindFriendSession(LocalUserNum, FriendList);
}

void FOnlineSessionMpaGDK::OnGetActivitiesComplete(bool bWasSuccessful, TArray<FOnlineSession> OnlineSessions, int32 LocalUserNum)
{
	if (!bWasSuccessful)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FindFriendSession: Failed to query Friend's multiplayer activity"));
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnGetActivitiesComplete_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
		return;
	}

	if (OnlineSessions.IsEmpty())
	{
		// Friend has no advertised active session
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("FindFriendSession: Friend has no multiplayer activity"));
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnGetActivitiesComplete_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
		return;
	}

	TArray<FOnlineSessionSearchResult> FriendSessions;
	FriendSessions.Reserve(OnlineSessions.Num());
	for (FOnlineSession& OnlineSession : OnlineSessions)
	{
		FOnlineSessionSearchResult SearchResult;
		SearchResult.Session = MoveTemp(OnlineSession);
		FriendSessions.Add(SearchResult);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnGetActivitiesComplete_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, true, FriendSessions);
}

bool FOnlineSessionMpaGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	int32 ControllerId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(LocalUserId);
	if (ControllerId < 0 || ControllerId >= MAX_LOCAL_PLAYERS)
	{
		GDKSubsystem->ExecuteNextTick([this, ControllerId]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_FindFriendSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(ControllerId, false, TArray<FOnlineSessionSearchResult>());
		});
		return false;
	}

	TArray<FUniqueNetIdRef> FriendList;
	FriendList.Add(Friend.AsShared());
	return FindFriendSession(ControllerId, FriendList);
}

bool FOnlineSessionMpaGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	int32 ControllerId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(LocalUserId);
	if (ControllerId < 0 || ControllerId >= MAX_LOCAL_PLAYERS)
	{
		GDKSubsystem->ExecuteNextTick([this, ControllerId]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_FindFriendSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(ControllerId, false, TArray<FOnlineSessionSearchResult>());
		});
		return false;
	}

	return FindFriendSession(ControllerId, FriendList);
}

bool FOnlineSessionMpaGDK::FindFriendSession(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& FriendList)
{
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FindFriendSession: Failed to retrieve Friend's multiplayer, no available GDKContext for user %d"), LocalUserNum);
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_FindFriendSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
		return false;
	}

	TArray<uint64> FriendListXuids;
	FriendListXuids.Reserve(FriendList.Num());
	for (FUniqueNetIdRef Friend : FriendList)
	{
		FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(*Friend);
		FriendListXuids.Add(GDKFriend->ToUint64());
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaGetActivities>(GDKSubsystem, GDKContext, FriendListXuids, FOnlineAsyncTaskGDKMpaGetActivities::FOnComplete::CreateThreadSafeSP(this, &FOnlineSessionMpaGDK::OnGetActivitiesComplete, LocalUserNum));
	return true;
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	return SendSessionInviteToFriend_Internal(GDKContext, SessionName, Friend);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	if (!LocalUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite friend to session %s, LocalUserId is invalid"), *SessionName.ToString());
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserId);
	return SendSessionInviteToFriend_Internal(GDKContext, SessionName, Friend);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriend_Internal(FGDKContextHandle GDKContext, FName SessionName, const FUniqueNetId& Friend)
{
	if (!Friend.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
		return false;
	}

	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friend %s to session %s, GDKContext of local user is invalid"), *Friend.ToString(), *SessionName.ToString());
		return false;
	}

	FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(Friend);

	TArray<uint64> FriendsToInvite;
	FriendsToInvite.Add(GDKFriend->ToUint64());

	return SendSessionInviteToFriendsByXuids(GDKContext, SessionName, FriendsToInvite);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	return SendSessionInviteToFriendsByNetIds(GDKSubsystem->GetGDKContext(LocalUserNum), SessionName, Friends);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (!LocalUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite friend to session %s, LocalUserId is invalid"), *SessionName.ToString());
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserId);
	return SendSessionInviteToFriendsByNetIds(GDKContext, SessionName, Friends);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriendsByNetIds(FGDKContextHandle GDKContext, FName SessionName, const TArray<FUniqueNetIdRef>& Friends)
{
	if (Friends.IsEmpty())
	{
		// Return true in this case, but log it since it's strange
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Attempted to invite any empty array of friends to session %s"), *SessionName.ToString());
		return true;
	}

	TArray<uint64> FriendsToInvite;
	for (const FUniqueNetIdRef& Friend : Friends)
	{
		if (!Friend->IsValid())
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
			return false;
		}

		const FUniqueNetIdGDKRef GDKFriend = StaticCastSharedRef<const FUniqueNetIdGDK>(Friend);
		FriendsToInvite.Add(GDKFriend->ToUint64());
	}

	return SendSessionInviteToFriendsByXuids(GDKContext, SessionName, FriendsToInvite);
}

bool FOnlineSessionMpaGDK::SendSessionInviteToFriendsByXuids(FGDKContextHandle GDKContext, FName SessionName, const TArray<uint64>& FriendXuids)
{
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, GDKContext of local user is invalid"), *SessionName.ToString());
		return false;
	}

	FNamedOnlineSessionPtr SessionPtr = GetNamedSessionPtr(SessionName);
	if (!SessionPtr.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session does not exist"), *SessionName.ToString());
		return false;
	}

	if (SessionPtr->SessionState < EOnlineSessionState::Pending || SessionPtr->SessionState > EOnlineSessionState::InProgress)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session is in state %d"), *SessionName.ToString(), SessionPtr->SessionState);
		return false;
	}

	FOnlineSessionInfoMpaGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(SessionPtr->SessionInfo);
	if (!SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session has invalid info"), *SessionName.ToString());
		return false;
	}

	FString ConnectionString;
	SessionPtr->SessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKMpaSendInvites>(GDKSubsystem, GDKContext, FriendXuids, true/*AllowCrossPlatformJoin*/, ConnectionString);
	return true;
}

namespace UE
{
namespace Online
{
namespace Private
{

bool GetConnectStringFromSessionInfo(const FOnlineSessionInfoMpaGDKPtr& SessionInfo, FString& ConnectInfo, int32 PortOverride = 0)
{
	bool bSuccess = false;

	if (SessionInfo.IsValid())
	{
		const TSharedPtr<FInternetAddr> IpAddr = SessionInfo->GetHostAddr();
		if (IpAddr.IsValid() && IpAddr->IsValid())
		{
			if (PortOverride != 0)
			{
				ConnectInfo = FString::Printf(TEXT("%s:%d"), *IpAddr->ToString(false), PortOverride);
			}
			else
			{
				ConnectInfo = FString::Printf(TEXT("%s"), *IpAddr->ToString(true));
			}

			bSuccess = true;
		}
	}

	return bSuccess;
}

}
}
}

bool FOnlineSessionMpaGDK::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccess = false;
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (Session.IsValid())
	{
		FOnlineSessionInfoMpaGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(Session->SessionInfo);
		
		if (SessionInfo.IsValid() && PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(Session->SessionSettings);
			bSuccess = UE::Online::Private::GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			if (SessionInfo.IsValid() && Session->SessionSettings.bIsLANMatch)
			{
				bSuccess = UE::Online::Private::GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
			}
			else
			{
				FOnlineSessionSetting* ConnectSetting = Session->SessionSettings.Settings.Find(SETTING_CUSTOM_JOIN_INFO);
				if (ConnectSetting != nullptr)
				{
					const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
					const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
					ConnectInfo = ConnectSetting->Data.ToString();
					int Pos = ConnectInfo.Find(KeyPtr);
					ConnectInfo = ConnectInfo.Left(Pos);
					bSuccess = !ConnectInfo.IsEmpty();
				}
			}
		}		

		if (!bSuccess)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for session %s in GetResolvedConnectString()"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning,
			TEXT("Unknown session name (%s) specified to GetResolvedConnectString()"),
			*SessionName.ToString());
	}

	return bSuccess;
}


bool FOnlineSessionMpaGDK::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)
{
	bool bSuccess = false;
	if (SearchResult.Session.SessionInfo.IsValid())
	{
		FOnlineSessionInfoMpaGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(SearchResult.Session.SessionInfo);

		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(SearchResult.Session.SessionSettings);
			bSuccess = UE::Online::Private::GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			if (SearchResult.Session.SessionSettings.bIsLANMatch)
			{
				bSuccess = UE::Online::Private::GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
			}
			else
			{
				const FOnlineSessionSetting* ConnectSetting = SearchResult.Session.SessionSettings.Settings.Find(SETTING_CUSTOM_JOIN_INFO);
				if (ConnectSetting != nullptr)
				{
					const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
					const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
					ConnectInfo = ConnectSetting->Data.ToString();
					int Pos = ConnectInfo.Find(KeyPtr);
					ConnectInfo = ConnectInfo.Left(Pos);
					bSuccess = !ConnectInfo.IsEmpty();
				}
			}
		}
	}

	if (!bSuccess || ConnectInfo.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info in search result to GetResolvedConnectString()"));
	}

	return bSuccess;
}

FOnlineSessionSettings* FOnlineSessionMpaGDK::GetSessionSettings(FName SessionName)
{
	FNamedOnlineSessionPtr MySession = GetNamedSessionPtr(SessionName);
	if (!MySession.IsValid())
	{
		return nullptr;
	}

	return &MySession->SessionSettings;
}

bool FOnlineSessionMpaGDK::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray<FUniqueNetIdRef> Players;
	Players.Add(FUniqueNetIdGDK::Cast(PlayerId));
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

bool FOnlineSessionMpaGDK::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
		return false;
	}

	if (!Session->SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to join for session (%s)"), *SessionName.ToString());
		return false;
	}

	for (const FUniqueNetIdRef& PlayerId : Players)
	{
		FUniqueNetIdMatcher PlayerMatch(*PlayerId);
		if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
		{
			Session->RegisteredPlayers.Add(PlayerId);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *PlayerId->ToDebugString(), *SessionName.ToString());
		}
		if (GDKSubsystem->IsLocalPlayer(*PlayerId))
		{
			FSessionSettings* MemberSettings = Session->SessionSettings.MemberSettings.Find(PlayerId);
			if (MemberSettings)
			{
				if (!MemberSettings->Find(FName("Registered")))
				{
					MemberSettings->Add(FName("Registered"), FOnlineSessionSetting(true, EOnlineDataAdvertisementType::ViaOnlineService));
					UE_LOG_ONLINE_SESSION(Log, TEXT("Marking Player %s as registered in session %s"), *PlayerId->ToDebugString(), *SessionName.ToString());
				}
			}
		}

		RegisterVoice(*PlayerId);
	}

	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(Session->SessionInfo);
	check(GDKSessionInfo);
	if (Players.Num() == 1 && *Players[0] == *GDKSessionInfo->GetOwnerId())
	{
		// No need to set activity again if it's registering local session owner
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterPlayers_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, true);
		return true;
	}
	else
	{	//-V523 disabling this pvs warning because the code below is identical to the contents of the if() directly above
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterPlayers_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, true);

		// Not necessary to update currentPlayers since it's not used by xbox system anyway

		return true;
	}
}

void FOnlineSessionMpaGDK::RegisterVoice(const FUniqueNetId& PlayerId)
{
	IOnlineVoicePtr VoiceInt = GDKSubsystem->GetVoiceInterface();

	if (VoiceInt.IsValid())
	{
		if (!GDKSubsystem->IsLocalPlayer(PlayerId))
		{
			VoiceInt->RegisterRemoteTalker(PlayerId);
		}
		else
		{
			int32 LocalUserNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(PlayerId);
			VoiceInt->RegisterLocalTalker(LocalUserNum);
		}
	}
}

bool FOnlineSessionMpaGDK::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray<FUniqueNetIdRef> Players;
	Players.Add(FUniqueNetIdGDK::Cast(PlayerId));
	return UnregisterPlayers(SessionName, Players);
}

bool FOnlineSessionMpaGDK::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
		return false;
	}

	if (!Session->SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to leave for session (%s)"), *SessionName.ToString());
		return false;
	}

	for (const FUniqueNetIdRef& PlayerId : Players)
	{
		FUniqueNetIdMatcher PlayerMatch(*PlayerId);
		int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
		if (RegistrantIndex != INDEX_NONE)
		{
			Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
			UnregisterVoice(*PlayerId);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Player %s is not part of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
		}
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UnregisterPlayers_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, true);

	// Not necessary to update currentPlayers since it's not used by xbox system anyway

	return true;
}

void FOnlineSessionMpaGDK::UnregisterVoice(const FUniqueNetId& PlayerId)
{
	IOnlineVoicePtr VoiceInt = GDKSubsystem->GetVoiceInterface();

	if (VoiceInt.IsValid())
	{
		if (!GDKSubsystem->IsLocalPlayer(PlayerId))
		{
			VoiceInt->UnregisterRemoteTalker(PlayerId);
		}
		else
		{
			const int32 LocalUserNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(PlayerId);
			VoiceInt->UnregisterLocalTalker(LocalUserNum);
		}
	}
}

void FOnlineSessionMpaGDK::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef GDKPlayerId = FUniqueNetIdGDK::Cast(PlayerId);

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*GDKPlayerId);
	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::UnknownError);
		});
		return;
	}

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::SessionDoesNotExist);
		});
		return;
	}

	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(NamedSession->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::UnknownError);
		});
		return;
	}

	if (GDKSessionInfo->ContainsLocalPlayer(GDKPlayerId->ToUint64()))
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::AlreadyInSession);
		});
		return;
	}

	GDKSessionInfo->RegisterLocalPlayer(GDKPlayerId->ToUint64());

	FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete RegisterLocalPlayerCompleteDelegate = FOnlineAsyncTaskGDKMpaSetActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName,  FUniqueNetIdGDKRef GDKPlayerId, FOnRegisterLocalPlayerCompleteDelegate Delegate)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
		Delegate.ExecuteIfBound(*GDKPlayerId, bWasSuccessful ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	}, GDKPlayerId, Delegate);

	FString ConnectionString;
	NamedSession->SessionSettings.Get(SETTING_CUSTOM_JOIN_INFO, ConnectionString);
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaSetActivity>(
		GDKSubsystem,
		GDKContext,
		GDKPlayerId,
		SessionName,
		ConnectionString,
		"",
		NamedSession->RegisteredPlayers.Num(),
		true,
		NamedSession->SessionSettings,
		RegisterLocalPlayerCompleteDelegate);
}

void FOnlineSessionMpaGDK::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef GDKPlayerId = FUniqueNetIdGDK::Cast(PlayerId);
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*GDKPlayerId);
	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpaGDK>(NamedSession->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	if (!GDKSessionInfo->ContainsLocalPlayer(GDKPlayerId->ToUint64()))
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	GDKSessionInfo->UnregisterLocalPlayer(GDKPlayerId->ToUint64());

	FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete UnregisterLocalPlayerCompleteDelegate = FOnlineAsyncTaskGDKMpaDeleteActivity::FOnComplete::CreateLambda([this](bool bWasSuccessful, FUniqueNetIdGDKRef GDKPlayerId, FOnUnregisterLocalPlayerCompleteDelegate Delegate)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_RegisterLocalPlayer_Delegate);
		Delegate.ExecuteIfBound(*GDKPlayerId, bWasSuccessful);
	}, GDKPlayerId, Delegate);

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMpaDeleteActivity>(GDKSubsystem, GDKContext, GDKPlayerId, UnregisterLocalPlayerCompleteDelegate);
}

namespace UE
{
namespace Online
{
namespace Private
{

FString FindUrlParameter(const FString& Uri, const TCHAR* Name)
{
	FString StartToken = FString::Printf(TEXT("%s="), Name);
	int32 Start = Uri.Find(StartToken);
	if (Start == INDEX_NONE)
	{
		return FString();
	}

	Start += StartToken.Len();
	int32 End = Uri.Find(TEXT("&"), ESearchCase::IgnoreCase, ESearchDir::FromStart, Start);

	// If the session is at the end of the string then end will return not found.
	int32 CharCount = MAX_int32;
	if (End != INDEX_NONE)
	{
		CharCount = End - Start;
	}

	return(Uri.Mid(Start, CharCount));
}

}
}
}

void FOnlineSessionMpaGDK::SaveInviteFromActivation(const FString& InActivationUri)
{
	UE_LOG_ONLINE(Log, TEXT("[FOnlineSessionMpaGDK::SaveInviteFromActivation] "));

	FString ActivationUri = FGenericPlatformHttp::UrlDecode(InActivationUri);

	FString ConnectionString = UE::Online::Private::FindUrlParameter(ActivationUri, TEXT("connectionString"));
	if (ConnectionString.IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[FOnlineSessionMpaGDK::SaveInviteFromActivation] Can't find connectionString in the uri: %s"), *ActivationUri);
		return;
	}

	FString SenderXuidString = UE::Online::Private::FindUrlParameter(ActivationUri, TEXT("sender"));
	if (SenderXuidString.IsEmpty())
	{
		SenderXuidString = UE::Online::Private::FindUrlParameter(ActivationUri, TEXT("joineeXuid"));
	}
	FString InvitedXuidString = UE::Online::Private::FindUrlParameter(ActivationUri, TEXT("invitedUser"));
	if (InvitedXuidString.IsEmpty())
	{
		InvitedXuidString = UE::Online::Private::FindUrlParameter(ActivationUri, TEXT("joinerXuid"));
	}

	uint64_t SenderXuid = FCString::Atoi64(*SenderXuidString);
	uint64_t InvitedXuid = FCString::Atoi64(*InvitedXuidString);

	// Trying to retrieve the user immediately could cause errors, since GDK Users might still not be created
	GDKSubsystem->ExecuteNextTick([this, SenderXuid, InvitedXuid, ConnectionString]
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // XUserFindUserById is not safe to call on time-sensitive threads

			FGDKUserHandle GDKUser;
			XUserFindUserById(InvitedXuid, GDKUser.GetInitReference());

			if (GDKUser.IsValid())
			{
				// If the user already exists (GDKUserHandle non null), we process it immediately
				SaveSessionInvite(SenderXuid, GDKUser, ConnectionString);
			}
			else
			{
				// If the user doesn't exist, (a signed in but inactive user), we'll need to add it first
				FGDKLocalTaskBlock Block;
				HRESULT Result = XUserAddByIdWithUiAsync(InvitedXuid, Block);
				if (SUCCEEDED(Result))
				{
					Result = Block.BlockUntilComplete();
					if (SUCCEEDED(Result))
					{
						XUserFindUserById(InvitedXuid, GDKUser.GetInitReference());

						if (GDKUser.IsValid())
						{
							// With this we make sure the User Manager knows and tracks this new user
							IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(GDKUser);

							SaveSessionInvite(SenderXuid, GDKUser, ConnectionString);
						}
					}
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[FOnlineSessionMpaGDK::SaveInviteFromActivation] XUserAddByIdWithUiAsync failed with code 0x%0.8X."), Result);
				}
			}
		});
}

void FOnlineSessionMpaGDK::SaveSessionInvite(uint64 SenderXuid, FGDKUserHandle AcceptingUser, const FString& ConnectionString)
{
	if (!AcceptingUser.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::SaveSessionInvite] AcceptingUser invalid, session invite was not saved"));
		return;
	}

	uint64 UserId;
	ensure(SUCCEEDED(XUserGetId(AcceptingUser, &UserId)));

	UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionMpaGDK::SaveSessionInvite] AcceptingUser=[%lld] ConnectionString=[%s]"), UserId, *ConnectionString);

	PendingInvite = FPendingInviteData();
	PendingInvite.Sender = SenderXuid;
	PendingInvite.AcceptingUser = AcceptingUser;
	PendingInvite.ConnectionString = ConnectionString;
	PendingInvite.LoggedNotProcessedYetTime = FPlatformTime::Seconds() + SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS;
	PendingInvite.bHaveInvite = true;
}

void FOnlineSessionMpaGDK::Tick(float DeltaTime)
{
	TickPendingInvites(DeltaTime);
	TickPendingSessionUserInvites(DeltaTime);
#ifdef UE_PLAYFAB_MATCHMAKING
	TickMatchmaking();
	TickLobby();
#endif // UE_PLAYFAB_MATCHMAKING
}

#ifdef UE_PLAYFAB_MATCHMAKING
void FOnlineSessionMpaGDK::TickLobby()
{
	if (!PlayfabHandle)
	{
		return;
	}
	HRESULT ProcessingResult = S_OK;
	uint32_t StateChangeCount;
	const PFLobbyStateChange* const* StateChanges;
	ProcessingResult = PFMultiplayerStartProcessingLobbyStateChanges(PlayfabHandle, &StateChangeCount, &StateChanges);
	if (FAILED(ProcessingResult))
	{
		return;
	}

	for (uint32 i = 0; i < StateChangeCount; ++i)
	{
		const PFLobbyStateChange& StateChange = *StateChanges[i];

		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] state changed to %d"), StateChange.stateChangeType);

		switch (StateChange.stateChangeType)
		{
		case PFLobbyStateChangeType::JoinArrangedLobbyCompleted:
		{
			const PFLobbyJoinArrangedLobbyCompletedStateChange& JoinStateChange = static_cast<const PFLobbyJoinArrangedLobbyCompletedStateChange&>(StateChange);
			const PFLobby* Lobby = JoinStateChange.lobby;

			// Lobby initialization isn't immediate, we need to wait a tick to make calls using the handle.
			GDKSubsystem->ExecuteNextTick([this,  Lobby]()
				{
					const PFEntityKey* Owner;
					HRESULT Result = PFLobbyGetOwner(Lobby, &Owner);
					if (FAILED(Result))
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::TickLobby] Failed to determine owner Result  0x%08x. Matchmaking will abort "), Result);
						GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, false);
						CleanupMatchmaking();
						return;
					}
					FUniqueNetIdGDKRef FirstUserNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(MMState.MMLocalPlayers[0]);

					if (HaveEntityIdForXuid(FirstUserNetId->ToUint64()))
					{
						//if owner = us then create session
						if (Owner && strcmp(Owner->id,GetEntityIdForXuid(FirstUserNetId->ToUint64())) == 0)
						{
							MMState.bHost = true;
							UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] We are the lobby owner. Creating session."));

							CreateSession(MMState.MMLocalPlayers[0].Get(), MMState.MMSessionName, MMState.MMSessionSettings);
							FNamedOnlineSessionPtr NamedSession = GDKSubsystem->GetSessionInterfaceGDK()->GetNamedSessionPtr(MMState.MMSessionName);
							NamedSession->OwningUserId = MMState.MMLocalPlayers[0];
							QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_TickLobby_Delegate);
							GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, true);
							CleanupMatchmaking();
						}
						else
						{
							UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] We are not the lobby owner. Awaiting session info."));
						}
					}
					else
					{						
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::TickLobby] Error identifying lobby owner. Matchmaking will abort "));
						GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, false);
						CleanupMatchmaking();
						return;						
					}
				});
			break;
		}
		case PFLobbyStateChangeType::Updated:
		{
			const PFLobbyUpdatedStateChange& UpdatedStateChange = static_cast<const PFLobbyUpdatedStateChange&>(StateChange);
			const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
			const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
			
			for(uint32 Key=0; Key < UpdatedStateChange.updatedLobbyPropertyCount;++Key)
			{
				UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] Lobby Property update"));

				if(MMState.bHost)
				{
					break;
				}

				if(strcmp(UpdatedStateChange.updatedLobbyPropertyKeys[Key], KeyPtr)==0)
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] Lobby property updated with session info"));

					const char* Property = nullptr;
					HRESULT Result = PFLobbyGetLobbyProperty(
						UpdatedStateChange.lobby,
						UpdatedStateChange.updatedLobbyPropertyKeys[Key],
						&Property);

					if(FAILED(Result))
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::TickLobby] Lobby property updated failed to get session info. Matchmaking will abort"));
						CancelMatchmaking(0, MMState.MMSessionName);
					}
					else
					{	
						UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickLobby] Lobby property aquired session info"));
						FOnlineSession OnlineSession;
						OnlineSession.OwningUserId = FUniqueNetIdGDK::Create(0); //empty ID. We don't know who the owner is, just that it's not us.
						OnlineSession.SessionSettings.Set(SETTING_CUSTOM_JOIN_INFO, FString(Property));

						FOnlineSessionSearchResult SearchResult;
						SearchResult.Session = OnlineSession;			

						// Join session flow will call matchamiking delegates.
						JoinSession(MMState.MMLocalPlayers[0].Get(), MMState.MMSessionName, SearchResult);						
					}

				}
				}
			break;
		}
		}

	}
	ProcessingResult = PFMultiplayerFinishProcessingLobbyStateChanges(PlayfabHandle, StateChangeCount, StateChanges);
}

void FOnlineSessionMpaGDK::TickMatchmaking()
{
	if (!PlayfabHandle)
	{
		return;
	}
	HRESULT ProcessingResult = S_OK;
	HRESULT TicketResult = E_FAIL;

	uint32_t StateChangeCount = 0;
	const PFMatchmakingStateChange* const* StateChanges;
	ProcessingResult = PFMultiplayerStartProcessingMatchmakingStateChanges(PlayfabHandle, &StateChangeCount, &StateChanges);
	if (FAILED(ProcessingResult))
	{
		return;
	}

	for (uint32 i = 0; i < StateChangeCount; ++i)
	{
		const PFMatchmakingStateChange& StateChange = *StateChanges[i];

		UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickMatchmaking] Matchmaking ticket state changed: %d"), StateChange.stateChangeType);

		switch (StateChange.stateChangeType)
		{
		case PFMatchmakingStateChangeType::TicketStatusChanged:
		{
			const PFMatchmakingTicketStatusChangedStateChange& TicketStatusChanged = static_cast<const PFMatchmakingTicketStatusChangedStateChange&>(StateChange);

			PFMatchmakingTicketStatus Status;
			if (SUCCEEDED(PFMatchmakingTicketGetStatus(TicketStatusChanged.ticket, &Status)))
			{
				UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickMatchmaking] Matchmaking ticket status changed: %d"), Status);
			}
			break;
		}
		case PFMatchmakingStateChangeType::TicketCompleted:
		{
			const PFMatchmakingTicketCompletedStateChange& TicketCompleted = static_cast<const PFMatchmakingTicketCompletedStateChange&>(StateChange);

			UE_LOG_ONLINE_SESSION(Verbose, TEXT("[FOnlineSessionMpaGDK::TickMatchmaking] Matchmaking complete. Result: 0x%08x"), TicketCompleted.result);
			TicketResult = TicketCompleted.result;
			if (FAILED(TicketCompleted.result))
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_TickMatchmaking_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnMatchmakingCompleteDelegates(MMState.MMSessionName, false);
				CleanupMatchmaking();
			}
			break;
		}
		}
	}

	ProcessingResult = PFMultiplayerFinishProcessingMatchmakingStateChanges(PlayfabHandle, StateChangeCount, StateChanges);

	if (FAILED(ProcessingResult))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpaGDK::TickMatchmaking] Matchmaking failed to process state change Error: 0x%08x"), ProcessingResult);
		return;
	}

	if (FAILED(TicketResult))
	{
		return;
	}

	const PFMatchmakingMatchDetails* Match;
	HRESULT Result = PFMatchmakingTicketGetMatch(MMState.MMticket, &Match);
	if (FAILED(Result))
	{
		PFMultiplayerDestroyMatchmakingTicket(PlayfabHandle, MMState.MMticket);
		return;
	}

	FString MatchId = Match->matchId;
	FString LobbyArrangementString = Match->lobbyArrangementString;

	PFLobbyArrangedJoinConfiguration JoinConfig(
		Match->memberCount,
		PFLobbyOwnerMigrationPolicy::Automatic,
		PFLobbyAccessPolicy::Private,
		0,// memberPropertyCount
		nullptr,// memberPropertyKeys;
		nullptr// memberPropertyValues;
	);

	PFEntityKey LocalUserEntity;
	FUniqueNetIdGDKRef FirstUserNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(MMState.MMLocalPlayers[0]);
	if (HaveEntityIdForXuid(FirstUserNetId->ToUint64()))
	{
		LocalUserEntity.id = GetEntityIdForXuid(FirstUserNetId->ToUint64());
		LocalUserEntity.type = "title_player_account";
		Result = PFMultiplayerSetEntityToken(PlayfabHandle, &LocalUserEntity, GetEntityTokenForXuid(FirstUserNetId->ToUint64()));
		if (FAILED(Result))
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Playfab Failed to set entity token"));
			return;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Playfab Failed to set entity token"));
		return;
	}

	Result = PFMultiplayerJoinArrangedLobby(
		PlayfabHandle,
		&LocalUserEntity,
		Match->lobbyArrangementString,
		&JoinConfig,
		nullptr, // optional asyncContext
		&MMState.MMLobby);

	PFMultiplayerDestroyMatchmakingTicket(PlayfabHandle, MMState.MMticket);

	if (FAILED(Result))
	{
		return;
	}

}
#endif // UE_PLAYFAB_MATCHMAKING

void FOnlineSessionMpaGDK::TickPendingInvites(float DeltaTime)
{
	if (!PendingInvite.bHaveInvite)
	{
		return;
	}

	// Warn if we haven't processed this in a timely manner
	if (!PendingInvite.bLoggedNotProcessedYet && PendingInvite.LoggedNotProcessedYetTime < FPlatformTime::Seconds())
	{
		PendingInvite.bLoggedNotProcessedYet = true;
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: Haven't processed the invite in %d seconds. ConvertedNetworkConnectivityLevel=[%s]"), SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS, EOnlineServerConnectionStatus::ToString(GDKSubsystem->ConvertedNetworkConnectivityLevel));
		// Intentional fall through
	}

	if (GDKSubsystem->ConvertedNetworkConnectivityLevel != EOnlineServerConnectionStatus::Connected)
	{
		// Don't process invites until we're fully connected
		return;
	}

	if (!PendingInvite.AcceptingUser.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: bHaveInvite is true but AcceptingUser is null."));
		PendingInvite.bHaveInvite = false;
		return;
	}

	uint64 UserId;
	if (FAILED(XUserGetId(PendingInvite.AcceptingUser, &UserId)))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: bHaveInvite is true but AcceptingUser has no xuid."));
		PendingInvite.bHaveInvite = false;
		return;
	}

	const FPlatformUserId AcceptingUserId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromGDKUser(PendingInvite.AcceptingUser);
	if (AcceptingUserId == PLATFORMUSERID_NONE)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: bHaveInvite is true but unknown player %" UINT64_FMT), UserId);
		PendingInvite.bHaveInvite = false;
		return;
	}

	FGDKContextHandle Context = GDKSubsystem->GetGDKContext(PendingInvite.AcceptingUser);
	if (!Context)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: couldn't create an GDKContext for the AcceptingUser."));
		return;
	}
#ifdef UE_PLAYFAB_MATCHMAKING
	if (!HaveEntityIdForXuid(UserId))// Used to delay invite processing on resume, Playfab takes time to re initialize.
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpaGDK::TickPendingInvites: No PlayfabID for AcceptingUser"));
		return;
	}
#endif //UE_PLAYFAB_MATCHMAKING
	FOnlineSession OnlineSession;
	OnlineSession.OwningUserId = FUniqueNetIdGDK::Create(PendingInvite.Sender);
	OnlineSession.SessionSettings.Set(SETTING_CUSTOM_JOIN_INFO, PendingInvite.ConnectionString);
	
	FOnlineSessionSearchResult SearchResult;
	SearchResult.Session = OnlineSession;
	FOnlineSessionInfoMpaGDKPtr GDKSessionInfo = MakeShared<FOnlineSessionInfoMpaGDK>();

	SearchResult.Session.SessionInfo = GDKSessionInfo;
	SearchResult.Session.SessionSettings.NumPublicConnections = 1; // We don't actually know this, but since we are accepting an invite assume an open space (OssAdapater checks this)

	FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Create(UserId);
	FPlatformUserId PlatformUserId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(*UniqueNetId);
	const int32 LocalUserIndex = GDKSubsystem->GetIdentityGDK()->GetLocalUserNumFromPlatformUserId(PlatformUserId);

	if (GDKSubsystem->GetSessionInterfaceGDK()->OnSessionUserInviteAcceptedDelegates.IsBound())
	{
		ProcessPendingSessionUserInvite(PendingInvite.AcceptingUser, LocalUserIndex, SearchResult);
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionMpaGDK::TickPendingInvites] OnSessionUserInviteAcceptedDelegate not bound yet, saving info for a later attempt."));
		PendingSessionUserInvite = FPendingSessionUserInvite(LocalUserIndex, MakeShared<FOnlineSessionSearchResult>(SearchResult));
	}

	PendingInvite = FPendingInviteData();
}

void FOnlineSessionMpaGDK::TickPendingSessionUserInvites(float DeltaTime)
{
	if (GDKSubsystem->GetSessionInterfaceGDK()->OnSessionUserInviteAcceptedDelegates.IsBound() && PendingSessionUserInvite.IsSet())
	{
		FGDKUserHandle AcceptingUser = GDKSubsystem->GetIdentityGDK()->GetUserForPlatformUserId(PendingSessionUserInvite->AcceptingUserIndex);
		ProcessPendingSessionUserInvite(AcceptingUser, PendingSessionUserInvite->AcceptingUserIndex, *PendingSessionUserInvite->SearchResult);
		PendingSessionUserInvite.Reset();
	}
}

void FOnlineSessionMpaGDK::ProcessPendingSessionUserInvite(FGDKUserHandle AcceptingUser, int32 LocalUserIndex, FOnlineSessionSearchResult const& SearchResult)
{
	FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Create(AcceptingUser);

	bool bSkipPrivilegeCheckOnSessionJoin = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bSkipPrivilegeCheckOnSessionJoin"), bSkipPrivilegeCheckOnSessionJoin, GEngineIni);

	if (bSkipPrivilegeCheckOnSessionJoin)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnSessionUserInviteAccepted_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(true, LocalUserIndex, UniqueNetId, SearchResult);
	}
	else
	{
		IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate Delegate = IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, LocalUserIndex, SearchResult](const FUniqueNetId& LocalUserId, EUserPrivileges::Type Privilege, uint32 PrivilegeResult)
			{
				FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Cast(LocalUserId);

				if (PrivilegeResult == static_cast<uint32>(IOnlineIdentity::EPrivilegeResults::NoFailures))
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnSessionUserInviteAccepted_Delegate);
					GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(true, LocalUserIndex, UniqueNetId, SearchResult);
				}
				else
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpaGDK_OnSessionUserInviteAccepted_Delegate);
					GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserIndex, UniqueNetId, FOnlineSessionSearchResult());
				}
			});

		FGDKContextHandle Context = GDKSubsystem->GetGDKContext(AcceptingUser);
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGetUserPrivilege>(GDKSubsystem, Context, UniqueNetId, EUserPrivileges::CanPlayOnline, Delegate, EShowPrivilegeResolveUI::Show);
	}
}

FNamedOnlineSession* FOnlineSessionMpaGDK::AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, SessionSettings);
	Sessions.Emplace(NewSession);
	return &NewSession.Get();
}

FNamedOnlineSessionRef FOnlineSessionMpaGDK::AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, SessionSettings);
	Sessions.Emplace(NewSession);
	return NewSession;
}

FNamedOnlineSession* FOnlineSessionMpaGDK::AddNamedSession(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, Session);
	Sessions.Emplace(NewSession);
	return &NewSession.Get();
}

FNamedOnlineSessionRef FOnlineSessionMpaGDK::AddNamedSessionRef(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, Session);
	Sessions.Emplace(NewSession);
	return NewSession;
}

const FMpaActivity* FOnlineSessionMpaGDK::GetMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID) const
{
	uint64 Id = LocalPlayerID->ToUint64();
	return MpaActivitySetByLocalPlayer.Find(Id);
}

void FOnlineSessionMpaGDK::SetMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID, const FMpaActivity& InMpaActivity)
{ 
	uint64 Id = LocalPlayerID->ToUint64();
	FMpaActivity& MpaActivity = MpaActivitySetByLocalPlayer.FindOrAdd(Id);
	MpaActivity = InMpaActivity;
}

void FOnlineSessionMpaGDK::ClearMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID)
{
	uint64 Id = LocalPlayerID->ToUint64();
	MpaActivitySetByLocalPlayer.Remove(Id);
}

#endif //WITH_GRDK
