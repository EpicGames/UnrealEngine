// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDKPackage.h"
#include "Online/CoreOnline.h"
#include <string>

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_activity_c.h>
#ifdef UE_PLAYFAB_MATCHMAKING
#include <Party.h>
#include <PFMultiplayer.h>
#include <PFMatchmaking.h>
#include <PFLobby.h>
#endif
THIRD_PARTY_INCLUDES_END
struct FMpaActivity
{
	std::string ConnectionString;
	std::string GroupId;
	uint32 CurrentPlayers = 0;
	uint32 MaxPlayers = 0;
	XblMultiplayerActivityJoinRestriction JoinRestriction = XblMultiplayerActivityJoinRestriction::Followed;
	bool AllowCrossPlatformJoin = true;

	bool operator==(const FMpaActivity& Other) const 
	{ 
		return ConnectionString == Other.ConnectionString &&
			GroupId == Other.GroupId &&
			CurrentPlayers == Other.CurrentPlayers &&
			MaxPlayers == Other.MaxPlayers &&
			JoinRestriction == Other.JoinRestriction &&
			AllowCrossPlatformJoin == Other.AllowCrossPlatformJoin;
	}
};

class FOnlineSessionMpaGDK : public TSharedFromThis<FOnlineSessionMpaGDK, ESPMode::ThreadSafe>
{
public:

	// IOnlineSession interface
	FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr);
	FNamedOnlineSession* GetNamedSession(FName SessionName);
	FNamedOnlineSessionPtr GetNamedSessionPtr(FName SessionName) const;

	void RemoveNamedSession(FName SessionName);

	bool HasPresenceSession() const;

	EOnlineSessionState::Type GetSessionState(FName SessionName) const;

	bool CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings);
	bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings);
	bool StartSession(FName SessionName);
	bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData);
	bool EndSession(FName SessionName);
	bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate());
	bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId);
	bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName);
	bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName);
	bool FindSessions(int32 SearchingPlayerControllerIndex, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate);
	bool CancelFindSessions();
	bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) { return false; }
	bool JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession);
	bool JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession);
	bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend);
	bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend);
	bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList);
	bool FindFriendSession(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& FriendList);
	bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend);
	bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend);
	bool SendSessionInviteToFriend_Internal(FGDKContextHandle GDKContext, FName SessionName, const FUniqueNetId& Friend);
	bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends);
	bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends);
	bool SendSessionInviteToFriendsByNetIds(FGDKContextHandle GDKContext, FName SessionName, const TArray<FUniqueNetIdRef>& Friends);
	bool SendSessionInviteToFriendsByXuids(FGDKContextHandle GDKContext, FName SessionName, const TArray<uint64>& FriendXuids);
	bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType);
	bool GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo);
	FOnlineSessionSettings* GetSessionSettings(FName SessionName);
	bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited);
	bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited);
	void RegisterVoice(const FUniqueNetId& PlayerId);
	bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId);
	bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players);
	void UnregisterVoice(const FUniqueNetId& PlayerId);
	void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate);
	void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate);
	void Tick(float DeltaTime);
	void TickPendingSessionUserInvites(float DeltaTime);
	void TickPendingInvites(float DeltaTime);
	FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session);
	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSession& Session);
	int32 GetNumSessions() { return Sessions.Num(); }
	void DumpSessionState() {}

	const FMpaActivity* GetMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID) const;
	void SetMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID, const FMpaActivity& MpaActivity);
	void ClearMpaActivity(FUniqueNetIdGDKPtr LocalPlayerID);

PACKAGE_SCOPE:
	FOnlineSessionMpaGDK(class FOnlineSubsystemGDK* InSubsystem);

	void SaveInviteFromActivation(const FString& ActivationUri);

	void ClearSessionState(FGDKContextHandle GDKContext, FUniqueNetIdGDKRef NetID);


protected:
	void SaveSessionInvite(uint64 SenderXuid, FGDKUserHandle AcceptingUser, const FString& ConnectionString);
	void ProcessPendingSessionUserInvite(FGDKUserHandle AcceptingUser, int32 LocalUserIndex, FOnlineSessionSearchResult const& SearchResult);

	void OnGetActivitiesComplete(bool bWasSuccessful, TArray<FOnlineSession> OnlineSessions, int32 LocalUserNum);

#ifdef UE_PLAYFAB_MATCHMAKING
	bool CreateMatchmakingTicket(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings);
	void ShareMatchmakingString(FString& ConnectString);
	void TickMatchmaking();
	void TickLobby();
	bool CleanupMatchmaking();
#endif // UE_PLAYFAB_MATCHMAKING

	struct FPendingInviteData
	{
		/** Whether there is a pending invite or not */
		bool bHaveInvite = false;

		/** Cached xuid to the user who send the invite */
		uint64 Sender = 0;

		/** Cached pointer to the user who accepted the invite */
		FGDKUserHandle AcceptingUser;

		/** Cached connection string to join */
		FString ConnectionString;

		/** Have we logged that we haven't processed this in a timely manner? */
		bool bLoggedNotProcessedYet = false;

		/** Time to log that we haven't processed this yet */
		double LoggedNotProcessedYetTime = 0.0;
	};

	FPendingInviteData PendingInvite;

	struct FPendingSessionUserInvite
	{
		FPendingSessionUserInvite(
			int32 InAcceptingUserIndex,
			TSharedRef<FOnlineSessionSearchResult> InSearchResult)
			: AcceptingUserIndex(InAcceptingUserIndex)
			, SearchResult(InSearchResult)
		{
		}

		int32 AcceptingUserIndex;
		TSharedRef<FOnlineSessionSearchResult> SearchResult;
	};

	TOptional<FPendingSessionUserInvite> PendingSessionUserInvite;

	const int32 SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS = 10;

	class FOnlineSubsystemGDK* GDKSubsystem;

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current sessions */
	TArray<FNamedOnlineSessionRef> Sessions;

	TMap<uint64, FMpaActivity> MpaActivitySetByLocalPlayer;
#ifdef UE_PLAYFAB_MATCHMAKING

	PFMultiplayerHandle PlayfabHandle = nullptr;
	struct MatchmakingState
	{	
		const PFMatchmakingTicket* MMticket = nullptr;
		TArray< FUniqueNetIdRef > MMLocalPlayers;
		FName MMSessionName;
		FOnlineSessionSettings MMSessionSettings;
		PFLobbyHandle MMLobby =nullptr;
		bool bHost = false;

		void Reset()
		{
			MMticket = nullptr;
			MMLocalPlayers.Empty();
			MMLobby = nullptr;
			bHost = false;
		}
	};

	MatchmakingState MMState;
	
#endif // UE_PLAYFAB_MATCHMAKING 

};

typedef TSharedPtr<FOnlineSessionMpaGDK, ESPMode::ThreadSafe> FOnlineSessionMpaGDKPtr;
