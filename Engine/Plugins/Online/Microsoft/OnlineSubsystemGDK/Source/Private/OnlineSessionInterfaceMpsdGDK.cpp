// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "Online/OnlineSessionNames.h"

#if WITH_ENGINE
#include "Engine/EngineBaseTypes.h"
#include "Interfaces/VoiceInterface.h"
#endif //WITH_ENGINE

#include "OnlineSubsystemGDKPrivate.h"
#include "Misc/Base64.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemSessionSettings.h"
#include "OnlineEventsInterfaceGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "GDKRuntimeModule.h"
#include "GDKThreadCheck.h"

#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Base64.h"

#include "SocketSubsystem.h"
#include "IPAddress.h"

#include "AsyncTasks/OnlineAsyncTaskGDKCreateSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKJoinSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCreateMatchSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSubmitMatchTicket.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGameSessionReady.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCancelMatchmaking.h"
#include "AsyncTasks/OnlineAsyncTaskGDKDestroySession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKRegisterLocalUser.h"
#include "AsyncTasks/OnlineAsyncTaskGDKUnregisterLocalUser.h"
#include "AsyncTasks/OnlineAsyncTaskGDKUpdateSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKUpdateSessionMember.h"
#include "AsyncTasks/OnlineAsyncTaskGDKMeasureAndUploadQos.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSendSessionInviteToFriends.h"
#include "AsyncTasks/OnlineAsyncTaskGDKSetSessionActivity.h"
#include "AsyncTasks/OnlineAsyncTaskGDKFindSessionById.h"
#include "AsyncTasks/OnlineAsyncTaskGDKClearSessionActivity.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserProfile.h"
#include "AsyncTasks/OnlineAsyncTaskGDKFindSessionsBySearchHandle.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryActivitiesForUsers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKGetUserPrivilege.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameInvite.h>
#include <XGameRuntimeFeature.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace 
{
	/** The maximum number of GDK sessions to return when searching for orphaned sessions. */
	const int32 MAX_ORPHANED_SESSIONS_RESULTS = 100;
	const int32 MAX_RETRIES = 20;

	/** Amount of time to wait for a saved session invite to be processed before logging that it's missed */
	const int32 SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS = 10;

	const TCHAR* GetSessionMemberStatusString(XblMultiplayerSessionMemberStatus Status)
	{

		switch (Status)
		{
		case XblMultiplayerSessionMemberStatus::Active:
			return TEXT("Active");
		case XblMultiplayerSessionMemberStatus::Inactive:
			return TEXT("Inactive");
		case XblMultiplayerSessionMemberStatus::Ready:
			return TEXT("Ready");
		case XblMultiplayerSessionMemberStatus::Reserved:
			return TEXT("Reserved");
		}

		return TEXT("Unknown");
	}
	
	template <typename OutputterType>
	void GenerateSessionDebugInfo(FGDKMultiplayerSessionHandle GDKSession, const OutputterType& DebugOut)
	{
		if (!GDKSession.IsValid())
		{
			DebugOut(TEXT("  No Valid GDK Session"));
			return;
		}
		
		const XblMultiplayerSessionReference* SessionReference = XblMultiplayerSessionSessionReference(GDKSession);
		const XblMultiplayerSessionInfo* SessionInfo = XblMultiplayerSessionGetInfo(GDKSession);
		const XblMultiplayerSessionConstants* SessionConstants = XblMultiplayerSessionSessionConstants(GDKSession);
		const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
		
		DebugOut(*FString::Printf(TEXT("  SessionId=[%ls], HandleId=[%ls]"),
			*FOnlineSessionMpsdGDK::SessionReferenceToUri(*SessionReference),
			UTF8_TO_TCHAR(SessionInfo->SearchHandleId)));
			
		DebugOut(*FString::Printf(TEXT("  Branch=[%ls], Revision[%llu]"),
			UTF8_TO_TCHAR(SessionInfo->Branch),
			SessionInfo->ChangeNumber));
		
		const XblMultiplayerSessionMember* SessionMembers = nullptr;
		uint64 NumSessionMembers = 0;
		HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &SessionMembers, &NumSessionMembers);
			
		DebugOut(*FString::Printf(TEXT("  Members->Size=[%llu/%u]. Members:"),
			NumSessionMembers,
			SessionConstants->MaxMembersInSession));
		
		if (SUCCEEDED(Result))
		{
			for (int i=0; i< NumSessionMembers; ++i)
			{
				const XblMultiplayerSessionMember& SessionMember = SessionMembers[i];
				DebugOut(*FString::Printf(TEXT("    Gamertag=[%ls], GDK ID=[%llu], Status=[%s]"),
					UTF8_TO_TCHAR(SessionMember.Gamertag),
					SessionMember.Xuid,
					GetSessionMemberStatusString(SessionMember.Status)));
			}
		}

		FString PropertiesJson = UTF8_TO_TCHAR(SessionProperties->SessionCustomPropertiesJson);
		TSharedPtr< FJsonObject > JsonObject;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(MoveTemp(PropertiesJson));

		
		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			DebugOut(*FString::Printf(TEXT("  Custom Properties Size=[%d] Fields:"), JsonObject->Values.Num()));
			for (const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::ElementType& Pair : JsonObject->Values)
			{
				FString OutString;
				if (Pair.Value->TryGetString(OutString))
				{
					DebugOut(*FString::Printf(TEXT("    %s=[%s]"),
						*Pair.Key,
						*OutString));
				}
				else
				{
					DebugOut(*FString::Printf(TEXT("    %s=[Non-String Value]"),
						*Pair.Key));
				}
			}
		}
		else
		{
			DebugOut(TEXT("  Custom Properties Size=[Empty]"));
		}
	} 

	template <typename OutputterType>
	void GenerateSessionDebugInfo(const FNamedOnlineSession& Session, const OutputterType& DebugOut)
	{
		FGDKMultiplayerSessionHandle GDKSession;

		TSharedPtr<FOnlineSessionInfoMpsdGDK> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session.SessionInfo);
		if (SessionInfo.IsValid() && SessionInfo->IsValid())
		{
			GDKSession = SessionInfo->GetGDKMultiplayerSession();
		}

		const XblMultiplayerSessionMember* Host = FOnlineSessionMpsdGDK::GetGDKSessionHost(GDKSession);

		DebugOut(*FString::Printf(TEXT("Session=[%s], Host=[%ls]"),
			*Session.SessionName.ToString(),
			Host ? UTF8_TO_TCHAR(Host->Gamertag) : L"No Host"));

		DebugOut(*FString::Printf(TEXT("State: %s"),
			EOnlineSessionState::ToString(Session.SessionState)));

		GenerateSessionDebugInfo(GDKSession, DebugOut);
	}
}

FOnlineSessionMpsdGDK::FPendingSessionUserInvite::FPendingSessionUserInvite(
	int32 InAcceptingUserIndex,
	bool InWasSuccessful,
	TSharedRef<FOnlineSessionSearchResult> InSearchResult,
	XblMultiplayerInviteHandle InInviteHandle)
	: AcceptingUserIndex(InAcceptingUserIndex)
	, bWasSuccessful(InWasSuccessful)
	, SearchResult(InSearchResult)
	, InviteHandle(InInviteHandle)
{

}

FOnlineSessionMpsdGDK::FOnlineSessionMpsdGDK(class FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
	, bIsDestroyingSessions(false)
	, bOnlyHostUpdateSession(true)
	, bHandleXblSubscriptionLost(true)
	, bHasSessionActivityInProgress(false)
{
	Initialize();
}

//WMM TODO: This doesn't do anything with the certificates... //Add to NamedSession of the user?
void FOnlineSessionMpsdGDK::WriteDTLSCertificatesToService(const TMap<FGuid, TArray<uint8>>& DTLSCertificateDictionary)
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session->SessionInfo);
		if (GDKInfo.IsValid())
		{
			FGDKMultiplayerSessionHandle GDKSession = GDKInfo->GetGDKMultiplayerSession();
			if (GDKSession.IsValid())
			{
				uint64 MemberCount = 0;
				const XblMultiplayerSessionMember* Members = nullptr;
				XblMultiplayerSessionMembers(GDKSession, &Members, &MemberCount);
				for (unsigned int i = 0; i < MemberCount; ++i)
				{
					FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(Members[i].Xuid);
					if (GDKContext.IsValid())
					{
						constexpr const int32 RetryCount = 10;
						GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKUpdateSessionMember>(GDKSubsystem, GDKContext, Session->SessionName, GDKSession, RetryCount);
					}
				}
			}
		}
	}
}

FString FOnlineSessionMpsdGDK::SessionReferenceToUri(const XblMultiplayerSessionReference& SessionReference)
{
	// ToURIPath is coming at a later revision, so we'll do it by hand for now
	FString UriPath;
	UriPath += TEXT("/serviceconfigs/");
	UriPath += SessionReference.Scid;
	UriPath += TEXT("/sessiontemplates/");
	UriPath += SessionReference.SessionTemplateName;
	UriPath += TEXT("/sessions/");
	UriPath += SessionReference.SessionName;

	return UriPath;
}

FOnlineSessionMpsdGDK::~FOnlineSessionMpsdGDK()
{
	check(IsInGameThread());
	// Leave any sessions we are still in
	{
		FScopeLock Lock(&SessionLock);
		if (Sessions.Num() > 0)
		{
			TArray<FNamedOnlineSessionRef> SessionsCopy = Sessions;
			for (const FNamedOnlineSessionRef& CurrentSession : SessionsCopy)
			{
				DestroySession(CurrentSession->SessionName);
			}
		}
		FSessionMessageRouterPtr SessionRouter = GDKSubsystem->GetSessionMessageRouter();
		if (SessionRouter.IsValid())
		{
			SessionRouter->ClearOnConnectionIdChangedDelegate_Handle(OnConnectionIdChangedHandle);
			SessionRouter->ClearOnSubscriptionsLostDelegate_Handle(OnSubscriptionsLostHandle);
		}
	}
}

void FOnlineSessionMpsdGDK::Initialize()
{
	FString TemplateName;

	// Load our session-updating stats
	GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("SessionUpdateEventName"), SessionUpdateEventName, GEngineIni);

	// Load session updating permissions setting
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bOnlyHostUpdateSession"), bOnlyHostUpdateSession, GEngineIni);

	// Load subscription lost handling setting
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bHandleXblSubscriptionLost"), bHandleXblSubscriptionLost, GEngineIni);

	// Grab the user list, the Identity interface caches this at startup already so we can just use their list instead of
	// spending extra time making another cross-VM call.
	// Note that this forces a dependency on the identity interface being initialized before the session interface.
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	OnSubscriptionLostDestroyCompleteDelegate = FOnEndSessionCompleteDelegate::CreateRaw(this, &FOnlineSessionMpsdGDK::OnSubscriptionLostDestroyComplete);

	// Initialize session state after create/join
	OnSessionNeedsInitialStateDelegate = FOnSessionNeedsInitialStateDelegate::CreateRaw(this, &FOnlineSessionMpsdGDK::OnSessionNeedsInitialState);
	GDKSubsystem->GetSessionMessageRouter()->AddOnSessionNeedsInitialStateDelegate_Handle(OnSessionNeedsInitialStateDelegate);

	OnSessionChangedDelegate = FOnSessionChangedDelegate::CreateRaw(this, &FOnlineSessionMpsdGDK::OnSessionChanged);

	FOnConnectionIdChangedDelegate OnConnectionIdChangedDelegate = FOnConnectionIdChangedDelegate::CreateRaw(this, &FOnlineSessionMpsdGDK::OnMultiplayerConnectionIdChanged);
	OnConnectionIdChangedHandle = GDKSubsystem->GetSessionMessageRouter()->AddOnConnectionIdChangedDelegate_Handle(OnConnectionIdChangedDelegate);

	FOnSubscriptionsLostDelegate OnSubscriptionsLostDelegate = FOnSubscriptionsLostDelegate::CreateRaw(this, &FOnlineSessionMpsdGDK::OnMultiplayerSubscriptionsLost);
	OnSubscriptionsLostHandle = GDKSubsystem->GetSessionMessageRouter()->AddOnSubscriptionsLostDelegate_Handle(OnSubscriptionsLostDelegate);

#if !UE_BUILD_SHIPPING
	const bool bDebugSessions = FParse::Param(FCommandLine::Get(), TEXT("DebugGDKSessions"));
	if (bDebugSessions)
	{
		FCoreDelegates::OnGetOnScreenMessages.AddLambda([this](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
		{
			const auto PrintToScreen = [&](const TCHAR* const Output)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Output));
			};

			FScopeLock ScopeLock(&SessionLock);
			for (const FNamedOnlineSessionRef& Session : Sessions)
			{
				GenerateSessionDebugInfo(*Session, PrintToScreen);
			}
		});
	}
#endif
}

bool FOnlineSessionMpsdGDK::AreInvitesAndJoinViaPresenceAllowed(const FOnlineSessionSettings& OnlineSessionSettings)
{
	return OnlineSessionSettings.bAllowInvites || OnlineSessionSettings.bAllowJoinViaPresence || OnlineSessionSettings.bAllowJoinViaPresenceFriendsOnly;
}

bool FOnlineSessionMpsdGDK::CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	auto UniqueId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(HostingPlayerControllerIndex);
	if (!UniqueId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("Couldn't find unique id for HostingPlayerNum %d"), HostingPlayerControllerIndex);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	return CreateSession(*UniqueId, SessionName, NewSessionSettings);
}

/**
* Delegate fired when SetPresence task has completed
*/
DECLARE_DELEGATE_OneParam(FOnStartCreateSessionCompleteDelegate, bool /*bSuccessful*/);


bool FOnlineSessionMpsdGDK::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	// Check for an existing session
	if (GetNamedSessionPtr(SessionName).IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}

	FUniqueNetIdGDKRef HostGDKId = FUniqueNetIdGDK::Cast(HostingPlayerId);

	FString TemplateNameString;
	const FOnlineSessionSetting* TemplateNameSetting = NewSessionSettings.Settings.Find(SETTING_SESSION_TEMPLATE_NAME);
	if (TemplateNameSetting)
	{
		TemplateNameSetting->Data.GetValue(TemplateNameString);
	}

	FGDKUserHandle CreatingUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(*HostGDKId);

	FString Keyword;
	NewSessionSettings.Get(SEARCH_KEYWORDS, Keyword);

	auto CreateSessionCompleteDelegate = FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete::CreateLambda([this](bool bWasSuccessful, FName SessionName, FUniqueNetIdGDKRef HostGDKId, FOnlineSessionSettings NewSessionSettings)
	{
		if (!bWasSuccessful)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
			return;
		}

		auto CreateSearchHandleCompleteDelegate = FOnCreateSearchHandleCompleteDelegate::CreateLambda([this, SessionName](bool bWasSuccessful)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
		});

		FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
		if (NamedSession.IsValid())
		{
			FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
			FGDKMultiplayerSessionHandle GDKSession = GDKInfo->GetGDKMultiplayerSession();
			FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*HostGDKId);
			
			// check and see if we should create a handle. 
			const XblMultiplayerSessionConstants* SesessionConstants = XblMultiplayerSessionSessionConstants(GDKSession);
			if (SesessionConstants != nullptr && SesessionConstants->SessionCapabilities.Searchable)
			{
				CreateSearchHandle(NewSessionSettings, GDKSession, GDKContext, CreateSearchHandleCompleteDelegate);
			}		
			else
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
			}
		}

	}, HostGDKId, NewSessionSettings);

	if (!StartCreateSession(HostingPlayerId, NewSessionSettings, Keyword, TemplateNameString, SessionName, CreateSessionCompleteDelegate))
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("Failed to create async create session operation"));
		RemoveNamedSession(SessionName);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CreateSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCreateSessionCompleteDelegates(SessionName, false);
		return false;
	}
	return true;
}

bool FOnlineSessionMpsdGDK::SetHostOnCreatedSession(FGDKMultiplayerSessionHandle GDKSession, uint64 CreatingUserId)
{
	const XblMultiplayerSessionReference* SessionReference = XblMultiplayerSessionSessionReference(GDKSession);
	check(SessionReference);

	GDKSubsystem->GetSessionMessageRouter()->AddOnSessionChangedDelegate(OnSessionChangedDelegate, SessionReference);
	// Now that the session is created, we can get the device token and set this console as the host.
	const XblMultiplayerSessionMember* HostMember = nullptr;
	auto MemberArray = GetMemberArray(GDKSession);

	for (const XblMultiplayerSessionMember& Member : *MemberArray)
	{
		if (Member.Xuid == CreatingUserId)
		{
			HostMember = &Member;
			break;
		}
	}

	FUniqueNetIdRef CreatingUserUniqueId(FUniqueNetIdGDK::Create(CreatingUserId));

	FGDKContextHandle Context = GDKSubsystem->GetGDKContext(CreatingUserId);
	check(Context.IsValid());

	if (HostMember == nullptr)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Could not find creator in session members. Not setting host."));
		return false;
	}

	// Simple host selection - the user that creates the session is the host.
	XblMultiplayerSessionSetHostDeviceToken(GDKSession, HostMember->DeviceToken);
	return true;
}

bool FOnlineSessionMpsdGDK::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{
	return IsPlayerInSessionImpl(GDKSubsystem->GetSessionInterface().Get(), SessionName, UniqueId);
}

bool FOnlineSessionMpsdGDK::FindSessions(int32 SearchingPlayerControllerIndex, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	auto UniqueId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(SearchingPlayerControllerIndex);
	if (!UniqueId.IsValid())
	{
		SearchSettings->SearchState = EOnlineAsyncTaskState::Failed;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_FindSessions_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindSessionsCompleteDelegates(false);
		return false;
	}

	return FindSessions(*UniqueId, SearchSettings);
}

bool FOnlineSessionMpsdGDK::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	// Don't start another search while one is in progress
	if (!CurrentSessionSearch.IsValid() && SearchSettings->SearchState != EOnlineAsyncTaskState::InProgress)
	{
		// Copy the search pointer so we can keep it around
		CurrentSessionSearch = SearchSettings;
		SearchSettings->SearchState = EOnlineAsyncTaskState::InProgress;

		FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(SearchingPlayerId);

		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKFindSessionsBySearchHandle>(
			GDKSubsystem,
			GDKSubsystem->GetSessionInterfaceGDK().Get(),
			GDKContext,
			SearchSettings);
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Ignoring GDK Session Search request while one is pending."));
	}
	return true;
}

bool FOnlineSessionMpsdGDK::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	UNREFERENCED_PARAMETER(FriendId);

	auto TriggerErrorDelegateNextTick = [this, CompletionDelegate](const int32 LambdaUserNum)
	{
		GDKSubsystem->ExecuteNextTick([this, CompletionDelegate, LambdaUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_FindSessionById_Delegate);
			CompletionDelegate.ExecuteIfBound(LambdaUserNum, false, FOnlineSessionSearchResult());
		});
	};

	if (!SearchingUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid local-user: %s"), *SearchingUserId.ToString());
		TriggerErrorDelegateNextTick(0);
		return false;
	}

	FOnlineIdentityGDKPtr IdentityPtr = GDKSubsystem->GetIdentityGDK();
	if (!IdentityPtr.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Identity Interface invalid"));
		TriggerErrorDelegateNextTick(0);
		return false;
	}

	const int32 LocalUserNum = IdentityPtr->GetPlatformUserIdFromUniqueNetId(SearchingUserId);
	if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid local-user: %s"), *SearchingUserId.ToString());
		TriggerErrorDelegateNextTick(0);
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unknown local-user: %s"), *SearchingUserId.ToString());
		TriggerErrorDelegateNextTick(LocalUserNum);
		return false;
	}

	if (!SessionId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session id: %s"), *SessionId.ToString());
		TriggerErrorDelegateNextTick(LocalUserNum);
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKFindSessionById>(GDKSubsystem, GDKContext, LocalUserNum, SessionId.ToString(), CompletionDelegate);
	return true;
}

bool FOnlineSessionMpsdGDK::CancelFindSessions()
{
	// Unsupported
	GDKSubsystem->ExecuteNextTick([this]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_CancelFindSessions_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnCancelFindSessionsCompleteDelegates(false);
	});

	return false;
}

int32 FOnlineSessionMpsdGDK::GetHostingPlayerNum(const FUniqueNetId& HostNetId) const
{
	FOnlineIdentityGDKPtr IdentityPtr = GDKSubsystem->GetIdentityGDK();
	if (!IdentityPtr.IsValid())
	{
		return INDEX_NONE;
	}

	return IdentityPtr->GetPlatformUserIdFromUniqueNetId(HostNetId);
}

bool FOnlineSessionMpsdGDK::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings)
{
	return GDKSubsystem->GetMatchmakingInterfaceGDK()->StartMatchmaking(LocalPlayers, SessionName, NewSessionSettings, SearchSettings);
}

bool FOnlineSessionMpsdGDK::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName)
{
	return GDKSubsystem->GetMatchmakingInterfaceGDK()->CancelMatchmaking(SearchingPlayerNum, SessionName);
}

bool FOnlineSessionMpsdGDK::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName)
{
	return GDKSubsystem->GetMatchmakingInterfaceGDK()->CancelMatchmaking(SearchingPlayerId, SessionName);
}

bool FOnlineSessionMpsdGDK::JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	FUniqueNetIdPtr UniqueId = GDKSubsystem->GetIdentityGDK()->GetUniquePlayerId(ControllerIndex);
	if (!UniqueId.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([SessionName, ControllerIndex, this]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("JoinSession failed; unable to find player at index %d"), ControllerIndex);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
		});
		return false;
	}

	return JoinSession(*UniqueId, SessionName, DesiredSession);
}

bool FOnlineSessionMpsdGDK::JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	UE_LOG_ONLINE_SESSION(Log, TEXT("User %s Attempting to join session %s"), *UserId.ToString(), *SessionName.ToString());

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(UserId);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session %s failed, user %s has no GDK context"), *SessionName.ToString(), *UserId.ToString());
		GDKSubsystem->ExecuteNextTick([SessionName, this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
		});
		return false;
	}

	GDKSubsystem->EnableSessionEventHandlers(GDKContext);

	// work out if we're already in the session of this name or not
	FNamedOnlineSessionPtr NamedSessionCheck = GetNamedSessionPtr(SessionName);
	if (NamedSessionCheck.IsValid())
	{
		// Check if we're trying to join a different session of this same type while in a different session (different than joining the same session multiple times)
		const FString ExistingSessionId(NamedSessionCheck->GetSessionIdStr());
		const FString NewSessionId(DesiredSession.GetSessionIdStr());

		if (ExistingSessionId == NewSessionId)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session failed; session (%s) already exists, can't join twice"), *SessionName.ToString());
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session failed; already in session of type %s, you must leave session %s before joining %s"), *SessionName.ToString(), *ExistingSessionId, *NewSessionId);
		}

		GDKSubsystem->ExecuteNextTick([SessionName, this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::AlreadyInSession);
		});
		return false;
	}

	// Check for a Join from URI (Join In Progress from Matchmade Sessions)
	FString SessionURI;
	if (DesiredSession.Session.SessionSettings.Get(SETTING_GAME_SESSION_URI, SessionURI))
	{
		FNamedOnlineSessionRef NamedSession = AddNamedSessionRef(SessionName, DesiredSession.Session.SessionSettings);
		FString SessionTemplateName;
		DesiredSession.Session.SessionSettings.Get(SETTING_SESSION_TEMPLATE_NAME, SessionTemplateName);

		XblMultiplayerSessionReference SessionReference;
		XblMultiplayerSessionReferenceParseFromUriPath(TCHAR_TO_UTF8(*SessionURI), &SessionReference);

		const bool bIsMatchmakingSession = true;
		const bool bSetActivity = DesiredSession.Session.SessionSettings.bUsesPresence;
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKJoinSession>(GDKSubsystem, GDKContext, &SessionReference, NamedSession, MAX_RETRIES, bIsMatchmakingSession, bSetActivity, TOptional<FString>());
		return true;
	}

	// Create a named session from the search result data
	FNamedOnlineSessionRef NamedSession = AddNamedSessionRef(SessionName, DesiredSession.Session);
	NamedSession->HostingPlayerNum = INDEX_NONE;
	NamedSession->LocalOwnerId = FUniqueNetIdGDK::Cast(UserId);

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(DesiredSession.Session.SessionInfo);
	if (!GDKInfo.IsValid() || !GDKInfo->IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Join session %s failed, invalid session info on search result"), *SessionName.ToString());
		RemoveNamedSession(SessionName);

		GDKSubsystem->ExecuteNextTick([SessionName, this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_JoinSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnJoinSessionCompleteDelegates(SessionName, EOnJoinSessionCompleteResult::UnknownError);
		});
		return false;
	}

	const bool bIsMatchmakingSession = false;
	const bool bSetActivity = DesiredSession.Session.SessionSettings.bUsesPresence;

	if (FGDKMultiplayerSessionHandle GDKSession = GDKInfo->GetGDKMultiplayerSession())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKJoinSession>(GDKSubsystem, GDKContext, GDKSession, NamedSession, MAX_RETRIES, bIsMatchmakingSession, bSetActivity, GDKInfo->GetSessionInviteHandleString());
		return true;
	}
	else if (const XblMultiplayerSessionReference* GDKSessionRef = GDKInfo->GetGDKMultiplayerSessionRef())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKJoinSession>(GDKSubsystem, GDKContext, GDKSessionRef, NamedSession, MAX_RETRIES, bIsMatchmakingSession, bSetActivity, GDKInfo->GetSessionInviteHandleString());
		return true;
	}

	// We shouldn't get here if LiveInfo->IsValid() passed
	checkNoEntry();
	return false;
}

const XblMultiplayerSessionMember* FOnlineSessionMpsdGDK::GetCurrentUserFromSession(FGDKMultiplayerSessionHandle GDKSession)
{
	const XblMultiplayerSessionMember* SessionMembers = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &SessionMembers, &NumMembers);
	if (SUCCEEDED(Result))
	{
		for (int i = 0; i < NumMembers; ++i)
		{
			const XblMultiplayerSessionMember& Member = SessionMembers[i];
			if (Member.IsCurrentUser)
			{
				return &Member;
			}
		}
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
// This API returns the "Advertised Session" which my friend is in, not all sessions they are in.
//-----------------------------------------------------------------------------

bool FOnlineSessionMpsdGDK::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FindFriendSession: Failed to retrieve Friend's multiplayer, no available GDKContext for user %d"), LocalUserNum);
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_FindFriendSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
		return false;
	}

	FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(Friend);

	TArray<uint64> FriendIds;
	FriendIds.Add(GDKFriend->ToUint64());
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKQueryActivitiesForUsers>(GDKSubsystem, GDKContext, FriendIds, FOnGDKQueryActivitiesForUsersComplete::CreateThreadSafeSP(this, &FOnlineSessionMpsdGDK::OnQueryActivitiesComplete, LocalUserNum, GDKContext));
	return true;
}

void FOnlineSessionMpsdGDK::OnQueryActivitiesComplete(const FOnlineError& ErrorResult, const FOnlineGDKActivitiesResultMap& Results, int32 LocalUserNum, FGDKContextHandle GDKContext)
{
	if (ErrorResult.bSucceeded)
	{
		if (Results.Num() < 1)
		{
			// Friend has no advertised active session
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("FindFriendSession: Friend has no multiplayer activity"));
			GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnQueryActivitiesComplete_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
			});
		}
		else
		{
			// We only want the first result
			XblMultiplayerSessionReference SessionReference;
			auto It = Results.begin();
			FString SessionIdStr = It->Value->ToString();
			XblMultiplayerSessionReferenceParseFromUriPath(TCHAR_TO_UTF8(*SessionIdStr), &SessionReference);
			GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKFindSessionById>(GDKSubsystem, GDKContext, LocalUserNum, &SessionReference, FOnSingleSessionResultCompleteDelegate::CreateThreadSafeSP(this, &FOnlineSessionMpsdGDK::OnQueryFriendSessionDetailsComplete, GDKContext));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FindFriendSession: Failed to query Friend's multiplayer activity"));
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnQueryActivitiesComplete_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
		return;
	}
}

void FOnlineSessionMpsdGDK::OnQueryFriendSessionDetailsComplete(int32 LocalUserNum, bool bSucceeded, const FOnlineSessionSearchResult& SearchResult, FGDKContextHandle GDKContext)
{
	if(bSucceeded)
	{
		TSharedPtr<FOnlineSessionInfoMpsdGDK> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SearchResult.Session.SessionInfo);
		FGDKMultiplayerSessionHandle FriendSession = SessionInfo->GetGDKMultiplayerSession();
		const XblMultiplayerSessionMember* SessionHost = GetGDKSessionHost(FriendSession);

		FString HostDisplayName = (SessionHost != nullptr) ? UTF8_TO_TCHAR(SessionHost->Gamertag) : TEXT("");

		GDKSubsystem->ExecuteNextTick([this, LocalUserNum, FriendSession, HostDisplayName]()
		{
			FOnlineSessionSearchResult SearchResult = CreateSearchResultFromSession(FriendSession, HostDisplayName);

			// Trigger success delegate
			{
				TArray<FOnlineSessionSearchResult> FriendSessions;
				FriendSessions.Add(SearchResult);

				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnQueryFriendSessionDetailsComplete_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, true, FriendSessions);
			}
		});
	}
	else
	{
		//WMM TODO: figure out how to pass this along
		// Ignore JSON parse errors, they indicate we don't have access to read the session (private sessions)
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FindFriendSession: Failed to retrieve MultiplayerSession"));
					
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnQueryFriendSessionDetailsComplete_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, TArray<FOnlineSessionSearchResult>());
		});
	}
}

bool FOnlineSessionMpsdGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	int32 ControllerId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(LocalUserId);
	if (ControllerId < 0 || ControllerId >= MAX_LOCAL_PLAYERS)
	{
		GDKSubsystem->ExecuteNextTick([this, ControllerId]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_FindFriendSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(ControllerId, false, TArray<FOnlineSessionSearchResult>());
		});
		return false;
	}

	return FindFriendSession(ControllerId, Friend);
}

bool FOnlineSessionMpsdGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	bool bSuccessfullyJoinedFriendSession = false;

	UE_LOG_ONLINE_SESSION(Display, TEXT("FOnlineSessionMpsdGDK::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) - not implemented"));

	int32 LocalUserNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(LocalUserId);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_FindFriendSession_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, bSuccessfullyJoinedFriendSession, TArray<FOnlineSessionSearchResult>());

	return bSuccessfullyJoinedFriendSession;
}

bool FOnlineSessionMpsdGDK::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	if (!Friend.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friend %s to session %s, LocalUserNum %d is invalid"), *Friend.ToString(), *SessionName.ToString(), LocalUserNum);
		return false;
	}

	FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(Friend);

	TArray<uint64> FriendsToInvite;
	FriendsToInvite.Add(GDKFriend->ToUint64());

	return SendSessionInviteToFriends_Internal(GDKContext, SessionName, FriendsToInvite);
}

bool FOnlineSessionMpsdGDK::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	if (!LocalUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite friend to session %s, LocalUserId is invalid"), *SessionName.ToString());
		return false;
	}

	if (!Friend.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
		return false;
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserId);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friend %s to session %s, LocalUserId %s is invalid"), *Friend.ToString(), *SessionName.ToString(), *LocalUserId.ToString());
		return false;
	}

	FUniqueNetIdGDKRef GDKFriend = FUniqueNetIdGDK::Cast(Friend);

	TArray<uint64> FriendsToInvite;
	FriendsToInvite.Add(GDKFriend->ToUint64());

	return SendSessionInviteToFriends_Internal(GDKContext, SessionName, FriendsToInvite);
}

bool FOnlineSessionMpsdGDK::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (Friends.Num() < 1)
	{
		// Return true in this case, but log it since it's strange
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Attempted to invite any empty array of friends to session %s"), *SessionName.ToString());
		return true;
	}

	for (const FUniqueNetIdRef& Friend : Friends)
	{
		if (!Friend->IsValid())
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
			return false;
		}
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, LocalUserNum %d is invalid"), *SessionName.ToString(), LocalUserNum);
		return false;
	}

	TArray<uint64> FriendsToInvite;
	for (const FUniqueNetIdRef& Friend : Friends)
	{
		const FUniqueNetIdGDKRef GDKFriend = StaticCastSharedRef<const FUniqueNetIdGDK>(Friend);
		FriendsToInvite.Add(GDKFriend->ToUint64());
	}

	return SendSessionInviteToFriends_Internal(GDKContext, SessionName, FriendsToInvite);
}

bool FOnlineSessionMpsdGDK::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (!LocalUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite friend to session %s, LocalUserId is invalid"), *SessionName.ToString());
		return false;
	}

	if (Friends.Num() < 1)
	{
		// Return true in this case, but log it since it's strange
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Attempted to invite any empty array of friends to session %s"), *SessionName.ToString());
		return true;
	}

	for (const FUniqueNetIdRef& Friend : Friends)
	{
		if (!Friend->IsValid())
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite invalid friend to session %s"), *SessionName.ToString());
			return false;
		}
	}

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(LocalUserId);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, LocalUserId %s is invalid"), *SessionName.ToString(), *LocalUserId.ToString());
		return false;
	}

	TArray<uint64> FriendsToInvite;
	for (const FUniqueNetIdRef& Friend : Friends)
	{
		const FUniqueNetIdGDKRef GDKFriend = StaticCastSharedRef<const FUniqueNetIdGDK>(Friend);
		FriendsToInvite.Add(GDKFriend->ToUint64());
	}

	return SendSessionInviteToFriends_Internal(GDKContext, SessionName, FriendsToInvite);}

bool FOnlineSessionMpsdGDK::SendSessionInviteToFriends_Internal(FGDKContextHandle GDKContext, FName SessionName, const TArray<uint64>& FriendXuids)
{
	FNamedOnlineSessionPtr SessionPtr = GetNamedSessionPtr(SessionName);
	if (!SessionPtr.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session does not exist"), *SessionName.ToString());
		return false;
	}

	if (SessionPtr->SessionState < EOnlineSessionState::Pending || SessionPtr->SessionState > EOnlineSessionState::InProgress)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session is in state %d"), *SessionName.ToString(), SessionPtr->SessionState);
		return false;
	}

	FOnlineSessionInfoMpsdGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SessionPtr->SessionInfo);
	if (!SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session has invalid info"), *SessionName.ToString());
		return false;
	}

	FGDKMultiplayerSessionHandle  GDKSession = SessionInfo->GetGDKMultiplayerSession();
	if (!GDKSession.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot Invite Friends to session %s, that session has an invalid reference"), *SessionName.ToString());
		return false;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKSendSessionInviteToFriends>(GDKSubsystem, GDKContext, GDKSession, FriendXuids);
	return true;
}

void FOnlineSessionMpsdGDK::SetUserActiveSessionActivity(const FUniqueNetIdGDK& PlayerId, FGDKMultiplayerSessionHandle GDKSession)
{
	check(IsInGameThread());

	if (bHasSessionActivityInProgress)
	{
		if (GDKSession.IsValid())
		{
			QueuedActiveSessionActivities.Add(PlayerId.AsShared(), TOptional<const FGDKMultiplayerSessionHandle>(GDKSession));
		}
		else
		{
			QueuedActiveSessionActivities.Add(PlayerId.AsShared(), TOptional<const FGDKMultiplayerSessionHandle>());
		}
	}
	else
	{
		const bool bWantsToBeInActiveSession = GDKSession.IsValid();

		SetUserActiveSessionActivity_Impl(PlayerId, bWantsToBeInActiveSession, GDKSession);
	}
}

void FOnlineSessionMpsdGDK::OnSetUserActiveSessionActivityComplete(const FUniqueNetIdGDK& PlayerId, const bool bUserInActiveSession)
{
	check(IsInGameThread());

	ClearSessionActivityInProgress();

	if (bUserInActiveSession)
	{
		UsersWithActiveSessionActivities.Add(PlayerId.AsShared());
	}
	else
	{
		UsersWithActiveSessionActivities.Remove(PlayerId.AsShared());
	}

	// Check if our session state changed while we were updating it with XBL
	TOptional<const FGDKMultiplayerSessionHandle> QueuedState;
	if (QueuedActiveSessionActivities.RemoveAndCopyValue(PlayerId.AsShared(), QueuedState))
	{
		const FGDKMultiplayerSessionHandle SessionHandle = QueuedState.Get(FGDKMultiplayerSessionHandle());
		const bool bUserWantsToBeInActiveSession = SessionHandle.IsValid();

		// If the user wants to be in a session, or we are currently in a session, we need to update our state
		if (bUserInActiveSession || bUserWantsToBeInActiveSession)
		{
			SetUserActiveSessionActivity_Impl(PlayerId, bUserWantsToBeInActiveSession, SessionHandle);
		}
	}
}

void FOnlineSessionMpsdGDK::SetUserActiveSessionActivity_Impl(const FUniqueNetIdGDK& PlayerId, const bool bWantsToBeInActiveSession, FGDKMultiplayerSessionHandle GDKSession)
{
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(PlayerId);
	if (!GDKContext)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to update user activity for %s, they have no GDK context"), *PlayerId.ToString());
		return;
	}

	const bool bIsInActiveSession = UsersWithActiveSessionActivities.Contains(PlayerId.AsShared());
	if (bIsInActiveSession && !bWantsToBeInActiveSession)
	{
		bHasSessionActivityInProgress = true;
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKClearSessionActivity>(GDKSubsystem, GDKContext, nullptr);
	}
	else if (bWantsToBeInActiveSession)
	{
		bHasSessionActivityInProgress = true;
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKSetSessionActivity>(GDKSubsystem, GDKContext, GDKSession);
	}
}

XblMultiplayerSessionChangeTypes FOnlineSessionMpsdGDK::GetSubscriptionType(const FOnlineSessionSettings& SessionSettings)
{
	FString SessionChangeTypeString;
	if (!SessionSettings.Get(SETTING_SESSION_SUBSCRIPTION_TYPES, SessionChangeTypeString))
	{
		// Default to original behavior when not specified
		return XblMultiplayerSessionChangeTypes::Everything;
	}

	if (SessionChangeTypeString == TEXT("None"))
	{
		// Special case (non-additive), immediately return the specified value
		return XblMultiplayerSessionChangeTypes::None;
	}
	else if (SessionChangeTypeString == TEXT("Everything"))
	{
		// Special case (non-additive), immediately return the specified value
		return XblMultiplayerSessionChangeTypes::Everything;
	}

	XblMultiplayerSessionChangeTypes ReturnValue = XblMultiplayerSessionChangeTypes::None;

	TArray<FString> SessionChangeTypeArray;
	if (SessionChangeTypeString.ParseIntoArray(SessionChangeTypeArray, TEXT(","), true) > 0)
	{
		for (const FString& SessionChangeType : SessionChangeTypeArray)
		{
			if (SessionChangeType == TEXT("ArbitrationPropertyChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::ArbitrationPropertyChange;
			}
			else if (SessionChangeType == TEXT("CustomPropertyChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::CustomPropertyChange;
			}
			else if (SessionChangeType == TEXT("HostDeviceTokenChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::HostDeviceTokenChange;
			}
			else if (SessionChangeType == TEXT("InitializationStateChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::InitializationStateChange;
			}
			else if (SessionChangeType == TEXT("MatchmakingStatusChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::MatchmakingStatusChange;
			}
			else if (SessionChangeType == TEXT("MemberCustomPropertyChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::MemberCustomPropertyChange;
			}
			else if (SessionChangeType == TEXT("MemberListChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::MemberListChange;
			}
			else if (SessionChangeType == TEXT("MemberStatusChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::MemberStatusChange;
			}
			else if (SessionChangeType == TEXT("SessionJoinabilityChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::SessionJoinabilityChange;
			}
			else if (SessionChangeType == TEXT("TournamentPropertyChange"))
			{
				ReturnValue = ReturnValue | XblMultiplayerSessionChangeTypes::TournamentPropertyChange;
			}
		}
	}

	return ReturnValue;
}

void FOnlineSessionMpsdGDK::SetCurrentUserActive(FGDKMultiplayerSessionHandle GDKSession, bool bIsActive)
{
	if (GDKSession)
	{
		//. Mark the current user as active, or otherwise
		XblMultiplayerSessionMemberStatus Status = bIsActive ? XblMultiplayerSessionMemberStatus::Active : XblMultiplayerSessionMemberStatus::Inactive;
		XblMultiplayerSessionCurrentUserSetStatus(GDKSession, Status);
	}
}

/** Get a resolved connection string from a session info */
static bool GetConnectStringFromSessionInfo(const FOnlineSessionInfoMpsdGDKPtr& SessionInfo, FString& ConnectInfo, int32 PortOverride = 0)
{
	bool bSuccess = false;

	if (SessionInfo.IsValid())
	{
		const TSharedPtr<FInternetAddr> IpAddr = SessionInfo->GetHostAddr();
		if (IpAddr.IsValid() && IpAddr->IsValid())
		{
			if (PortOverride != 0)
			{
				ConnectInfo = FString::Printf(TEXT("%s:%d"), *IpAddr->ToString(false), PortOverride);
			}
			else
			{
				ConnectInfo = FString::Printf(TEXT("%s"), *IpAddr->ToString(true));
			}

			bSuccess = true;
		}
	}

	return bSuccess;
}

bool FOnlineSessionMpsdGDK::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType)
{
	bool bSuccess = false;
	// Find the session
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (Session.IsValid())
	{
		if (FOnlineSessionInfoMpsdGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session->SessionInfo))
		{
			if (PortType == NAME_BeaconPort)
			{
				int32 BeaconListenPort = GetBeaconPortFromSessionSettings(Session->SessionSettings);
				bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
			}
			else if (PortType == NAME_GamePort)
			{
				if (Session->SessionSettings.bIsLANMatch)
				{
					bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
				}
				else
				{
					FOnlineSessionSetting* ConnectSetting = Session->SessionSettings.Settings.Find(SETTING_CUSTOM_JOIN_INFO);
					if (ConnectSetting != nullptr)
					{
						const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
						const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
						ConnectInfo = ConnectSetting->Data.ToString();
						int Pos = ConnectInfo.Find(KeyPtr);
						ConnectInfo = ConnectInfo.Left(Pos);
						bSuccess = !ConnectInfo.IsEmpty();
					}
				}
			}
		}

		if (!bSuccess)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info for session %s in GetResolvedConnectString()"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning,
			TEXT("Unknown session name (%s) specified to GetResolvedConnectString()"),
			*SessionName.ToString());
	}

	return bSuccess;
}

bool FOnlineSessionMpsdGDK::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo)
{
	bool bSuccess = false;
	if (SearchResult.Session.SessionInfo.IsValid())
	{
		FOnlineSessionInfoMpsdGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SearchResult.Session.SessionInfo);

		if (PortType == NAME_BeaconPort)
		{
			int32 BeaconListenPort = GetBeaconPortFromSessionSettings(SearchResult.Session.SessionSettings);
			bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo, BeaconListenPort);
		}
		else if (PortType == NAME_GamePort)
		{
			if (SearchResult.Session.SessionSettings.bIsLANMatch)
			{
				bSuccess = GetConnectStringFromSessionInfo(SessionInfo, ConnectInfo);
			}
			else
			{
				const FOnlineSessionSetting* ConnectSetting = SearchResult.Session.SessionSettings.Settings.Find(SETTING_CUSTOM_JOIN_INFO);
				if (ConnectSetting != nullptr)
				{
					const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
					const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
					ConnectInfo = ConnectSetting->Data.ToString();
					int Pos = ConnectInfo.Find(KeyPtr);
					ConnectInfo = ConnectInfo.Left(Pos);
					bSuccess = !ConnectInfo.IsEmpty();
				}
			}
		}
	}

	if (!bSuccess || ConnectInfo.IsEmpty())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info in search result to GetResolvedConnectString()"));
	}

	return bSuccess;
}

FOnlineSessionSettings* FOnlineSessionMpsdGDK::GetSessionSettings(FName SessionName)
{
	FNamedOnlineSessionPtr MySession = GetNamedSessionPtr(SessionName);
	if (!MySession.IsValid())
	{
		return nullptr;
	}

	return &MySession->SessionSettings;
}

bool FOnlineSessionMpsdGDK::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray<FUniqueNetIdRef> Players;
	Players.Add(FUniqueNetIdGDK::Cast(PlayerId));
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

bool FOnlineSessionMpsdGDK::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccess = false;
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (Session.IsValid())
	{
		if (Session->SessionInfo.IsValid())
		{
			for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

				FUniqueNetIdMatcher PlayerMatch(*PlayerId);
				if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
				{
					Session->RegisteredPlayers.Add(PlayerId);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *Players[PlayerIdx]->ToDebugString(), *SessionName.ToString());
				}
				if (GDKSubsystem->IsLocalPlayer(*PlayerId))
				{
					FSessionSettings* MemberSettings = Session->SessionSettings.MemberSettings.Find(PlayerId);
					if (MemberSettings)
					{
						if (!MemberSettings->Find(FName("Registered")))
						{
							MemberSettings->Add(FName("Registered"), FOnlineSessionSetting(true, EOnlineDataAdvertisementType::ViaOnlineService));
							UE_LOG_ONLINE_SESSION(Log, TEXT("Marking Player %s as registered in session %s"), *Players[PlayerIdx]->ToDebugString(), *SessionName.ToString());
						}
					}
				}

				RegisterVoice(*PlayerId);
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to join for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterPlayers_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

bool FOnlineSessionMpsdGDK::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray<FUniqueNetIdRef> Players;
	Players.Add(FUniqueNetIdGDK::Cast(PlayerId));
	return UnregisterPlayers(SessionName, Players);
}

bool FOnlineSessionMpsdGDK::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccess = false;

	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (Session.IsValid())
	{
		if (Session->SessionInfo.IsValid())
		{
			for (int32 PlayerIdx=0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

				FUniqueNetIdMatcher PlayerMatch(*PlayerId);
				int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
				if (RegistrantIndex != INDEX_NONE)
				{
					Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
					UnregisterVoice(*PlayerId);
					// The unetdriver cleanup triggers this unregister before the OnSessionChanged event so trigger this delegate here.
					// the test in OnSessionChanged will remain as different underlying net systems will rely on it.
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterPlayers_Delegate);
					GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionParticipantLeftDelegates(SessionName, *PlayerId, EOnSessionParticipantLeftReason::Left);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Player %s is not part of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
				}
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to leave for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterPlayers_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

bool FOnlineSessionMpsdGDK::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("UFOnlineSessionMpsdGDK::UpdateSession %s"), *SessionName.ToString());
	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to update session %s, it does not exist"), *SessionName.ToString());
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	EOnlineSessionState::Type SessionState = NamedSession->SessionState;
	if (SessionState <= EOnlineSessionState::Creating || SessionState >= EOnlineSessionState::Destroying)
	{
		GDKSubsystem->ExecuteNextTick([this, SessionName, SessionState]()
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to update session %s, it is state %s, which may not be updated"), *SessionName.ToString(), EOnlineSessionState::ToString(SessionState));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UpdateSession_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
		});
		return false;
	}

	NamedSession->SessionSettings = UpdatedSessionSettings;

	if (bShouldRefreshOnlineData)
	{
		FGDKContextHandle GDKContext;
		const FUniqueNetIdPtr& HostNetId(NamedSession->OwningUserId);
		if (HostNetId.IsValid())
		{
			GDKContext = GDKSubsystem->GetGDKContext(*HostNetId);
		}

		if (!GDKContext.IsValid())
		{
			if (!bOnlyHostUpdateSession)
			{
				const FUniqueNetIdPtr& LocalOwnerId(NamedSession->LocalOwnerId);
				if (LocalOwnerId.IsValid())
				{
					GDKContext = GDKSubsystem->GetGDKContext(*LocalOwnerId);
				}
			}
		}

		if (!GDKContext.IsValid())
		{
			GDKSubsystem->ExecuteNextTick([this, SessionName]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UpdateSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
			});
			return false;
		}

		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKUpdateSession>(GDKSubsystem, GDKContext, SessionName, UpdatedSessionSettings, MAX_RETRIES);
		return true;
	}
	else //Update Player constants/Player group info
	{
		int32 NumPendingUpdates = 0;
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		if (GDKInfo.IsValid())
		{
			if (FGDKMultiplayerSessionHandle Session = GDKInfo->GetGDKMultiplayerSession())
			{
				const XblMultiplayerSessionMember* SessionMembers = nullptr;
				uint64 NumSessionMembers = 0;
				HRESULT Result = XblMultiplayerSessionMembers(Session, &SessionMembers, &NumSessionMembers);
				if (SUCCEEDED(Result))
				{					
					for (uint32 i=0;i< NumSessionMembers;++i)
					{
						const XblMultiplayerSessionMember& Member = SessionMembers[i];
						if (FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*FUniqueNetIdGDK::Create(Member.Xuid)))
						{
							constexpr const int32 MaxRetryCount = 10;
							GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKUpdateSessionMember>(GDKSubsystem, GDKContext, SessionName, Session, MaxRetryCount);
							++NumPendingUpdates;
						}
					}
				}
			}
		}

		if (NumPendingUpdates == 0)
		{
			GDKSubsystem->ExecuteNextTick([this, SessionName]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UpdateSession_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnUpdateSessionCompleteDelegates(SessionName, false);
			});
			return false;
		}

		return true;
	}
}

void FOnlineSessionMpsdGDK::ReadSettingsFromGDKSessionJson(FGDKMultiplayerSessionHandle GDKSession, FOnlineSession& Session, FGDKContextHandle GDKContext)
{
	FSessionSettings NewSettings;
	const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
	if (ensure(SessionProperties))
	{
		FString PropertiesJson = UTF8_TO_TCHAR(SessionProperties->SessionCustomPropertiesJson);
		ReadSettingsFromJson(PropertiesJson, Session, NewSettings, GDKContext);

		UpdateMatchMembersJson(NewSettings, GDKSession);

		// Finally, replace existing with new settings (this deletes settings that have been removed)
		Session.SessionSettings.Settings = MoveTemp(NewSettings);

		UpdateMatchMembers(Session.SessionSettings, GDKSession);

		TOptional<FName> SessionName = GetNamedSessionNameForGDKSessionHandle(GDKSession);
		if (SessionName.IsSet())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_ReadSettingsFromGDKSessionJson_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionSettingsUpdatedDelegates(SessionName.GetValue(), Session.SessionSettings);
		}
	}
}

void FOnlineSessionMpsdGDK::ReadSettingsFromGDKSearchHandleJson(FGDKMultiplayerSearchHandle SearchHandle, FOnlineSession& Session, FGDKContextHandle GDKContext)
{
	FSessionSettings NewSettings;
	const char* SessionCustomPropertiesJson = nullptr;
	HRESULT Result =  XblMultiplayerSearchHandleGetCustomSessionPropertiesJson(SearchHandle, &SessionCustomPropertiesJson);
	if (SUCCEEDED(Result))
	{		
		FString PropertiesJson = UTF8_TO_TCHAR(SessionCustomPropertiesJson);
		ReadSettingsFromJson(PropertiesJson, Session, NewSettings, GDKContext);
		// Finally, replace existing with new settings (this deletes settings that have been removed)
		Session.SessionSettings.Settings = MoveTemp(NewSettings);
	}
	UE_CLOG_ONLINE_SESSION(FAILED(Result), Warning, TEXT("Failed to get session Properties from search handle. result = (0x%0.8X)"), Result);

}

void FOnlineSessionMpsdGDK::ReadSettingsFromJson(FString& Json, FOnlineSession& Session, FSessionSettings& NewSettings, FGDKContextHandle GDKContext)
{

	// Copy existing values that are not service based
	for (const FOnlineKeyValuePairs<FName, FOnlineSessionSetting>::ElementType& Pair : Session.SessionSettings.Settings)
	{
		const FName						SettingName = Pair.Key;
		const FOnlineSessionSetting&	SettingValue = Pair.Value;

		if (SettingValue.AdvertisementType < EOnlineDataAdvertisementType::ViaOnlineService)
		{
			NewSettings.Add(SettingName, SettingValue);
		}
	}
	
	TSharedPtr< FJsonObject > JObj;
	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(Json);

	if (FJsonSerializer::Deserialize(Reader, JObj) && JObj.IsValid())
	{
		for (const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::ElementType& Pair : JObj->Values)
		{
			const FJsonObject::FStringType&	JSettingName(Pair.Key);
			const TSharedPtr<FJsonValue>&	JSettingValue = Pair.Value;

			FOnlineSessionSetting NewSetting;

			// Create setting of matching data type
			switch (JSettingValue->Type)
			{
			case EJson::Array:
			case EJson::Object:
			case EJson::String:
				NewSetting = FOnlineSessionSetting(JSettingValue->AsString(), EOnlineDataAdvertisementType::ViaOnlineService);
				break;

			case EJson::Number:
				NewSetting = FOnlineSessionSetting(JSettingValue->AsNumber(), EOnlineDataAdvertisementType::ViaOnlineService);
				break;

			case EJson::Boolean:
				NewSetting = FOnlineSessionSetting(JSettingValue->AsBool(), EOnlineDataAdvertisementType::ViaOnlineService);
				break;

			default: continue;
			}

			if (JSettingName == TEXT("HostGDKUserName"))
			{
				Session.OwningUserName = JSettingValue->AsString();					
			}
			else if (JSettingName == TEXT("_flags") && NewSetting.Data.GetType() == EOnlineKeyValuePairDataType::String)
			{
				FString SessionSettingsFlagsValue;
				NewSetting.Data.GetValue(SessionSettingsFlagsValue);

				int16 SessionSettingsFlags = 0;
				LexFromString(SessionSettingsFlags, *SessionSettingsFlagsValue);

				int32 BitShift = 0;
				Session.SessionSettings.bShouldAdvertise = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bAllowJoinInProgress = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bIsLANMatch = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bIsDedicated = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bUsesStats = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bAllowInvites = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bUsesPresence = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bAllowJoinViaPresence = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bAllowJoinViaPresenceFriendsOnly = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
				Session.SessionSettings.bAntiCheatProtected = (SessionSettingsFlags & (1 << BitShift++)) ? true : false;
			}
			else
			{
				NewSettings.Add(FName(JSettingName), NewSetting);
			}
		}
	}
}

void FOnlineSessionMpsdGDK::ExtractJsonMemberSettings(FGDKMultiplayerSessionHandle GDKSession, FString& OutJsonString)
{
	bool NeedComma = false;

	// We could have used JsonWriter here, but it escapes JSON characters, so instead
	// we just build it manually so that we can pass our snippets of JSON directly

	OutJsonString = FString(TEXT("{\"Members\":["));

	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);
	if (SUCCEEDED(Result))
	{
		for (uint32 i = 0; i<NumMembers; ++i)
		{
			const XblMultiplayerSessionMember& Member = Members[i];
			OutJsonString += FString::Printf(TEXT("%s{\"xuid\":\"%lld\", \"constants\":%s, \"properties\":%s}"),
						NeedComma ? TEXT(",") : TEXT(""),
						Member.Xuid,
						UTF8_TO_TCHAR(Member.CustomConstantsJson),
						UTF8_TO_TCHAR(Member.CustomPropertiesJson));

			NeedComma = true;
		}
	}

	OutJsonString +=  FString(TEXT("]}"));
}

void FOnlineSessionMpsdGDK::UpdateMatchMembersJson(FSessionSettings& UpdatedSettings, FGDKMultiplayerSessionHandle GDKSession)
{
	FString MemberSessionJson;

	ExtractJsonMemberSettings(GDKSession, MemberSessionJson);
	UpdatedSettings.Remove(SETTING_MATCH_MEMBERS_JSON);
	UpdatedSettings.Add(SETTING_MATCH_MEMBERS_JSON, FOnlineSessionSetting(MemberSessionJson, EOnlineDataAdvertisementType::DontAdvertise));
}

void FOnlineSessionMpsdGDK::UpdateMatchMembers(FOnlineSessionSettings& UpdatedSettings, FGDKMultiplayerSessionHandle GDKSession)
{
	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);

	if (SUCCEEDED(Result))
	{
		for (uint32 i = 0; i < NumMembers; ++i)
		{
			const XblMultiplayerSessionMember& Member = Members[i];
			TSharedPtr< FJsonObject > JsonObject;
			TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(Member.CustomPropertiesJson);
			FString Registered = FString(TEXT("Registered"));
			if (FJsonSerializer::Deserialize(Reader, JsonObject))
			{
				FSessionSettings * MemberSettings = UpdatedSettings.MemberSettings.Find(FUniqueNetIdGDK::Create(Member.Xuid));
				if (!MemberSettings)
				{
					MemberSettings = &UpdatedSettings.MemberSettings.Add(FUniqueNetIdGDK::Create(Member.Xuid), FSessionSettings());
				}

				for (FSessionSettings::TIterator It = MemberSettings->CreateIterator(); It; ++It)
				{
					// Only clear values that are marked for service use, they will get refreshed below. So this takes care of reflecting deletes from the service.
					if (It.Value().AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
					{
						if (It.Key().ToString() != Registered)
						{
							It.RemoveCurrent();
						}
					}
				}

				for (const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::ElementType& Pair : JsonObject->Values)
				{
					const FJsonObject::FStringType&	JSettingName(Pair.Key);
					const TSharedPtr<FJsonValue>&	JSettingValue = Pair.Value;

					FOnlineSessionSetting NewSetting;

					// Create setting of matching data type
					switch (JSettingValue->Type)
					{
					case EJson::Array:
					case EJson::Object:
					case EJson::String:
						NewSetting = FOnlineSessionSetting(JSettingValue->AsString(), EOnlineDataAdvertisementType::ViaOnlineService);
						break;

					case EJson::Number:
						NewSetting = FOnlineSessionSetting(JSettingValue->AsNumber(), EOnlineDataAdvertisementType::ViaOnlineService);
						break;

					case EJson::Boolean:
						NewSetting = FOnlineSessionSetting(JSettingValue->AsBool(), EOnlineDataAdvertisementType::ViaOnlineService);
						break;

					default: continue;
					}
					MemberSettings->Add(FName(JSettingName), NewSetting);
				}
			}
		}
	}
}

// function to handle checking if our value is different for a setting
bool IsSessionSettingsValueDifferent(const FString& SettingName, const FString& NewSettingValue, TSharedPtr<FJsonObject> CurrentSessionSettingsJson)
{
	FString OldSettingValue;
	if (CurrentSessionSettingsJson.IsValid() && CurrentSessionSettingsJson->TryGetStringField(SettingName, OldSettingValue))
	{
		if (OldSettingValue != NewSettingValue)
		{
			if (NewSettingValue.IsEmpty())
			{
				UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("  Removing existing setting.  Name=[%s] OldValue=[%s]"), *SettingName, *OldSettingValue);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("  Updating setting value.  Name=[%s] OldValue=[%s] NewValue=[%s]"), *SettingName, *OldSettingValue, *NewSettingValue);
			}
			return true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("  Ignoring same setting value.  Name=[%s] Value=[%s]"), *SettingName, *OldSettingValue);
			return false;
		}
	}
	else
	{
		if (!NewSettingValue.IsEmpty())
		{
			UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("  Adding new setting.  Name=[%s] Value=[%s]"), *SettingName, *NewSettingValue);
			return true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("  Ignoring empty non-existent setting.  Name=[%s]"), *SettingName);
			return false;
		}
	}
}

bool FOnlineSessionMpsdGDK::WriteSettingsToGDKJson(const FOnlineSessionSettings& SessionSettings, FGDKMultiplayerSessionHandle GDKSession, FGDKUserHandle HostUser, FOnlineSubsystemGDK* GDKSubsystem)
{
	bool bHasChanged = false;

	TSharedPtr<FJsonObject> CurrentSessionSettingsJson;

	// Deserialize our current values
	const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
	if (!SessionProperties)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("WriteSettingsToGDKJson: Cannot get session properties."));
		return false;
	}

	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(FString(UTF8_TO_TCHAR(SessionProperties->SessionCustomPropertiesJson)));
	FJsonSerializer::Deserialize(JsonReader, CurrentSessionSettingsJson);

	UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("Applying session settings onto GDK Session object"));

	// Set our session setting bools
	{
		int32 BitShift = 0;
		int32 SessionSettingsFlags = 0;
		SessionSettingsFlags |= ((int32)SessionSettings.bShouldAdvertise) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bAllowJoinInProgress) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bIsLANMatch) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bIsDedicated) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bUsesStats) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bAllowInvites) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bUsesPresence) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bAllowJoinViaPresence) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bAllowJoinViaPresenceFriendsOnly) << BitShift++;
		SessionSettingsFlags |= ((int32)SessionSettings.bAntiCheatProtected) << BitShift++;

		const FString NewFlagsValue(FString::FromInt(SessionSettingsFlags));
		if (IsSessionSettingsValueDifferent(TEXT("_flags"), NewFlagsValue, CurrentSessionSettingsJson))
		{
			FString SessionSettingsFlagsName(TEXT("_flags"));
			// Wrap with quotes so we send it as a JSON string and not a JSON number (we don't want to convert to double).  FString::FromInt will not wrap in quotes.
			FString SessionSettingsFlagsValue = FString::Printf(TEXT("\"%d\""), SessionSettingsFlags);

			XblMultiplayerSessionSetCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*SessionSettingsFlagsName), TCHAR_TO_UTF8(*SessionSettingsFlagsValue));
			bHasChanged = true;
		}
	}

	// Set our session custom settings
	for (FSessionSettings::TConstIterator It(SessionSettings.Settings); It; ++It)
	{
		const FName& SettingName = It.Key();
		const FOnlineSessionSetting& SettingValue = It.Value();

		// Only upload values that are marked for service use
		if (SettingValue.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
		{
			const FString SettingNameString = SettingName.ToString();
			const FString SettingValueString = SettingValue.Data.ToString();

			if (IsSessionSettingsValueDifferent(SettingNameString, SettingValueString, CurrentSessionSettingsJson))
			{
				if (SettingValueString.IsEmpty())
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("WriteSettingsToGDKJson: %s: <Empty>."), *SettingNameString);
					XblMultiplayerSessionDeleteCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*SettingNameString));
				}
				else
				{
					FString SettingValueStringJson = (SettingValue.Data.GetType() == EOnlineKeyValuePairDataType::String) ? FString::Printf(TEXT("\"%s\""), *SettingValueString) : *SettingValueString;
					
					UE_LOG_ONLINE_SESSION(Warning, TEXT("WriteSettingsToGDKJson: %s: %s."), *SettingNameString, *SettingValueStringJson);
					XblMultiplayerSessionSetCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*SettingNameString), TCHAR_TO_UTF8(*SettingValueStringJson));
				}

				bHasChanged = true;
			}
		}
	}

	// Set our local members session custom settings
	const XblMultiplayerSessionMember* Member = XblMultiplayerSessionCurrentUser(GDKSession);
	if(Member!=nullptr)
	{	
		TSharedPtr< FJsonObject > JsonObject;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(Member->CustomPropertiesJson);
		FJsonSerializer::Deserialize(Reader, JsonObject);	
		const FSessionSettings * MemberSettings = SessionSettings.MemberSettings.Find(FUniqueNetIdGDK::Create(Member->Xuid));
		if (MemberSettings)
		{
			for (FSessionSettings::TConstIterator It(*MemberSettings); It; ++It)
			{
				const FName& SettingName = It.Key();
				const FOnlineSessionSetting& SettingValue = It.Value();

				// Only upload values that are marked for service use
				if (SettingValue.AdvertisementType >= EOnlineDataAdvertisementType::ViaOnlineService)
				{
					const FString SettingNameString = SettingName.ToString();
					const FString SettingValueString = SettingValue.Data.ToString();
					
					//CDATODO make only update changes
					if (SettingValueString.IsEmpty())
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("WriteSettingsToGDKJson: %s: <Empty>."), *SettingNameString);
						XblMultiplayerSessionCurrentUserDeleteCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*SettingNameString));
					}
					else
					{
						FString SettingValueStringJson = (SettingValue.Data.GetType() == EOnlineKeyValuePairDataType::String) ? FString::Printf(TEXT("\"%s\""), *SettingValueString) : *SettingValueString;
													UE_LOG_ONLINE_SESSION(Warning, TEXT("WriteSettingsToGDKJson: %s: %s."), *SettingNameString, *SettingValueStringJson);
						XblMultiplayerSessionCurrentUserSetCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*SettingNameString), TCHAR_TO_UTF8(*SettingValueStringJson));

					}
					bHasChanged = true;
					
				}
			}
		}
	}

	// If we have a host, write the name of the host to the session
	// This is a workaround for the client not having a reliable way to determine who the host is in splitscreen
	if (HostUser.IsValid())
	{
		uint64 GDKUserId = 0;
		HRESULT Res = XUserGetId(HostUser, &GDKUserId);
		UE_CLOG_ONLINE_SESSION(FAILED(Res), Warning, TEXT("Failed to get host user ID when hosting session. result = (0x%0.8X)"),Res);

		UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("Finished applying session stats. %d"), Res);
		const FString HostStatName(TEXT("HostGDKUserId"));
		const FString GDKUserIdString(FString::Printf(TEXT("\"%llu\""), GDKUserId));
		if (IsSessionSettingsValueDifferent(HostStatName, GDKUserIdString, CurrentSessionSettingsJson))
		{
			XblMultiplayerSessionSetCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*HostStatName), TCHAR_TO_UTF8(*GDKUserIdString));
			bHasChanged = true;
		}

		const FString HostUserName(TEXT("HostGDKUserName"));
		const FOnlineIdentityGDKPtr GDKIdentity = GDKSubsystem->GetIdentityGDK();
		const FString Gamertag = GDKIdentity->GetPlayerNickname(HostUser);
		const FString GDKUserNameString(FString::Printf(TEXT("\"%s\""), *Gamertag));
		if (IsSessionSettingsValueDifferent(HostUserName, Gamertag, CurrentSessionSettingsJson))
		{
			XblMultiplayerSessionSetCustomPropertyJson(GDKSession, TCHAR_TO_UTF8(*HostUserName), TCHAR_TO_UTF8(*GDKUserNameString));
			bHasChanged = true;
		}
	}

	// Get our session's join/read restriction from our settings (don't need to check these for
	// bHasChanged, as they're based on the flags above that would have already triggered a change)
	const XblMultiplayerSessionRestriction SessionRestriction = GetGDKSessionRestrictionFromSettings(SessionSettings);
	XblMultiplayerSessionPropertiesSetJoinRestriction(GDKSession, SessionRestriction);
	XblMultiplayerSessionPropertiesSetReadRestriction(GDKSession, SessionRestriction);

	UE_LOG_ONLINE_SESSION(VeryVerbose, TEXT("Finished applying session stats.  bHasChanged=[%d]"), bHasChanged);
	return bHasChanged;
}

void FOnlineSessionMpsdGDK::CreateSearchHandle(const FOnlineSessionSettings& SessionSettings, FGDKMultiplayerSessionHandle GDKSession, FGDKContextHandle GDKContext, const FOnCreateSearchHandleCompleteDelegate& Delegate)
{
	const XblMultiplayerSessionReference* SessionReference = XblMultiplayerSessionSessionReference(GDKSession);
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKCreateSearchHandle>(
		GDKSubsystem,
		GDKContext,
		*SessionReference,
		SessionSettings,
		Delegate);
}

bool FOnlineSessionMpsdGDK::StartSession(FName SessionName)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_StartSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, false);
		return false;
	}

	// Can't start a match multiple times
	if (Session->SessionState != EOnlineSessionState::Pending &&
		Session->SessionState != EOnlineSessionState::Ended)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online session (%s) in state %s"),
			*SessionName.ToString(),
			EOnlineSessionState::ToString(Session->SessionState));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_StartSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, false);
		return false;
	}

	// Generate a new RoundId for GDK events.
	FOnlineSessionInfoMpsdGDKPtr SessionInfoGDK = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session->SessionInfo);
	if (SessionInfoGDK.IsValid())
	{
		SessionInfoGDK->SetRoundId(FGuid::NewGuid());
	}

	// TODO: Handle join in progress vs. not join in progress.
	Session->SessionState = EOnlineSessionState::InProgress;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_StartSession_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnStartSessionCompleteDelegates(SessionName, true);
	return true;
}

bool FOnlineSessionMpsdGDK::EndSession(FName SessionName)
{
	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_EndSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, false);
		return false;
	}

	if (Session->SessionState != EOnlineSessionState::InProgress)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
			*SessionName.ToString(),
			EOnlineSessionState::ToString(Session->SessionState));
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_EndSession_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, false);
		return false;
	}

	Session->SessionState = EOnlineSessionState::Ended;
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_EndSession_Delegate);
	GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnEndSessionCompleteDelegates(SessionName, true);
	return true;
}

bool FOnlineSessionMpsdGDK::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	UE_LOG_ONLINE_SESSION(Log, TEXT("Attempting to destroy session %s"), *SessionName.ToString());

	// Technically we can't actually destroy a session on GDK, all we can do is Leave() it,
	// hope everyone else leaves also, and let it time out.

	FNamedOnlineSessionPtr Session = GetNamedSessionPtr(SessionName);
	if (!Session.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_NamedSessionInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, false);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_NamedSessionInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			}
		});
		return false;
	}

	if (Session->SessionState == EOnlineSessionState::Destroying)
	{
		// Purposefully skip the delegate call as one should already be in flight
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Already in process of destroying session (%s)"), *SessionName.ToString());
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_SessionStateDestroying_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, false);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_SessionStateDestroying_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, false);
			}
		});
		return false;
	}

	Session->SessionState = EOnlineSessionState::Destroying;

	FOnlineSessionInfoMpsdGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(Session->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Destroying an online session (%s) will null GDK info. No writes to the MPSD will occur."), *SessionName.ToString());
		RemoveNamedSession(SessionName);
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_SessionInfoInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, true);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_SessionInfoInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, true);
			}
		});
		return false;
	}

	FGDKMultiplayerSessionHandle GDKSession = GDKSessionInfo->GetGDKMultiplayerSession();
	if (!GDKSession)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Destroying a session with a null GDK MultiplayerSession (%s)"), *SessionName.ToString());
		RemoveNamedSession(SessionName);
		GDKSubsystem->ExecuteNextTick([this, SessionName, CompletionDelegate]()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_GDKMultiplayerSessionHandleInvalid_CompletionDelegate);
				CompletionDelegate.ExecuteIfBound(SessionName, true);
			}
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_DestroySession_GDKMultiplayerSessionHandleInvalid_TriggerDelegates);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnDestroySessionCompleteDelegates(SessionName, true);
			}
		});
		return true;
	}

	FSessionMessageRouterPtr SessionRouter = GDKSubsystem->GetSessionMessageRouter();
	if (SessionRouter.IsValid())
	{
		SessionRouter->ClearOnSessionChangedDelegate(OnSessionChangedDelegate, XblMultiplayerSessionSessionReference(GDKSession));
	}

	if (IsConsoleHost(GDKSession))
	{
		GDKSubsystem->GetMatchmakingInterfaceGDK()->RemoveMatchmakingTicket(Session->SessionName);
	}

	GDKSubsystem->ExecuteNextTick([this, SessionName, GDKSession, CompletionDelegate]
	{
		// Start a task to Leave() the first local user found in the session.
		// The tasks will chain until every local user is removed.
		CreateDestroyTask(SessionName, GDKSession, GDKSubsystem, true, CompletionDelegate);
	});
	return true;
}

FUniqueNetIdPtr FOnlineSessionMpsdGDK::CreateSessionIdFromString(const FString& SessionIdStr)
{
	FUniqueNetIdPtr SessionId;
	if (!SessionIdStr.IsEmpty())
	{
		SessionId = FUniqueNetIdString::Create(SessionIdStr, GDK_SUBSYSTEM);
	}
	return SessionId;
}

FNamedOnlineSession* FOnlineSessionMpsdGDK::GetNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return &Session.Get();
		}
	}
	return nullptr;
}

FNamedOnlineSessionPtr FOnlineSessionMpsdGDK::GetNamedSessionPtr(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return Session;
		}
	}
	return nullptr;
}

void FOnlineSessionMpsdGDK::RemoveNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex]->SessionName == SessionName)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("Removing Session %s"), *SessionName.ToString());
			Sessions.RemoveAtSwap(SearchIndex);
			return;
		}
	}
}

bool FOnlineSessionMpsdGDK::HasPresenceSession() const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionSettings.bUsesPresence)
		{
			return true;
		}
	}

	return false;
}

FNamedOnlineSession* FOnlineSessionMpsdGDK::AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, SessionSettings);
	Sessions.Emplace(NewSession);
	return &NewSession.Get();
}

FNamedOnlineSessionRef FOnlineSessionMpsdGDK::AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, SessionSettings);
	Sessions.Emplace(NewSession);
	return NewSession;
}

FNamedOnlineSession* FOnlineSessionMpsdGDK::AddNamedSession(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, Session);
	Sessions.Emplace(NewSession);
	return &NewSession.Get();
}

FNamedOnlineSessionRef FOnlineSessionMpsdGDK::AddNamedSessionRef(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	FNamedOnlineSessionRef NewSession = MakeShared<FNamedOnlineSession, ESPMode::ThreadSafe>(SessionName, Session);
	Sessions.Emplace(NewSession);
	return NewSession;
}

EOnlineSessionState::Type FOnlineSessionMpsdGDK::GetSessionState(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (const FNamedOnlineSessionRef& Session : Sessions)
	{
		if (Session->SessionName == SessionName)
		{
			return Session->SessionState;
		}
	}

	return EOnlineSessionState::NoSession;
}

bool FOnlineSessionMpsdGDK::StartCreateSession(
	const int32 UserIndex,
	const FOnlineSessionSettings& SessionSettings,
	const FString& Keyword, const FString& SessionTemplateName, FName SessionName, FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate)
{
	// Create a new session and deep copy the game settings
	FNamedOnlineSessionRef Session = AddNamedSessionRef(SessionName, SessionSettings);
	Session->SessionState = EOnlineSessionState::Creating;
	Session->bHosting = true;
	Session->LocalOwnerId = FUniqueNetIdGDK::Create(0);

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(UserIndex);

	return InternalStartCreateSession(GDKContext, SessionSettings, Keyword, SessionTemplateName, SessionName, Delegate);
}

bool FOnlineSessionMpsdGDK::StartCreateSession(
	const FUniqueNetId& UserId,
	const FOnlineSessionSettings& SessionSettings,
	const FString& Keyword, const FString& SessionTemplateName, FName SessionName, FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate)
{
	FUniqueNetIdGDKRef HostGDKId = FUniqueNetIdGDK::Cast(UserId);

	// Create a new session and deep copy the game settings
	FNamedOnlineSessionRef Session = AddNamedSessionRef(SessionName, SessionSettings);
	Session->SessionState = EOnlineSessionState::Creating;
	Session->bHosting = true;
	Session->LocalOwnerId = HostGDKId;

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(UserId);

	return InternalStartCreateSession(GDKContext, SessionSettings, Keyword, SessionTemplateName, SessionName, Delegate);
}

bool FOnlineSessionMpsdGDK::InternalStartCreateSession(
	FGDKContextHandle GDKContext,
	const FOnlineSessionSettings& SessionSettings,
	const FString& Keyword, const FString& SessionTemplateName, FName SessionName, FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate)
{
	if (!GDKContext)
	{
		return false;
	}

	GDKSubsystem->EnableSessionEventHandlers(GDKContext);

	uint64 GDKUserId;
	if (FAILED(XblContextGetXboxUserId(GDKContext, &GDKUserId)))
	{
		return false;
	}

	FString SessionId = FGuid::NewGuid().ToString();
	//Platform::String^ UniqueSessionName = ref new Platform::String(*SessionId);

	FString TemplateNameString;
	const FOnlineSessionSetting* TemplateNameSetting = SessionSettings.Settings.Find(SETTING_SESSION_TEMPLATE_NAME);
	if (TemplateNameSetting)
	{
		TemplateNameSetting->Data.GetValue(TemplateNameString);
	}

	const ANSICHAR* Scid = nullptr;;
	XblGetScid(&Scid);

	XblMultiplayerSessionReference SessionRef = XblMultiplayerSessionReferenceCreate(Scid, TCHAR_TO_UTF8(*SessionTemplateName), TCHAR_TO_UTF8(*SessionId));

	TArray<uint64> InitiatorGDKUserIds;
	InitiatorGDKUserIds.Add(GDKUserId);


	FString CustomConstantsJson(TEXT("{}"));
	const FOnlineSessionSetting* CustomJsonSetting = SessionSettings.Settings.Find(SETTING_CUSTOM);
	if (CustomJsonSetting)
	{
		FString CustomJsonStr;
		CustomJsonSetting->Data.GetValue(CustomJsonStr);

		FString ReaderJsonStr(CustomJsonStr);
		TSharedPtr< FJsonObject > JsonObject;
		TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(MoveTemp(ReaderJsonStr));
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			CustomConstantsJson = CustomJsonStr;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpsdGDK::InternalStartCreateSession] Custom Json FString set as SETTING_CUSTOM was not valid Json [%s]"), *CustomJsonStr);
		}
	}

	XblMultiplayerSessionInitArgs InitArgs;
	FMemory::Memzero(&InitArgs, sizeof(InitArgs));
	InitArgs.MaxMembersInSession = 0;
	InitArgs.Visibility = XblMultiplayerSessionVisibility::Open;
	InitArgs.InitiatorXuids = InitiatorGDKUserIds.GetData();
	InitArgs.InitiatorXuidsCount = InitiatorGDKUserIds.Num();
	const FTCHARToUTF8 CustomConstsJsonStr(*CustomConstantsJson);
	InitArgs.CustomJson = CustomConstsJsonStr.Get();

	FGDKMultiplayerSessionHandle GDKSession = FGDKMultiplayerSessionHandle(XblMultiplayerSessionCreateHandle(GDKUserId, &SessionRef, &InitArgs));

	// Add keyword (Writing to an array to support multiple keywords in the future: WMM TODO
	TArray<const ANSICHAR*> Keywords;
	TArray<TArray<ANSICHAR>> KeywordBuff;
	if(!Keyword.IsEmpty())
	{
		TArray<ANSICHAR>& AnsicharBuff = KeywordBuff.AddDefaulted_GetRef();
		const FTCHARToUTF8 TempAnsiKeyword(*Keyword);
		AnsicharBuff.Append(TempAnsiKeyword.Get(), FCStringAnsi::Strlen(TempAnsiKeyword.Get()) + 1);
		Keywords.Add(AnsicharBuff.GetData());
	}

	XblMultiplayerSessionPropertiesSetKeywords(GDKSession, Keywords.GetData(), Keywords.Num());

	FString PlayerCustomConstantBlob;
	FString Key = FString::Printf(TEXT("%s%llu"), SETTING_SESSION_MEMBER_CONSTANT_CUSTOM_JSON_XUID_PREFIX, GDKUserId);

	const FOnlineSessionSetting* CurrentPlayerConstantCustomJsonSetting = SessionSettings.Settings.Find(FName(*Key));
	if (CurrentPlayerConstantCustomJsonSetting)
	{
		CurrentPlayerConstantCustomJsonSetting->Data.GetValue(PlayerCustomConstantBlob);
	}

	// Set current user to be active and joined
	if (PlayerCustomConstantBlob.IsEmpty())
	{
		XblMultiplayerSessionJoin(GDKSession, nullptr, true, true);
	}
	else
	{
		XblMultiplayerSessionJoin(GDKSession, TCHAR_TO_UTF8(*PlayerCustomConstantBlob), true, true);
	}
		
	// Indicate what events to subscribe to
	HRESULT Result = XblMultiplayerSessionSetSessionChangeSubscription(GDKSession, GetSubscriptionType(SessionSettings));
	if (Result != S_OK)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Could not Subscribe to session (0x%0.8X)."), Result);
	}

	// The client's external/WAN address should be set for the user so that other clients can initiate
	// connections.  Xbox platform doesn't currently have a way to retrieve this address so the local
	// address is used instead.  This means that clients will only be able to connect to each other if
	// they are on the same local network. WMM: TODO!! 

	GDKSubsystem->RefreshNetworkConnectivityLevel();

	bool BindAll = false;
	TSharedRef<FInternetAddr> LocalIp = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, BindAll);
#if WITH_ENGINE
	LocalIp->SetPort(FURL::UrlConfig.DefaultPort);
#else
	FString Port;
	// Allow the command line to override the default port
	if (FParse::Value(FCommandLine::Get(),TEXT("Port="),Port) == false)
	{
		Port = GConfig->GetStr( TEXT("URL"), TEXT("Port"), GEngineIni );
	}
	
	LocalIp->SetPort(FCString::Atoi( *Port ));
#endif //WITN_ENGINE
	if (LocalIp->IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Attempting to set secure device address for current user: address: %s."), *LocalIp->ToString(true));
		Result = XblMultiplayerSessionCurrentUserSetSecureDeviceAddressBase64(GDKSession, TCHAR_TO_UTF8(*FBase64::Encode(LocalIp->ToString(true))));
		if (Result != S_OK)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Could not set secure device address for current user: address: %s (0x%0.8X)."), *LocalIp->ToString(true), Result);
		}
	}

	// Add custom settings
	FGDKUserHandle GDKUser;
	XblContextGetUser(GDKContext, GDKUser.GetInitReference());
		
	WriteSettingsToGDKJson(SessionSettings, GDKSession, GDKUser, GDKSubsystem);

	bool bCreateActivity = AreInvitesAndJoinViaPresenceAllowed(SessionSettings);
	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKCreateSession>(
		GDKSubsystem,
		GDKContext,
		FUniqueNetIdGDK::Create(GDKUserId),
		SessionName,
		GDKSession,
		true,
		bCreateActivity,
		Delegate);

	return true;
}

void FOnlineSessionMpsdGDK::DetermineSessionHost(FName SessionName, FGDKMultiplayerSessionHandle GDKSession)
{
	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		return;
	}

	bool bHosting = NamedSession->bHosting;
	int32 HostingPlayerNum = NamedSession->HostingPlayerNum;

	const XblMultiplayerSessionMember* SessionMember = GetGDKSessionHost(GDKSession);
	if (!SessionMember)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Host not currently set, unable to update host/owner session information"));
		return;
	}

	const FUniqueNetIdRef HostNetId = FUniqueNetIdGDK::Create(SessionMember->Xuid);
	if (SessionMember->IsCurrentUser)
	{
		bHosting = true;
		HostingPlayerNum = GetHostingPlayerNum(*HostNetId);
	}

	NamedSession->OwningUserId = HostNetId;
	NamedSession->bHosting = bHosting;
	NamedSession->HostingPlayerNum = HostingPlayerNum;

	UE_LOG_ONLINE_SESSION(Log, TEXT("Picking %" UINT64_FMT " as host for session"), SessionMember->Xuid);
}

const XblMultiplayerSessionMember* FOnlineSessionMpsdGDK::GetMemberFromDeviceToken(FGDKMultiplayerSessionHandle GDKSession, const FString& DeviceToken)
{
	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);
	if (SUCCEEDED(Result))
	{
		for (uint64 i = 0; i< NumMembers; ++i)
		{
			const XblMultiplayerSessionMember& Member = Members[i];
			if (FCString::Stricmp(UTF8_TO_TCHAR(Member.DeviceToken.Value), *DeviceToken) == 0)
			{
				return &Member;
			}
		}
	}

	return nullptr;
}

void FOnlineSessionMpsdGDK::RegisterVoice(const FUniqueNetId& PlayerId)
{
#if WITH_ENGINE
	IOnlineVoicePtr VoiceInt = GDKSubsystem->GetVoiceInterface();

	if (VoiceInt.IsValid())
	{
		if (!GDKSubsystem->IsLocalPlayer(PlayerId))
		{
			VoiceInt->RegisterRemoteTalker(PlayerId);
		}
		else
		{
			int32 LocalUserNum =
				GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(PlayerId);
			VoiceInt->RegisterLocalTalker(LocalUserNum);
		}
	}
#endif //WITH_ENGINE
}

void FOnlineSessionMpsdGDK::UnregisterVoice(const FUniqueNetId& PlayerId)
{
#if WITH_ENGINE
	IOnlineVoicePtr VoiceInt = GDKSubsystem->GetVoiceInterface();

	if (VoiceInt.IsValid())
	{
		if (!GDKSubsystem->IsLocalPlayer(PlayerId))
		{
			VoiceInt->UnregisterRemoteTalker(PlayerId);
		}
		else
		{
			const int32 LocalUserNum = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(PlayerId);
			VoiceInt->UnregisterLocalTalker(LocalUserNum);
		}
	}
#endif //WITH_ENGINE
}

FString FOnlineSessionMpsdGDK::GetLocalBase64Addr()
{
	return FString();
}

const TSharedRef<TArray<XblMultiplayerSessionMember>> FOnlineSessionMpsdGDK::GetMemberArray(FGDKMultiplayerSessionHandle Session)
{
	TSharedRef<TArray<XblMultiplayerSessionMember>> MemberArray = MakeShared<TArray<XblMultiplayerSessionMember>>();
	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	XblMultiplayerSessionMembers(Session, &Members, &NumMembers);
	if (NumMembers > 0)
	{
		MemberArray->Append(Members, NumMembers);
	}
	return MemberArray;
}

const XblMultiplayerSessionMember* FOnlineSessionMpsdGDK::GetGDKSessionHost(FGDKMultiplayerSessionHandle GDKSession)
{
	const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
	if (!SessionProperties)
	{
		return nullptr;
	}

	// WMM - Host token is always empty, because member tokens are always empty... why?
	FString HostToken = UTF8_TO_TCHAR(SessionProperties->HostDeviceToken.Value);
	//if (HostToken.IsEmpty())
	//{
	//	return nullptr;
	//}

	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);

	if (SUCCEEDED(Result))
	{
		for (uint64 i = 0; i < NumMembers; ++i)
		{
			const XblMultiplayerSessionMember& Member = Members[i];
			if (FString(UTF8_TO_TCHAR(Member.DeviceToken.Value)) == HostToken)
			{
				return &Member;
			}
		}
	}

	return nullptr;
}

FOnlineSessionSearchResult FOnlineSessionMpsdGDK::CreateSearchResultFromSearchHandle (FGDKMultiplayerSearchHandle SearchHandle, const FString& HostDisplayName, FGDKContextHandle GDKContext)
{
	FOnlineSessionSearchResult NewSearchResult;

	NewSearchResult.Session.SessionInfo = MakeShared<FOnlineSessionInfoMpsdGDK>(SearchHandle);
	
	NewSearchResult.Session.OwningUserName = UTF8_TO_TCHAR(StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NewSearchResult.Session.SessionInfo)->GetGDKMultiplayerSessionRef()->SessionName);

	ReadSettingsFromGDKSearchHandleJson(SearchHandle, NewSearchResult.Session, GDKContext);

	size_t MaxSlots = 0;
	size_t FilledSlots = 0;
	XblMultiplayerSearchHandleGetMemberCounts(SearchHandle, &MaxSlots, &FilledSlots);
	size_t OpenSlots = MaxSlots - FilledSlots;

	NewSearchResult.Session.NumOpenPrivateConnections = 0;
	NewSearchResult.Session.NumOpenPublicConnections = OpenSlots;
	NewSearchResult.Session.SessionSettings.NumPublicConnections = MaxSlots;


	XblMultiplayerSearchHandleGetSessionClosed(SearchHandle, &NewSearchResult.Session.SessionSettings.bAllowJoinInProgress);
	return NewSearchResult;
}


FOnlineSessionSearchResult FOnlineSessionMpsdGDK::CreateSearchResultFromSession(FGDKMultiplayerSessionHandle GDKSession, const FString& HostDisplayName, FGDKContextHandle GDKContext)
{
	FOnlineSessionSearchResult NewSearchResult;
	NewSearchResult.Session.SessionInfo = MakeShared<FOnlineSessionInfoMpsdGDK>(GDKSession);

	// Try to find the host.
	const XblMultiplayerSessionMember* Host = GetGDKSessionHost(GDKSession);

	if (!HostDisplayName.IsEmpty())
	{
		NewSearchResult.Session.OwningUserName = HostDisplayName;
	}
	else if (Host)
	{
		NewSearchResult.Session.OwningUserName = UTF8_TO_TCHAR(Host->Gamertag);
	}


	if (Host)
	{
		NewSearchResult.Session.OwningUserId = FUniqueNetIdGDK::Create(Host->Xuid);
	}
	else
	{
		const XblMultiplayerSessionReference* SessionReference = XblMultiplayerSessionSessionReference(GDKSession);
		if (ensure(SessionReference))
		{
			NewSearchResult.Session.OwningUserName = UTF8_TO_TCHAR(SessionReference->SessionName);
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::CreateSearchResultFromSession: Bad session handle, failed to get owning user name"));
		}
	}

	ReadSettingsFromGDKSessionJson(GDKSession, NewSearchResult.Session, GDKContext);

	GenerateSessionDebugInfo(GDKSession, [](const TCHAR* const Output)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), Output);
	});

	// Find number of open slots.
	int32 MaxSlots = 0;
	const XblMultiplayerSessionConstants* SessionConstants = XblMultiplayerSessionSessionConstants(GDKSession);
	if (ensure(SessionConstants))
	{
		MaxSlots = SessionConstants->MaxMembersInSession;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::CreateSearchResultFromSession: SessionConstants=nullptr, failed to get MaxSlots"));
	}

	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	XblMultiplayerSessionMembers(GDKSession, &Members, &NumMembers);
	int32 FilledSlots = NumMembers;
	int32 OpenSlots = MaxSlots - FilledSlots;

	NewSearchResult.Session.NumOpenPrivateConnections = 0;
	NewSearchResult.Session.NumOpenPublicConnections = OpenSlots;
	NewSearchResult.Session.SessionSettings.NumPublicConnections = MaxSlots;

	NewSearchResult.Session.SessionSettings.bAllowJoinInProgress = false;
	const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
	if (ensure(SessionProperties))
	{
		NewSearchResult.Session.SessionSettings.bAllowJoinInProgress = !SessionProperties->Closed;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::CreateSearchResultFromSession: SessionProperties=nullptr, failed to get bAllowJoinInProgress"));
	}

	return NewSearchResult;
}

void FOnlineSessionMpsdGDK::SaveSessionInvite(FGDKUserHandle AcceptingUser, const XblMultiplayerInviteHandle& SessionHandle)
{
	FString SessionHandleDataString = UTF8_TO_TCHAR(SessionHandle.Data);

	if (AcceptingUser.IsValid())
	{
		uint64 UserId;
		ensure(SUCCEEDED(XUserGetId(AcceptingUser, &UserId)));

		UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionMpsdGDK::SaveSessionInvite] AcceptingUser=[%lld] SessionHandle=[%s]"), UserId, *SessionHandleDataString);

		// Set the invite data on the game thread since that's where it will be consumed
		GDKSubsystem->ExecuteNextTick([this, AcceptingUser, SessionHandle]
			{
				PendingInvite = FPendingInviteData();
				PendingInvite.AcceptingUser = AcceptingUser;
				PendingInvite.SessionHandle = SessionHandle;
				PendingInvite.LoggedNotProcessedYetTime = FPlatformTime::Seconds() + SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS;
				PendingInvite.bHaveInvite = true;
			});
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpsdGDK::SaveSessionInvite] AcceptingUser invalid, session invite was not saved"));
	}
}

FString FindUrlParameter(FString Uri, const TCHAR* Name)
{
	FString StartToken = FString::Printf(TEXT("%s="), Name);
	int32 Start = Uri.Find(StartToken);
	if (Start == INDEX_NONE)
	{
		return FString();
	}

	Start += StartToken.Len();
	int32 End = Uri.Find(TEXT("&"), ESearchCase::IgnoreCase, ESearchDir::FromStart, Start);

	// If the session is at the end of the string then end will return not found.
	int32 CharCount = MAX_int32;
	if (End != INDEX_NONE)
	{
		CharCount = End - Start;
	}

	return(Uri.Mid(Start, CharCount));
}

void FOnlineSessionMpsdGDK::SaveInviteFromActivation(const FString& ActivationUri)
{
	// WMM - This is temporary until this is supported in the GDK API
	FString Handle = FindUrlParameter(ActivationUri, TEXT("handle"));
	if (Handle.IsEmpty())
	{
		UE_LOG_ONLINE(Warning, TEXT("[FOnlineSessionMpsdGDK::SaveInviteFromActivation] Can't find handle in the uri: %s"), *ActivationUri);
		return;
	}
	
	if (Handle.Len() >= XBL_GUID_LENGTH)
	{
		return;
	}
	XblMultiplayerInviteHandle InviteHandle;
	FMemory::Memzero(&InviteHandle, sizeof(InviteHandle));
	FCStringAnsi::Strncpy(InviteHandle.Data, TCHAR_TO_UTF8(*Handle), XBL_GUID_LENGTH);

	FString Invited = TEXT("invitedXuid");
	FString Joiner = TEXT("joinerXuid");

	FString XuidString = FindUrlParameter(ActivationUri, *Invited);
	if (XuidString.IsEmpty())
	{
		XuidString = FindUrlParameter(ActivationUri, *Joiner);
	}
	uint64_t Xuid = FCString::Atoi64(*XuidString);

	// Trying to retrieve the user immediately could cause errors, since GDK Users might still not be created
	GDKSubsystem->ExecuteNextTick([this, Xuid, InviteHandle]
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // XUserFindUserById is not safe to call on time-sensitive threads

			FGDKUserHandle GDKUser;
			XUserFindUserById(Xuid, GDKUser.GetInitReference());

			if (GDKUser.IsValid())
			{
				// If the user already exists (GDKUserHandle non null), we process it immediately
				SaveSessionInvite(GDKUser, InviteHandle);
			}
			else
			{
				// If the user doesn't exist, (a signed in but inactive user), we'll need to add it first
				FGDKLocalTaskBlock Block;
				HRESULT Result = XUserAddByIdWithUiAsync(Xuid, Block);
				if (SUCCEEDED(Result))
				{
					Result = Block.BlockUntilComplete();
					if (SUCCEEDED(Result))
					{
						XUserFindUserById(Xuid, GDKUser.GetInitReference());

						if (GDKUser.IsValid())
						{
							// With this we make sure the User Manager knows and tracks this new user
							IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(GDKUser);

							SaveSessionInvite(GDKUser, InviteHandle);
						}
					}
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("[FOnlineSessionMpsdGDK::SaveInviteFromActivation] XUserAddByIdWithUiAsync failed with code 0x%0.8X."), Result);
				}
			}
		});
}

bool FOnlineSessionMpsdGDK::IsConsoleHost(FGDKMultiplayerSessionHandle GDKSession)
{
	if (!GDKSession.IsValid())
	{
		return false;
	}

	const XblMultiplayerSessionMember* HostMember = GetGDKSessionHost(GDKSession);
	if (!HostMember)
	{
		return false;
	}

	const TSharedRef<TArray<XblMultiplayerSessionMember>> Members = GetMemberArray(GDKSession);

	for (const XblMultiplayerSessionMember& Member : *Members)
	{
		if (Member.IsCurrentUser && (FCStringAnsi::Strcmp(Member.DeviceToken.Value, HostMember->DeviceToken.Value) == 0))
		{
			return true;
		}
	}

	return false;
}

void FOnlineSessionMpsdGDK::Tick(float DeltaTime)
{
	TickPendingInvites(DeltaTime);
	TickPendingSessionUserInvites(DeltaTime);
}

void FOnlineSessionMpsdGDK::TickPendingSessionUserInvites(float DeltaTime)
{
	if (GDKSubsystem->GetSessionInterfaceGDK()->OnSessionUserInviteAcceptedDelegates.IsBound() && PendingSessionUserInvite.IsSet())
	{
		FGDKUserHandle AcceptingUser = GDKSubsystem->GetIdentityGDK()->GetUserForPlatformUserId(PendingSessionUserInvite->AcceptingUserIndex);

		FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Create(AcceptingUser);

		OnGetSessionForInviteComplete(PendingSessionUserInvite->AcceptingUserIndex, PendingSessionUserInvite->bWasSuccessful, *PendingSessionUserInvite->SearchResult, PendingSessionUserInvite->InviteHandle, UniqueNetId);

		PendingSessionUserInvite.Reset();
	}
}

void FOnlineSessionMpsdGDK::TickPendingInvites(float DeltaTime)
{
	if (!PendingInvite.bHaveInvite)
	{
		return;
	}

	// Warn if we haven't processed this in a timely manner
	if (!PendingInvite.bLoggedNotProcessedYet && PendingInvite.LoggedNotProcessedYetTime < FPlatformTime::Seconds())
	{
		PendingInvite.bLoggedNotProcessedYet = true;
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::TickPendingInvites: Haven't processed the invite in %d seconds. bIsDestroyingSessions=%s ConvertedNetworkConnectivityLevel=[%s]"), SESSION_INVITE_PROCESSING_LOG_TIMEOUT_SECONDS, *LexToString(bIsDestroyingSessions), EOnlineServerConnectionStatus::ToString(GDKSubsystem->ConvertedNetworkConnectivityLevel));
		// Intentional fall through
	}

	if (bIsDestroyingSessions)
	{
		// Don't accept invites while we're destroying all of our sessions
		return;
	}

	if (GDKSubsystem->ConvertedNetworkConnectivityLevel != EOnlineServerConnectionStatus::Connected)
	{
		// Don't process invites until we're fully connected
		return;
	}

	if (!PendingInvite.AcceptingUser.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::TickPendingInvites: bHaveInvite is true but AcceptingUser is null."));
		PendingInvite.bHaveInvite = false;
		return;
	}

	uint64 UserId;
	if (FAILED(XUserGetId(PendingInvite.AcceptingUser, &UserId)))
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::TickPendingInvites: bHaveInvite is true but AcceptingUser has no xuid."));
		PendingInvite.bHaveInvite = false;
		return;
	}

	const FPlatformUserId AcceptingUserId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromGDKUser(PendingInvite.AcceptingUser);
	if (AcceptingUserId == PLATFORMUSERID_NONE)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::TickPendingInvites: bHaveInvite is true but unknown player %" UINT64_FMT), UserId);
		PendingInvite.bHaveInvite = false;
		return;
	}

	FGDKContextHandle Context = GDKSubsystem->GetGDKContext(PendingInvite.AcceptingUser);
	if (!Context)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::TickPendingInvites: couldn't create an GDKContext for the AcceptingUser."));
		return;
	}


	XblMultiplayerInviteHandle SessionHandle = PendingInvite.SessionHandle;
	FUniqueNetIdGDKRef UniqueNetId = FUniqueNetIdGDK::Create(UserId);

	FPlatformUserId PlatformUserId = GDKSubsystem->GetIdentityGDK()->GetPlatformUserIdFromUniqueNetId(*UniqueNetId);
	const int32 LocalUserIndex = GDKSubsystem->GetIdentityGDK()->GetLocalUserNumFromPlatformUserId(PlatformUserId);
	
	bool bSkipPrivilegeCheckOnSessionJoin = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bSkipPrivilegeCheckOnSessionJoin"), bSkipPrivilegeCheckOnSessionJoin, GEngineIni);

	if (bSkipPrivilegeCheckOnSessionJoin)
	{	
		//if the privilege check gets skipped and a privilege is missing, FOnlineAsyncTaskGDKFindSessionById might silently fail
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKFindSessionById>(
			GDKSubsystem,
			Context,
			LocalUserIndex,
			SessionHandle,
			FOnSingleSessionResultCompleteDelegate::CreateThreadSafeSP(this, &FOnlineSessionMpsdGDK::OnGetSessionForInviteComplete, SessionHandle, UniqueNetId));
	}
	else
	{
		IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate Delegate = IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this, Context, SessionHandle, LocalUserIndex](const FUniqueNetId& LocalUserId, EUserPrivileges::Type Privilege, uint32 PrivilegeResult)
		{
			FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(LocalUserId);

			if (PrivilegeResult != static_cast<uint32>(IOnlineIdentity::EPrivilegeResults::NoFailures))
			{
				GDKSubsystem->ExecuteNextTick([this, LocalUserIndex, GDKUserId]()
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnGetSessionForInviteComplete_Delegate);
						GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(false, LocalUserIndex, GDKUserId, FOnlineSessionSearchResult());
					});
			}
			else
			{
				GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKFindSessionById>(
					GDKSubsystem,
					Context,
					LocalUserIndex, 
					SessionHandle,
					FOnSingleSessionResultCompleteDelegate::CreateThreadSafeSP(this, &FOnlineSessionMpsdGDK::OnGetSessionForInviteComplete, SessionHandle, GDKUserId));
			}
		});

		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGetUserPrivilege>(GDKSubsystem, Context, UniqueNetId, EUserPrivileges::CanPlayOnline, Delegate, EShowPrivilegeResolveUI::Show);
	}

	PendingInvite = FPendingInviteData();
}

void FOnlineSessionMpsdGDK::OnGetSessionForInviteComplete(int32 AcceptingUserIndex, bool bWasSuccessful, const FOnlineSessionSearchResult& SearchResult, XblMultiplayerInviteHandle InviteHandle, FUniqueNetIdGDKRef UniqueNetIdRef)
{
	// If the delegate is bound, we'll execute it, if not, we'll save the information to trigger it later at TickPendingSessionUserInvites
	if (GDKSubsystem->GetSessionInterfaceGDK()->OnSessionUserInviteAcceptedDelegates.IsBound())
	{
		if (!bWasSuccessful)
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("[FOnlineSessionMpsdGDK::OnGetSessionForInviteComplete] Task unsuccessful. Triggering delegates."));

			GDKSubsystem->ExecuteNextTick([this, AcceptingUserIndex, UniqueNetIdRef]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnGetSessionForInviteComplete_Delegate);
					GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(false, AcceptingUserIndex, UniqueNetIdRef, FOnlineSessionSearchResult());
				});
			return;
		}

		GDKSubsystem->ExecuteNextTick([this, AcceptingUserIndex, UniqueNetIdRef, InviteHandle, SearchResult]()
			{
				TSharedPtr<FOnlineSessionInfoMpsdGDK> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SearchResult.Session.SessionInfo);
				if (SessionInfo.IsValid())
				{
					SessionInfo->SetSessionInviteHandle(InviteHandle);
				}

				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnGetSessionForInviteComplete_Delegate);
				GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionUserInviteAcceptedDelegates(true, AcceptingUserIndex, UniqueNetIdRef, SearchResult);
			});
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("[FOnlineSessionMpsdGDK::OnGetSessionForInviteComplete] OnSessionUserInviteAcceptedDelegate not bound yet, saving info for a later attempt."));

		PendingSessionUserInvite = FPendingSessionUserInvite(AcceptingUserIndex, bWasSuccessful, MakeShared<FOnlineSessionSearchResult>(SearchResult), InviteHandle);
	}
}
	
void FOnlineSessionMpsdGDK::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef GDKPlayerId = FUniqueNetIdGDK::Cast(PlayerId);

	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*GDKPlayerId);
	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::UnknownError);
		});
		return;
	}

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::SessionDoesNotExist);
		});
		return;
	}

	FOnlineSessionInfoMpsdGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::UnknownError);
		});
		return;
	}

	FGDKMultiplayerSessionHandle GDKSession = GDKSessionInfo->GetGDKMultiplayerSession();
	if (!GDKSession)
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::UnknownError);
		});
		return;
	}

	// We want to prevent players that already are in the session from registering again
	const XblMultiplayerSessionMember* Members;
	uint64 MembersCount = 0;
	XblMultiplayerSessionMembers(GDKSession, &Members, &MembersCount);
	for (uint64 i = 0; i < MembersCount; i++)
	{
		if (*GDKPlayerId == Members[i].Xuid)
		{
			GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_RegisterLocalPlayer_Delegate);
				Delegate.ExecuteIfBound(*GDKPlayerId, EOnJoinSessionCompleteResult::AlreadyInSession);
			});
			return;
		}
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKRegisterLocalUser>(GDKSubsystem, GDKContext, SessionName, GDKPlayerId, GDKSession, GetSubscriptionType(NamedSession->SessionSettings), Delegate, TArray<const XblMultiplayerSessionMember*>());
}

void FOnlineSessionMpsdGDK::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	FUniqueNetIdGDKRef GDKPlayerId = FUniqueNetIdGDK::Cast(PlayerId);
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*GDKPlayerId);
	if (!GDKContext.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	FOnlineSessionInfoMpsdGDKPtr GDKSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	if (!GDKSessionInfo.IsValid())
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	FGDKMultiplayerSessionHandle GDKSession = GDKSessionInfo->GetGDKMultiplayerSession();
	if (!GDKSession)
	{
		GDKSubsystem->ExecuteNextTick([GDKPlayerId, Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_UnregisterLocalPlayer_Delegate);
			Delegate.ExecuteIfBound(*GDKPlayerId, false);
		});
		return;
	}

	GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKUnregisterLocalUser>(GDKSubsystem, GDKContext, SessionName, GDKSession, GDKPlayerId, Delegate);
}

bool FOnlineSessionMpsdGDK::CanUserJoinSession(FGDKUserHandle JoiningUser, FGDKMultiplayerSessionHandle GDKSession)
{
	if (!JoiningUser.IsValid() || !GDKSession.IsValid())
	{
		return false;
	}

	const TSharedRef<TArray<XblMultiplayerSessionMember>> Members = FOnlineSessionMpsdGDK::GetMemberArray(GDKSession);

	const XblMultiplayerSessionConstants* SessionConstants = XblMultiplayerSessionSessionConstants(GDKSession);
	check(SessionConstants);

	if (static_cast<uint32>(Members->Num()) < SessionConstants->MaxMembersInSession)
	{
		return true;
	}

	// Check for a reservation
	uint64 JoiningUserId;
	if (SUCCEEDED(XUserGetId(JoiningUser, &JoiningUserId)))
	{		
		for (const XblMultiplayerSessionMember& CurrentMember : *Members)
		{

			if (CurrentMember.Xuid == JoiningUserId)
			{
				return true;
			}
		}
	}

	return false;
}

XblMultiplayerSessionRestriction FOnlineSessionMpsdGDK::GetGDKSessionRestrictionFromSettings(const FOnlineSessionSettings& SessionSettings)
{
	if (SessionSettings.bShouldAdvertise)
	{
		// "None" restriction means anyone may interact with this session if there's room, this is the session default
		//WMM TODO: Fix this.. need to distinguish between the two, since currently windows cannot use this
		//return XblMultiplayerSessionRestriction::None;
		return XblMultiplayerSessionRestriction::Followed;
	}

	if (SessionSettings.bAllowJoinViaPresence)
	{
		// "Followed" restriction means anyone who follows a member of this session may interact with this session
		return XblMultiplayerSessionRestriction::Followed;
	}

	// "Local" restriction means only people who created the session, are on the same console as session members, or
	// those who have been invited may interact with this session
	return XblMultiplayerSessionRestriction::Local;
}

void FOnlineSessionMpsdGDK::OnSessionChanged(FName SessionName, XblMultiplayerSessionChangeTypes Diff)
{
	//Parameter Diff expected to be XblMultiplayerSessionChangeTypes::None for session changes made locally 
	
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpsdGDK::OnSessionChanged"));

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid() || !NamedSession->SessionInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Can't find platform session when received session changed event"));
		return;
	}

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	check(GDKInfo.IsValid());

	FGDKMultiplayerSessionHandle UpdatedGDKSession = GDKInfo->GetGDKMultiplayerSession();
	GenerateSessionDebugInfo(*NamedSession, [](const TCHAR* const Output)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), Output);
	});

	if ((Diff & XblMultiplayerSessionChangeTypes::InitializationStateChange) == XblMultiplayerSessionChangeTypes::InitializationStateChange)
	{
		OnInitializationStateChanged(SessionName);
	}

	const XblMultiplayerSessionMember* Members = nullptr;
	uint64 NumMembers = 0;
	HRESULT Result = XblMultiplayerSessionMembers(UpdatedGDKSession, &Members, &NumMembers);	

	if ((Diff & XblMultiplayerSessionChangeTypes::MemberListChange) == XblMultiplayerSessionChangeTypes::MemberListChange ||
		(Diff & XblMultiplayerSessionChangeTypes::MemberStatusChange) == XblMultiplayerSessionChangeTypes::MemberStatusChange ||
		(Diff == XblMultiplayerSessionChangeTypes::None && NamedSession->RegisteredPlayers.Num() != NumMembers))
	{
		OnMemberListChanged(UpdatedGDKSession, SessionName);
		
		TArray< FUniqueNetIdRef > RemovedMembers;
		TArray< FUniqueNetIdRef > AddedMembers;
		for (FUniqueNetIdRef Member : NamedSession->RegisteredPlayers)
		{
			bool bFound = false;
			if (ensure(Member->GetType() == GDK_SUBSYSTEM))
			{
				if (SUCCEEDED(Result))
				{
					for (int i = 0; i < NumMembers; ++i)
					{
						if (Members[i].Status == XblMultiplayerSessionMemberStatus::Active &&  StaticCastSharedRef<const FUniqueNetIdGDK>(Member)->ToUint64() == Members[i].Xuid)
						{
							bFound = true;
							break;
						}
					}
				}
				if (!bFound)
				{
					RemovedMembers.Add(Member);
				}
			}
		}

		if (SUCCEEDED(Result))
		{
			for (int i = 0; i < NumMembers; ++i)
			{
				if(Members[i].Status != XblMultiplayerSessionMemberStatus::Active)
				{
					continue;
				}
				bool bFound = false;
				for (FUniqueNetIdRef Member : NamedSession->RegisteredPlayers)
				{
					if (ensure(Member->GetType() == GDK_SUBSYSTEM))
					{
						if (StaticCastSharedRef<const FUniqueNetIdGDK>(Member)->ToUint64() == Members[i].Xuid)
						{
							bFound = true;
							break;
						}

					}

				}
				if (!bFound)
				{
					AddedMembers.Add(FUniqueNetIdGDK::Create(Members[i].Xuid));
				}
			}
		}

		for (FUniqueNetIdRef Member : RemovedMembers)
		{
			// This can also be called from UnregisterPlayers if that triggers first
			UE_LOG_ONLINE_SESSION(Log, TEXT("TriggerOnSessionParticipantLeftDelegates Player removed"));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnSessionChanged_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionParticipantLeftDelegates(SessionName, *Member, EOnSessionParticipantLeftReason::Left);
		}
		for (FUniqueNetIdRef Member : AddedMembers)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("TriggerOnSessionParticipantJoinedDelegates Player added"));
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnSessionChanged_Delegate);
			GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionParticipantJoinedDelegates(SessionName, *Member);
		}
	}

	if ((Diff & XblMultiplayerSessionChangeTypes::HostDeviceTokenChange) == XblMultiplayerSessionChangeTypes::HostDeviceTokenChange)
	{
		DetermineSessionHost(SessionName, UpdatedGDKSession);
	}

	if ((Diff & XblMultiplayerSessionChangeTypes::CustomPropertyChange) == XblMultiplayerSessionChangeTypes::CustomPropertyChange)
	{
		ReadSettingsFromGDKSessionJson(UpdatedGDKSession, *NamedSession, FGDKContextHandle());
	}

	if (((Diff & XblMultiplayerSessionChangeTypes::MemberCustomPropertyChange) == XblMultiplayerSessionChangeTypes::MemberCustomPropertyChange)
		|| ((Diff & XblMultiplayerSessionChangeTypes::MemberStatusChange) == XblMultiplayerSessionChangeTypes::MemberStatusChange))
	{
		UpdateMatchMembersJson(NamedSession->SessionSettings.Settings, UpdatedGDKSession);
		UpdateMatchMembers(NamedSession->SessionSettings, UpdatedGDKSession);
	}

	const XblMultiplayerSessionInfo* UpdatedSessionInfo = XblMultiplayerSessionGetInfo(UpdatedGDKSession);
	check(UpdatedSessionInfo);

	NamedSession->SessionSettings.Set(SETTING_CHANGE_NUMBER, static_cast<uint64>(UpdatedSessionInfo->ChangeNumber), EOnlineDataAdvertisementType::DontAdvertise);
}

void FOnlineSessionMpsdGDK::OnInitializationStateChanged(const FName& SessionName)
{
	check(IsInGameThread());

	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineSessionMpsdGDK::OnInitializationStateChanged - game thread"));

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::OnInitializationStateChanged - session doesn't exist or was destroyed before task ran"));
		return;
	}

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	if (!GDKInfo.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::OnInitializationStateChanged - session info failed to initialize"));
		return;
	}

	if (GDKInfo->IsSessionReady())
	{
		return;
	}

	// We only track initialization changes for the user associated with this session instance,
	// not other local users. The use of initialization groups in JoinSessionAsync ensures all
	// local users pass or fail QoS together.
	// Theoretically, this could be wrong if another local user joins the session via
	// matchmaking later on. I'm not sure this is something any game would actually try to
	// do, but it's something to be aware of.
	FGDKMultiplayerSessionHandle GDKSession = GDKInfo->GetGDKMultiplayerSession();
	const XblMultiplayerSessionMember* SessionMember = XblMultiplayerSessionCurrentUser(GDKSession);
	
	if (!SessionMember || !SessionMember->InitializeRequested)
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("  QoS not requested for this member, skipping"));
		FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKInfo->GetGDKMultiplayerSession());
		check(GDKContext);
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGameSessionReady>(GDKSubsystem, GDKContext, NamedSession->SessionName, GDKInfo->GetGDKMultiplayerSessionRef());
		return;
	}

	if (SessionMember->InitializationFailureCause != XblMultiplayerMeasurementFailure::None)
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("  Qos failed for this member, failure case: %u"), static_cast<uint32>(SessionMember->InitializationFailureCause));
		FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = GDKSubsystem->GetMatchmakingInterfaceGDK();
		MatchmakingInterface->SetTicketState(SessionName, EOnlineGDKMatchmakingState::None);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnInitializationStateChanged_Delegate);
		MatchmakingInterface->TriggerOnMatchmakingCompleteDelegates(NamedSession->SessionName, false);
		return;
	}

	if (SessionMember->InitializationEpisode == 0)
	{
		UE_LOG_ONLINE_SESSION(Log, TEXT("  QoS succeeded"));

		FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKInfo->GetGDKMultiplayerSession());
		check(GDKContext);
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKGameSessionReady>(GDKSubsystem, GDKContext, NamedSession->SessionName, GDKInfo->GetGDKMultiplayerSessionRef());
		return;
	}

	const XblMultiplayerSessionInitializationInfo* InitializationInfo = XblMultiplayerSessionGetInitializationInfo(GDKInfo->GetGDKMultiplayerSession());

	if (InitializationInfo && SessionMember->InitializationEpisode == InitializationInfo->Episode)
	{
		// This member is participating in QoS for this episode
		XblMultiplayerInitializationStage Stage = InitializationInfo->Stage;

		switch (Stage)
		{
			case XblMultiplayerInitializationStage::None:
				UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = None"));
				break;

			case XblMultiplayerInitializationStage::Unknown:
				UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = Unknown"));
				break;

			case XblMultiplayerInitializationStage::Joining:
				// Nothing to be done here, just wait for the other devices to finish joining the session
				UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = Joining"));
				break;

			case XblMultiplayerInitializationStage::Measuring:
			{
				// Title will measure and upload QoS result, service will do the evaluation.
				UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = Measuring"));

				FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKInfo->GetGDKMultiplayerSession());
				check(GDKContext);

				GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKMeasureAndUploadQos>(GDKSubsystem, GDKContext, NamedSession.ToSharedRef(), GDKInfo->GetGDKMultiplayerSession(), MAX_RETRIES, QOS_TIMEOUT_MILLISECONDS, QOS_PROBE_COUNT);
				break;
			}

			case XblMultiplayerInitializationStage::Evaluating:
				UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = Evaluating"));
				// @todo Currently the engine supports system-evaluated QoS. Code for title-evaluated
				// QoS would go here.
				break;

			case XblMultiplayerInitializationStage::Failed:
				{
					// QoS failed for the session overall
					UE_LOG_ONLINE_SESSION(Log, TEXT("  InitializationStage = Failed"));
					FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterface = GDKSubsystem->GetMatchmakingInterfaceGDK();
					MatchmakingInterface->SetTicketState(SessionName, EOnlineGDKMatchmakingState::None);
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnInitializationStateChanged_Delegate);
					MatchmakingInterface->TriggerOnMatchmakingCompleteDelegates(NamedSession->SessionName, false);
					break;
				}
			default:
				UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::OnInitializationStateChanged - Got unexpected InitializationStage: %u"), static_cast<uint32>(Stage));
				break;
		}
	}
}

void FOnlineSessionMpsdGDK::OnMemberListChanged(FGDKMultiplayerSessionHandle GDKSession, const FName& SessionName)
{
	check(IsInGameThread());

	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineSessionMpsdGDK::OnMemberListChanged - game thread"));

	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (!NamedSession.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::OnMatchmakingStatusChanged - session doesn't exist or was destroyed before task ran"));
		return;
	}

	UpdateMatchMembersJson(NamedSession->SessionSettings.Settings, GDKSession);
	UpdateMatchMembers(NamedSession->SessionSettings, GDKSession);

	bool AllowMigration = false;
	if (NamedSession->SessionSettings.Get(SETTING_ALLOW_ARBITER_MIGRATION, AllowMigration) && AllowMigration)
	{
		bool ShouldMigrate = true;

		const TSharedRef<TArray<XblMultiplayerSessionMember>> Members = FOnlineSessionMpsdGDK::GetMemberArray(GDKSession);
		const XblMultiplayerSessionProperties* SessionProperties = XblMultiplayerSessionSessionProperties(GDKSession);
		check(SessionProperties);

		for (const XblMultiplayerSessionMember& Member : *Members)
		{
			FString HostToken = UTF8_TO_TCHAR(SessionProperties->HostDeviceToken.Value);
			FString MemberToken = UTF8_TO_TCHAR(Member.DeviceToken.Value);
			if (HostToken == MemberToken)
			{
				ShouldMigrate = false;
				break;
			}
		}

		if (ShouldMigrate)
		{
			OnHostInvalid(SessionName);
		}
	}

	FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
	check(GDKInfo.IsValid());

	// @v2live Should this go inside the MatchmakingState check below?
	if (!NamedSession->OwningUserId.IsValid())
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpsdGDK::OnMemberListChanged: NamedSession->OwningUserId is not set, but the host should be handling this event."));
		return;
	}

	if (!GDKSubsystem->IsLocalPlayer(*NamedSession->OwningUserId))
	{
		return;
	}

	// Don't add players if the game doesn't support join in progress.
	// @todo: figure out how to support non-join-in-progress games!
	//   Idea: expose a function for the game to call that queries available players, reserves them, and pulls them
	//   this way the game has control and can pull players between rounds.
	//	 On the new multiplayer APIs, there are no parties, so this would likely require a second session to hold
	//	 waiting players.
	//   Idea 2: Maybe the engine can detect when the session switches to Pending. Maybe games want more control though
	if (!NamedSession->SessionSettings.bAllowJoinInProgress)
	{
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionMpsdGDK::OnMemberListChanged: Game is not join in progress, not resubmitting match ticket."));
		return;
	}

	// Cancel the current match ticket and re-advertise with the new number of open slots.
	// If the session isn't doing matchmaking, this is a no-op.
	GDKSubsystem->GetMatchmakingInterfaceGDK()->SubmitMatchingTicket(GDKInfo->GetGDKMultiplayerSession(), SessionName, true);
}

void FOnlineSessionMpsdGDK::OnHostInvalid(const FName& SessionName)
{
	FNamedOnlineSessionPtr NamedSession = GetNamedSessionPtr(SessionName);
	if (NamedSession.IsValid())
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		if (GDKInfo.IsValid())
		{
			FGDKMultiplayerSessionHandle GDKSession =  GDKInfo->GetGDKMultiplayerSession();
			if (FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKSession))
			{
				const XblMultiplayerSessionMember* CurrentUser = XblMultiplayerSessionCurrentUser(GDKSession);

				XblMultiplayerSessionSetHostDeviceToken(GDKSession, CurrentUser->DeviceToken);
				
				GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKSafeWriteSession>(GDKSubsystem, TEXT("FOnlineAsyncTaskGDKSafeWriteSession"), GDKContext, NamedSession->SessionName, GDKSession);
			}
		}
	}
}

void FOnlineSessionMpsdGDK::OnSessionNeedsInitialState(FName SessionName)
{
	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineSessionMpsdGDK::OnSessionNeedsInitialState"));

	// Sync with the MultiplayerSession's initialization state. Any other processing needed immediately
	// after creating/joining a session can be added here.
	OnInitializationStateChanged(SessionName);
}

#if !UE_BUILD_SHIPPING
static FAutoConsoleCommand CVarSimulateConnectionIdChanged(
	TEXT("Online.GDK.SimulateConnectionIdChanged"),
	TEXT("Simulate connection id changed flow.")
	TEXT("Usage: \"Online.GDK.SimulateConnectionIdChanged\""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) 
	{
			const IOnlineSubsystem* const Subsystem = IOnlineSubsystem::Get(GDK_SUBSYSTEM);
			if (Subsystem == nullptr)
			{
				UE_LOGF(LogConsoleResponse, Display, "Couldn't get GDK subsystem");
				return;
			}

			FOnlineSessionGDKPtr OnlineSessionGDK = StaticCastSharedPtr<FOnlineSessionGDK>(Subsystem->GetSessionInterface());
			OnlineSessionGDK->GetMpsdImpl()->OnMultiplayerConnectionIdChanged();
	})
);

static FAutoConsoleCommand CVarSimulateSubscriptionLost(
	TEXT("Online.GDK.SimulateSubscriptionLost"),
	TEXT("Simulate subscription lost flow.")
	TEXT("Usage: \"Online.GDK.SimulateSubscriptionLost\""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) 
	{
			const IOnlineSubsystem* const Subsystem = IOnlineSubsystem::Get(GDK_SUBSYSTEM);
			if (Subsystem == nullptr)
			{
				UE_LOGF(LogConsoleResponse, Display, "Couldn't get GDK subsystem");
				return;
			}

			FOnlineSessionGDKPtr OnlineSessionGDK = StaticCastSharedPtr<FOnlineSessionGDK>(Subsystem->GetSessionInterface());
			OnlineSessionGDK->GetMpsdImpl()->OnMultiplayerSubscriptionsLost();
	})
);
#endif

void FOnlineSessionMpsdGDK::OnMultiplayerConnectionIdChanged()
{
	FScopeLock Lock(&SessionLock);
	for (const FNamedOnlineSessionRef& CurrentSession : Sessions)
	{
		if (FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(CurrentSession->SessionInfo))
		{
			FGDKMultiplayerSessionHandle GDKMultiplayerSessionHandle = GDKInfo->GetLastDiffedMultiplayerSession();
			// As described at https://docs.microsoft.com/en-us/gaming/gdk/_content/gc/live/features/multiplayer/mpsd/live-mpsd-overview#mpsd-change-notification-handling-and-disconnect-detection
			// XblMultiplayerSessionCurrentUserSetStatus + XblMultiplayerWriteSessionAsync will set the new connection id to the MPSD session after reconnecting to RTA
			bool bIsActive = true;
			SetCurrentUserActive(GDKMultiplayerSessionHandle, bIsActive);
			
			if (FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(GDKMultiplayerSessionHandle))
			{
				GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKSafeWriteSession>(GDKSubsystem, TEXT("FOnlineAsyncTaskGDKSafeWriteSession"), GDKContext, CurrentSession->SessionName, GDKMultiplayerSessionHandle);
			}
		}
	}
}

/** Detect a loss of connection to the subscription service and exit multiplayer. */
void FOnlineSessionMpsdGDK::OnMultiplayerSubscriptionsLost()
{
	if (!bHandleXblSubscriptionLost)
	{
		return;
	}

	check(IsInGameThread());

	UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineSessionMpsdGDK::OnMultiplayerSubscriptionsLost - game thread"));
	UE_LOG_ONLINE_SESSION(Log, TEXT("  Connection to multiplayer service lost. Destroying session objects."));

	// We were automatically removed from any GDK sessions, so clean them up.
	if (!bIsDestroyingSessions)
	{
		FScopeLock Lock(&SessionLock);
		if (Sessions.Num() > 0)
		{
			bIsDestroyingSessions = true;	// if multiple users lose subscriptions simultaneously, only try to destroy once

			OnSubscriptionLostDestroyCompleteDelegateHandle = GDKSubsystem->GetSessionInterfaceGDK()->AddOnDestroySessionCompleteDelegate_Handle(OnSubscriptionLostDestroyCompleteDelegate);

			TArray<FNamedOnlineSessionRef> SessionsCopy = Sessions;
			for (const FNamedOnlineSessionRef& CurrentSession : SessionsCopy)
			{
				DestroySession(CurrentSession->SessionName);
			}
		}
	}

	if (!bIsDestroyingSessions)
	{
		GDKSubsystem->RecreateGDKContextOnSubscriptionLost();
	}
}

void FOnlineSessionMpsdGDK::OnSubscriptionLostDestroyComplete(FName SessionName, bool bWasSuccessful)
{
	FScopeLock Lock(&SessionLock);
	if (Sessions.Num() == 0 || !bWasSuccessful)
	{
		if (Sessions.Num() == 0)
		{
			UE_LOG_ONLINE_SESSION(Log, TEXT("FOnlineSessionMpsdGDK::OnSubscriptionLostDestroyComplete - all sessions destroyed."));
		}
		else if (!bWasSuccessful)
		{
			// @v2live: We currently give up when this occurs. Is this right?
			UE_LOG_ONLINE_SESSION(Warning, TEXT("FOnlineSessionMpsdGDK::OnSubscriptionLostDestroyComplete - couldn't destroy session %s."), *SessionName.ToString());
		}

		GDKSubsystem->GetSessionInterfaceGDK()->ClearOnDestroySessionCompleteDelegate_Handle(OnSubscriptionLostDestroyCompleteDelegateHandle);
		bIsDestroyingSessions = false;

		// Inform the game of the subscription failure
		// We don't know the original NetId here, but it shouldn't matter since sessions were destroyed for everyone
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSessionMpsdGDK_OnSubscriptionLostDestroyComplete_Delegate);
		GDKSubsystem->GetSessionInterfaceGDK()->TriggerOnSessionFailureDelegates(*FUniqueNetIdGDK::EmptyId(), ESessionFailure::ServiceConnectionLost);
	}
}

FNamedOnlineSession* FOnlineSessionMpsdGDK::GetNamedSessionForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef)
{
	check(IsInGameThread());

	FScopeLock Lock(&SessionLock);

	for (FNamedOnlineSessionRef& CurrentSession : Sessions)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(CurrentSession->SessionInfo);
		if (!GDKInfo.IsValid())
		{
			continue;
		}

		if (GDKInfo->GetGDKMultiplayerSessionRef() &&
			FOnlineSubsystemGDK::AreSessionReferencesEqual(GDKInfo->GetGDKMultiplayerSessionRef(), GDKSessionRef))
		{
			return &CurrentSession.Get();
		}
	}

	return nullptr;
}

TOptional<FName> FOnlineSessionMpsdGDK::GetNamedSessionNameForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef)
{
	FScopeLock Lock(&SessionLock);

	for (const FNamedOnlineSessionRef& CurrentSession : Sessions)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(CurrentSession->SessionInfo);
		if (!GDKInfo.IsValid())
		{
			continue;
		}

		if (XblMultiplayerSessionReferenceIsValid(GDKInfo->GetGDKMultiplayerSessionRef()) &&
			FOnlineSubsystemGDK::AreSessionReferencesEqual(GDKInfo->GetGDKMultiplayerSessionRef(), GDKSessionRef))
		{
			return CurrentSession->SessionName;
		}
	}

	return TOptional<FName>();
}

TOptional<FName> FOnlineSessionMpsdGDK::GetNamedSessionNameForGDKSessionHandle(const FGDKMultiplayerSessionHandle& GDKSession)
{
	FScopeLock Lock(&SessionLock);

	for (const FNamedOnlineSessionRef& CurrentSession : Sessions)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(CurrentSession->SessionInfo);
		if (!GDKInfo.IsValid())
		{
			continue;
		}

		if (GDKInfo->GetGDKMultiplayerSession() == GDKSession)
		{
			return CurrentSession->SessionName;
		}
	}

	return TOptional<FName>();
}

FOnlineSessionInfoMpsdGDKPtr FOnlineSessionMpsdGDK::GetSessionInfoForGDKSessionRef(const XblMultiplayerSessionReference& GDKSessionRef)
{
	FScopeLock Lock(&SessionLock);

	for (auto& CurrentSession : Sessions)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(CurrentSession->SessionInfo);
		if (!GDKInfo.IsValid())
		{
			continue;
		}

		if (XblMultiplayerSessionReferenceIsValid(GDKInfo->GetGDKMultiplayerSessionRef()) &&
			FOnlineSubsystemGDK::AreSessionReferencesEqual(GDKInfo->GetGDKMultiplayerSessionRef(), &GDKSessionRef))
		{
			return GDKInfo;
		}
	}

	return nullptr;
}

TSharedPtr<FInternetAddr> FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64(const FString& DeviceAddressBase64)
{
	FString IPAddr;
	bool bDecoded = FBase64::Decode(DeviceAddressBase64, IPAddr);

	UE_CLOG_ONLINE_SESSION(!bDecoded, Warning, TEXT("FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64 Decode failed"));
	if (bDecoded)
	{
		FString Addr, Port;
		bool bSplit = IPAddr.Split(TEXT(":"), &Addr, &Port, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

		UE_CLOG_ONLINE_SESSION(!bSplit, Warning, TEXT("FOnlineSessionMpsdGDK::GetAddrFromSecureDeviceAddressBase64 Split failed"));
		if (bSplit)
		{
			bool Valid = true;

			TSharedRef<FInternetAddr> HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
			HostAddr->SetIp(*Addr, Valid);
			HostAddr->SetPort(FCString::Atoi(*Port));

			check(Valid);
			if (Valid)
			{
				return HostAddr;
			}
		}
	}
	return TSharedPtr<FInternetAddr>();
}
#endif //WITH_GRDK