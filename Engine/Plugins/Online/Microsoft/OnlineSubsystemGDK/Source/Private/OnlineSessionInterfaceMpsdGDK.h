// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDKPackage.h"
#include "SessionMessageRouter.h"
#include "OnlineSessionSettings.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCreateSession.h"
#include "AsyncTasks/OnlineAsyncTaskGDKCreateSearchHandle.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/multiplayer_c.h>
#include <xsapi-c/profile_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#include "OnlineError.h"
#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.session"

using FOnlineGDKActivitiesResultMap = TUniqueNetIdMap<FUniqueNetIdPtr /*JoinableSessionId*/>;

namespace OnlineSessionGDK
{
#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError HResultError(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), InCode)); }
	}
}

#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

class FInternetAddr;
class FUniqueNetIdGDK;

class FOnlineSessionMpsdGDK : public TSharedFromThis<FOnlineSessionMpsdGDK, ESPMode::ThreadSafe>
{
private:
	/** Reference to the main GDK subsystem */
	class FOnlineSubsystemGDK* GDKSubsystem;

	/** Performs common constructor operations. */
	void Initialize();

	/**
	 * Tick invites captured from launch URI
	 * Waits for a delegate to be listening before triggering
	 */
	void TickPendingInvites(float DeltaTime);

	/*
	* Waits for OnSessionUserInviteAcceptedDelegate to be bound before triggering, if there is data saved 
	*/
	void TickPendingSessionUserInvites(float DeltaTime);

	/** Handle initialization flow during matchmaking */
	void OnInitializationStateChanged(const FName& SessionName);

	/** Handle players being added or removed during matchmaking */
	void OnMemberListChanged(FGDKMultiplayerSessionHandle LiveSession, const FName& SessionName);

	/** Read and react to any initial state from the MultiplayerSession after creating/joining */
	void OnSessionNeedsInitialState(FName SessionName);

	void OnSubscriptionLostDestroyComplete(FName SessionName, bool bWasSuccessful);

	//WMM - These events are not needed in GDK. WHen registering the Invite handler, any pending invites will automatically be triggered.
	/** Responds to protocol activations to determine if an invite was accepted, and joins the session if so. */
	//void OnActivated(Windows::ApplicationModel::Activation::IActivatedEventArgs^ EventArgs);

	/** Token to track activation events, for invites that occur after launch */
	//XTaskQueueRegistrationToken ActivatedToken;

	/**
	 * Turns a session handle into a Session, then into an FOnlineSessionSearchResult, then triggers
	 * the OnSessionInviteAccepted delegates.
	 */
	void SaveSessionInvite(
		FGDKUserHandle AcceptingUser,
		const XblMultiplayerInviteHandle& SessionHandle );

	/** This task calls AddNamedSession so it needs access. */
	friend class FOnlineAsyncTaskGDKCreateSession;

	/**
	 * Registers and updates voice data for the given player id
	 *
	 * @param PlayerId player to register with the voice subsystem
	 */
	void RegisterVoice(const FUniqueNetId& PlayerId);

	/**
	 * Unregisters a given player id from the voice subsystem
	 *
	 * @param PlayerId player to unregister with the voice subsystem
	 */
	void UnregisterVoice(const FUniqueNetId& PlayerId);

	/** Called when signin is complete */
	void CleanUpOrphanedSessions(FGDKUserHandle User) const;

	//WMM THis should be done through XUserRegisterForChangeEvent
	/** Token for un-registering the signin callback */
	//Windows::Foundation::EventRegistrationToken SignInCompletedToken;

	/** Any join/invite from a protocol activation */
	struct FPendingInviteData
	{
		/** Whether there is a pending invite or not */
		bool bHaveInvite = false;

		/** Cached pointer to the user who accepted the invite */
		FGDKUserHandle AcceptingUser;

		/** Cached handle to the session to join */
		XblMultiplayerInviteHandle SessionHandle;

		/** Have we logged that we haven't processed this in a timely manner? */
		bool bLoggedNotProcessedYet = false;

		/** Time to log that we haven't processed this yet */
		double LoggedNotProcessedYetTime = 0.0;
	};

	/** Contains information about a join/invite parsed from the protocol activation */
	FPendingInviteData PendingInvite;

	/*
	* We'll use this to store the result of a session search caused by a user invite until the corresponding delegates have been bound
	*/
	struct FPendingSessionUserInvite
	{
		FPendingSessionUserInvite(
			int32 InAcceptingUserIndex,
			bool InWasSuccessful,
			TSharedRef<FOnlineSessionSearchResult> InSearchResult,
			XblMultiplayerInviteHandle InInviteHandle);

		int32 AcceptingUserIndex;
		bool bWasSuccessful;
		TSharedRef<FOnlineSessionSearchResult> SearchResult;
		XblMultiplayerInviteHandle InviteHandle;
	};

	TOptional<FPendingSessionUserInvite> PendingSessionUserInvite;

	/** Starts QoS against the search results */
	//void PingResultsAndTriggerDelegates(const TSharedRef<FOnlineSessionSearch>& SearchSettings);

	/** Get LocalPlayerNum/ControllerId for a Host's netid and -1 on failure */
	int32 GetHostingPlayerNum(const FUniqueNetId& HostNetId) const;

PACKAGE_SCOPE:

	static const int QOS_TIMEOUT_MILLISECONDS = 5000;
	static const int QOS_PROBE_COUNT = 8;

	/**
	 * Constructor.
	 *
	 * @param InSubsystem Pointer to the subsystem that created this object.
	 */
	FOnlineSessionMpsdGDK(class FOnlineSubsystemGDK* InSubsystem);

	/**
	 * Parses a protocol activation URI. If it was from an accepted invite, begins the invite accepted flow.
	 * @param ActivationUri the protocol activation URI
	 */
	void SaveInviteFromActivation(const FString& ActivationUri);

	/**
	 * Session tick for various background tasks
	 */
	void Tick(float DeltaTime);

	/**
	 * Read from GDK session document into UE settings
	 *
	 * @param GDKSession the session object to read settings from
	 * @param SessionSettings the settings object to write to
	 */
	void ReadSettingsFromGDKSessionJson(FGDKMultiplayerSessionHandle GDKSession, FOnlineSession& Session, FGDKContextHandle GDKContext = FGDKContextHandle());

	/**
	 * Read from GDK session search handle into UE settings
	 *
	 * @param GDKSession the session object to read settings from
	 * @param SessionSettings the settings object to write to
	 */
	void ReadSettingsFromGDKSearchHandleJson(FGDKMultiplayerSearchHandle SearcHandle, FOnlineSession& Session, FGDKContextHandle GDKContext = FGDKContextHandle());
	void ReadSettingsFromJson(FString& Json, FOnlineSession& Session, FSessionSettings& NewSettings, FGDKContextHandle GDKContext = FGDKContextHandle());
	/**
	 * Write from UE settings into Live document (note that this does not write the session back to Live; caller should)
	 *
	 * @param UpdatedSessionSettings the settings object to read from
	 * @param LiveSession the session object to write to
	 * @param HostUser the user to mark as the intended host (optional)
	 * @param GDKSubsystem the subsystem to use for any necessary operations
	 * @return True if the session settings/host were different than the current LiveSession object
	 */
	static bool WriteSettingsToGDKJson(const FOnlineSessionSettings& UpdatedSessionSettings, FGDKMultiplayerSessionHandle LiveSession, FGDKUserHandle HostUser, FOnlineSubsystemGDK* GDKSubsystem);

	/**
	 * Extract one FString that contains all the members settings
	 *
	 * @param LiveSession the session object to read from
	 * @param OutJsonString the string to write to
	 */
	static void ExtractJsonMemberSettings(FGDKMultiplayerSessionHandle GDKSession, FString& OutJsonString);

	/** Replaces the JSON member list in UpdatedSettings with the latest from MPSDSession */
	static void UpdateMatchMembersJson(FSessionSettings& UpdatedSettings, FGDKMultiplayerSessionHandle GDKSession);

	/** Replaces the member list in UpdatedSettings with the latest from MPSDSession */
	static void UpdateMatchMembers(FOnlineSessionSettings& UpdatedSettings, FGDKMultiplayerSessionHandle GDKSession);

	/** Mark the user as active within the session */
	void SetCurrentUserActive(FGDKMultiplayerSessionHandle GDKSession, bool bIsActive);

	/** Look up the NamedSession corresponding to a GDK session reference */
	FNamedOnlineSession* GetNamedSessionForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef);

	/** Look up the Name of a session corresponding to a GDK session reference */
	TOptional<FName> GetNamedSessionNameForGDKSessionRef(const XblMultiplayerSessionReference* GDKSessionRef);

	/** Look up the Name of a session corresponding to a GDK session handle */
	TOptional<FName> GetNamedSessionNameForGDKSessionHandle(const FGDKMultiplayerSessionHandle& GDKSession);

	/** Look up the SessionInfo corresponding to a GDK session reference */
	FOnlineSessionInfoMpsdGDKPtr GetSessionInfoForGDKSessionRef(const XblMultiplayerSessionReference& GDKSessionRef);

	/* Set me as the host using a host device token, does nothing if token already set. Synchronous, so only use from lambda */
	FGDKMultiplayerSessionHandle SetHostDeviceTokenSynchronous(int32 UserNum, FName SessionName, FGDKMultiplayerSessionHandle GDKSession,
		FGDKContextHandle Context);

	/** Get my info from a session */
	static const XblMultiplayerSessionMember* GetCurrentUserFromSession(FGDKMultiplayerSessionHandle GDKSession);

	/** work out who we want as host and set it in the named session */
	void DetermineSessionHost(FName Session, FGDKMultiplayerSessionHandle GDKSession);

	/** Get member from device token */
	const XblMultiplayerSessionMember* GetMemberFromDeviceToken(FGDKMultiplayerSessionHandle GDKSession, const FString& DeviceToken);

	/** Turns a base64 secure device address into an FInternetAddr */
	static TSharedPtr<FInternetAddr> GetAddrFromSecureDeviceAddressBase64(const FString& DeviceAddressBase64);

	/** Gets the local ip address and returns as a base64 encoded string. */
	static FString GetLocalBase64Addr();

	/** Uses data from a MultiplayerSession to initialize an FOnlineSessionSearchResult. */
	FOnlineSessionSearchResult CreateSearchResultFromSession(
		FGDKMultiplayerSessionHandle GDKSession,
		const FString& HostDisplayName,
		FGDKContextHandle GDKContext = FGDKContextHandle());

	/** Uses data from a MultiplayerSession Search handle to initialize an FOnlineSessionSearchResult. */
	FOnlineSessionSearchResult CreateSearchResultFromSearchHandle(
		FGDKMultiplayerSearchHandle SearchHandle,
		const FString& HostDisplayName,
		FGDKContextHandle GDKContext = FGDKContextHandle());

	/** Returns the host of a session, or null if there is no host. */
	static const XblMultiplayerSessionMember* GetGDKSessionHost(
		FGDKMultiplayerSessionHandle GDKSession );

	/** Returns true if the local console is the host of the session */
	static bool IsConsoleHost( FGDKMultiplayerSessionHandle GDKSession );

	static FString SessionReferenceToUri(const XblMultiplayerSessionReference& MultiplayerReference);
	//https://{authority}/serviceconfigs/{service-config-id}/sessiontemplates/{session-template-name}/sessions/{session-name}

	/** Returns true if the session has either an open slot or a slot reserved for the user. */
	static bool CanUserJoinSession(
		FGDKUserHandle JoiningUser,
		FGDKMultiplayerSessionHandle GDKSession);

	/** Returns what the Multiplayer Session restriction should be based on session settings */
	static XblMultiplayerSessionRestriction GetLiveSessionRestrictionFromSettings(const FOnlineSessionSettings& SessionSettings);

	/** Returns true if this session allows invites and join in presence */
	static bool AreInvitesAndJoinViaPresenceAllowed(const FOnlineSessionSettings& OnlineSessionSettings);

	/** Determine the SessionChangeType for the specified settings */
	static XblMultiplayerSessionChangeTypes GetSubscriptionType(const FOnlineSessionSettings& SessionSettings);

	/** Update to new connection id for sessions */
	void OnMultiplayerConnectionIdChanged();

	/** On subscription loss, fire appropriate delegate after cleaning up the sessions. */
	void OnMultiplayerSubscriptionsLost();

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current search object */
	TSharedPtr<FOnlineSessionSearch> CurrentSessionSearch;

	/** Current session settings */
	TArray<FNamedOnlineSessionRef> Sessions;

	FCriticalSection SessionResultLock;
	int ExpectedResults;
	// Synchronization for session changes handlers and destruction on subscription loss
	bool bIsDestroyingSessions;
	FOnEndSessionCompleteDelegate OnSubscriptionLostDestroyCompleteDelegate;
	FDelegateHandle OnSubscriptionLostDestroyCompleteDelegateHandle;

PACKAGE_SCOPE:
	// IOnlineSession interface
	FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session);
	FNamedOnlineSessionRef AddNamedSessionRef(FName SessionName, const FOnlineSession& Session);

	// These classes need to access AddNamedSession
	friend class FOnlineMatchmakingInterfaceGDK;

public:
	virtual ~FOnlineSessionMpsdGDK();

	// IOnlineSession interface
	FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr);
	FNamedOnlineSession* GetNamedSession(FName SessionName);
	FNamedOnlineSessionPtr GetNamedSessionPtr(FName SessionName) const;
	void RemoveNamedSession(FName SessionName);
	bool HasPresenceSession() const;
	EOnlineSessionState::Type GetSessionState(FName SessionName) const;
	bool CreateSession(int32 HostingPlayerControllerIndex, FName SessionName, const FOnlineSessionSettings& NewSessionSettings);
	bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings);
	bool StartSession(FName SessionName);
	bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = false);
	bool EndSession(FName SessionName);
	bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate());
	bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId);
	bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName);
	bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName);
	bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings);
	bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate);
	bool CancelFindSessions();
	bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) { return false; }
	bool JoinSession(int32 ControllerIndex, FName SessionName, const FOnlineSessionSearchResult& DesiredSession);
	bool JoinSession(const FUniqueNetId& UserId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession);
	bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend);
	bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend);
	bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList);
	bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend);
	bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend);
	bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends);
	bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends);
	bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType);
	bool GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo);
	FOnlineSessionSettings* GetSessionSettings(FName SessionName);
	bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited);
	bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false);
	bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId);
	bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players);
	void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate);
	void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate);
	int32 GetNumSessions() { return Sessions.Num(); }
	void DumpSessionState() {}

	void CreateSearchHandle(const FOnlineSessionSettings& SessionSettings, FGDKMultiplayerSessionHandle GDKSession, FGDKContextHandle GDKContext, const FOnCreateSearchHandleCompleteDelegate& Delegate);

	// Use task to get session in response to notification
	void OnSessionChanged(FName SessionName, XblMultiplayerSessionChangeTypes Diff);

	void OnHostInvalid(const FName& SessionName);

	/** Handle sending a session invite to friends */
	bool SendSessionInviteToFriends_Internal(FGDKContextHandle GDKContext,
		FName SessionName,
		const TArray<uint64>& FriendXuids);

	/** Set if this user should have an active multiplayer session or not (nullptr for SessionReference) */
	void SetUserActiveSessionActivity(const FUniqueNetIdGDK& PlayerId, FGDKMultiplayerSessionHandle GDKSession);

	bool StartCreateSession(
		const int32 UserIndex,
		const FOnlineSessionSettings& SessionSettings,
		const FString& Keyword, const FString& SessionTemplateName, FName SessionName,
		FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate);
	
	bool StartCreateSession(
		const FUniqueNetId& UserId,
		const FOnlineSessionSettings& SessionSettings,
		const FString& Keyword, const FString& SessionTemplateName, FName SessionName,
		FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate);

	bool InternalStartCreateSession(
		FGDKContextHandle GDKContext,
		const FOnlineSessionSettings& SessionSettings,
		const FString& Keyword, const FString& SessionTemplateName, FName SessionName,
		FOnlineAsyncTaskGDKCreateSession::FOnGDKCreateSessionComplete Delegate);

	static const TSharedRef<TArray<XblMultiplayerSessionMember>> GetMemberArray(FGDKMultiplayerSessionHandle Session);
		
	/** Sets host on a newly created session */
	bool SetHostOnCreatedSession(FGDKMultiplayerSessionHandle GDKSession, uint64 CreatingUserId);

	void WriteDTLSCertificatesToService(const TMap<FGuid, TArray<uint8>>& DTLSCertificateDictionary);

	void OnQueryActivitiesComplete(const FOnlineError& ErrorResult, const FOnlineGDKActivitiesResultMap& Results, int32 LocalUserNum, FGDKContextHandle GDKContext);
	void OnQueryFriendSessionDetailsComplete(int32 LocalUserNum, bool bSucceeded, const FOnlineSessionSearchResult& SearchResult, FGDKContextHandle GDKContext);
	static XblMultiplayerSessionRestriction GetGDKSessionRestrictionFromSettings(const FOnlineSessionSettings& SessionSettings);
	void OnGetSessionForInviteComplete(int32 LocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& SearchResult, XblMultiplayerInviteHandle InviteHandle, FUniqueNetIdGDKRef UniqueNetId);
	FNamedOnlineSession* GetNamedSessionForGDKSessionRef(XblMultiplayerSessionReference* GDKSessionRef);
PACKAGE_SCOPE:
	/** Handle setting our cache state, and potentially starting a queued multiplayer activity async task */
	void OnSetUserActiveSessionActivityComplete(const FUniqueNetIdGDK& PlayerId, const bool bUserInActiveSession);
	/** Clear when finished running queued multiplayer activity async tasks */
	void ClearSessionActivityInProgress() { bHasSessionActivityInProgress = false; }

private:
	/** Actually starts a set/clear multiplayer activity async task*/
	void SetUserActiveSessionActivity_Impl(const FUniqueNetIdGDK& PlayerId, const bool bWantsToBeInActiveSession, FGDKMultiplayerSessionHandle GDKSession);

PACKAGE_SCOPE:
	FDelegateHandle OnSubscriptionLostDelegateHandle;

	// Initialize session state after create/join
	FOnSessionNeedsInitialStateDelegate OnSessionNeedsInitialStateDelegate;

	FOnSessionChangedDelegate OnSessionChangedDelegate;
	FDelegateHandle OnSubscriptionsLostHandle;
	FDelegateHandle OnConnectionIdChangedHandle;

	/** Event to be called when our session updates */
	FString SessionUpdateEventName;
	
	/** Whether only the host can update the session or if anyone can */
	bool bOnlyHostUpdateSession;

	/** Whether to handle the xbl subscription lost */
	bool bHandleXblSubscriptionLost;

	/** Do we have a Session Activity Set or Clear in progress? */
	bool bHasSessionActivityInProgress;
	/** Queued Set/Clear multiplayer session action information */
	TUniqueNetIdMap<TOptional<const FGDKMultiplayerSessionHandle> > QueuedActiveSessionActivities;
	/** List of users who are currently in active sessions */
	FUniqueNetIdSet UsersWithActiveSessionActivities;
};

typedef TSharedPtr<FOnlineSessionMpsdGDK, ESPMode::ThreadSafe> FOnlineSessionMpsdGDKPtr;
