// Copyright Epic Games, Inc. All Rights Reserved.

//TODO WMM - Stats does not look to be complete for GDK
#if WITH_GRDK
#include "OnlineStatsInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKStatsUpdate.h"
#include "AsyncTasks/OnlineAsyncTaskGDKStatsQuery.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.stats"

namespace OnlineStatsGDKHelpers
{
	//template helpers for getting the value of an online stat update
	template<typename T>
	T GetStatUpdateValue(const FOnlineStatUpdate& StatUpdate);

	template<>
	double GetStatUpdateValue<double>(const FOnlineStatUpdate& StatUpdate)
	{
		switch (StatUpdate.GetType())
		{
		case EOnlineKeyValuePairDataType::Float:
		{
			float Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}

		case EOnlineKeyValuePairDataType::Double:
		{
			double Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}
		}

		return 0.0;
	}

	template<>
	int64 GetStatUpdateValue<int64>(const FOnlineStatUpdate& StatUpdate)
	{
		switch (StatUpdate.GetType())
		{
		case EOnlineKeyValuePairDataType::Int32:
		{
			int32 Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}

		case EOnlineKeyValuePairDataType::Int64:
		{
			int64 Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}
		}

		return 0;
	}

	template<>
	uint64 GetStatUpdateValue<uint64>(const FOnlineStatUpdate& StatUpdate)
	{
		switch (StatUpdate.GetType())
		{
		case EOnlineKeyValuePairDataType::UInt32:
		{
			uint32 Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}

		case EOnlineKeyValuePairDataType::UInt64:
		{
			uint64 Result;
			StatUpdate.GetValue().GetValue(Result);
			return Result;
		}
		}

		return 0;
	}

	template<>
	FString GetStatUpdateValue<FString>(const FOnlineStatUpdate& StatUpdate)
	{
		FString Result;
		if (StatUpdate.GetType() == EOnlineKeyValuePairDataType::String)
		{
			StatUpdate.GetValue().GetValue(Result);
		}
		return Result;
	}
}

/** Online Stats Cache */

TArray<FUniqueNetIdRef> FOnlineGDKStatsCache::GetUsers()
{
	TArray<FUniqueNetIdRef> Result;

	CachedUserStats.GetKeys(Result);

	return Result;
}

const FOnlineStatsUserStats* FOnlineGDKStatsCache::GetStatsForUser(const FUniqueNetIdRef UserId) const
{
	const FOnlineStatsUserStats* Result = CachedUserStats.Find(UserId);

	return Result;
}

void FOnlineGDKStatsCache::AddUser(const FUniqueNetIdRef UserId)
{
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (!UserStats)
	{
		CachedUserStats.Add(UserId, FOnlineStatsUserStats(UserId));
	}
}

void FOnlineGDKStatsCache::AddStatForUser(const FUniqueNetIdRef UserId, const FString& StatName, const FOnlineStatValue& StatValue)
{
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (UserStats)
	{
		FOnlineStatValue* UserStatValue = UserStats->Stats.Find(StatName);
		if (!UserStatValue)
		{
			UserStats->Stats.Add(StatName, StatValue);
		}
	}
}

void FOnlineGDKStatsCache::DeleteUser(const FUniqueNetIdRef UserId)
{
	CachedUserStats.Remove(UserId);
}

void FOnlineGDKStatsCache::DeleteStatForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr)
{
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (UserStats)
	{
		UserStats->Stats.Remove(StatNameStr);
	}
}

void FOnlineGDKStatsCache::DeleteStatsForUser(const FUniqueNetIdRef UserId)
{
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (UserStats)
	{
		UserStats->Stats.Empty();
	}
}

template<>
double FOnlineGDKStatsCache::GetStatValue<double>(FOnlineStatValue* StatValue)
{
	double ReturnValue = 0.0f;
	if (StatValue)
	{
		StatValue->GetValue(ReturnValue);
	}
	return ReturnValue;
}

template<>
int64 FOnlineGDKStatsCache::GetStatValue<int64>(FOnlineStatValue* StatValue)
{
	int64 ReturnValue = 0;
	if (StatValue)
	{
		StatValue->GetValue(ReturnValue);
	}
	return ReturnValue;
}

template<>
uint64 FOnlineGDKStatsCache::GetStatValue<uint64>(FOnlineStatValue* StatValue)
{
	uint64 ReturnValue = 0;
	if (StatValue)
	{
		StatValue->GetValue(ReturnValue);
	}
	return ReturnValue;
}

template<>
FString FOnlineGDKStatsCache::GetStatValue<FString>(FOnlineStatValue* StatValue)
{
	FString ReturnValue;
	if (StatValue)
	{
		StatValue->GetValue(ReturnValue);
	}
	return ReturnValue;
}

template<typename T>
T FOnlineGDKStatsCache::GetStatValueForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr)
{
	FOnlineStatValue* StatValue = nullptr;
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (UserStats)
	{
		StatValue = UserStats->Stats.Find(StatNameStr);
	}

	return GetStatValue<T>(StatValue);
}

template<typename T>
void FOnlineGDKStatsCache::SetStatValueForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr, T Value)
{
	FOnlineStatsUserStats* UserStats = CachedUserStats.Find(UserId);
	if (UserStats)
	{
		UserStats->Stats.Add(StatNameStr, FOnlineStatValue(Value));
	}
	else
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("User %s not found when trying to set stat %s"), *UserId->ToDebugString(), *StatNameStr);
	}
}

template<typename T>
bool FOnlineGDKStatsCache::ApplyStatModification(const FUniqueNetIdRef UserId, const FString& StatName, const FOnlineStatUpdate& StatUpdate, bool& bOutHasUpdatedStat)
{
	bool bNeedsUpdate = false;
	bool bSuccess = false;

	T UpdateValue = OnlineStatsGDKHelpers::GetStatUpdateValue<T>(StatUpdate);

	switch (StatUpdate.GetModificationType())
	{
	case FOnlineStatUpdate::EOnlineStatModificationType::Sum:
	{
		if (StatUpdate.GetValue().IsNumeric())
		{
			T CurrentValue = GetStatValueForUser<T>(UserId, StatName);

			UpdateValue += CurrentValue;
			bNeedsUpdate = true;
			bSuccess = true;
		}
	}
	break;

	case FOnlineStatUpdate::EOnlineStatModificationType::Set:
	{
		bNeedsUpdate = true;
		bSuccess = true;
	}
	break;

	case FOnlineStatUpdate::EOnlineStatModificationType::Largest:
	{
		if (StatUpdate.GetValue().IsNumeric())
		{
			T CurrentValue = GetStatValueForUser<T>(UserId, StatName);
			bNeedsUpdate = (UpdateValue > CurrentValue);
			bSuccess = true;
		}
	}
	break;

	case FOnlineStatUpdate::EOnlineStatModificationType::Smallest:
	{
		if (StatUpdate.GetValue().IsNumeric())
		{
			T CurrentValue = GetStatValueForUser<T>(UserId, StatName);
			bNeedsUpdate = (UpdateValue < CurrentValue);
			bSuccess = true;
		}
	}
	break;

	default:
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Unsupported stat modification type: %d for stat %s"), (int)StatUpdate.GetModificationType(), *StatName);
	}
	break;
	}

	bOutHasUpdatedStat = bNeedsUpdate;
	if (bNeedsUpdate)
	{
		SetStatValueForUser<T>(UserId, StatName, UpdateValue);
	}

	return bSuccess;
}

FOnlineStatsGDK::FOnlineStatsGDK(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
	, StatsMode(EGDKStatsMode::Default)
{
	InitFromConfig();

	AppInitComplete = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FOnlineStatsGDK::OnEngineInitComplete);	
}

FOnlineStatsGDK::~FOnlineStatsGDK()
{
	UnhookEvents();
}

bool FOnlineStatsGDK::InitFromConfig()
{
	FString StatsModeString;
	if (GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("StatsMode"), StatsModeString, GEngineIni))
	{
		if (StatsModeString.Contains(TEXT("2013")))
		{
			StatsMode = EGDKStatsMode::Mode2013;
		}
		else if (StatsModeString.Contains(TEXT("2017")))
		{
			StatsMode = EGDKStatsMode::Mode2017;
		}
		else
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("FOnlineStatsLive: invalid configuration value \"%s\" for StatsMode."), *StatsModeString);
		}

		UE_LOG_ONLINE_STATS(Log, TEXT("FOnlineStatsLive: StatsMode set to \"%d\""), static_cast<int32>(StatsMode));
	}
	else
	{
		UE_LOG_ONLINE_STATS(Log, TEXT("FOnlineStatsLive: No StatsMode set; defaulting StatsMode to \"%d\""), static_cast<int32>(StatsMode));
	}

	return true;
}

void FOnlineStatsGDK::OnEngineInitComplete()
{
	// Query stats for all existent users
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	TArray<TSharedPtr<FUserOnlineAccount>> UserAccounts = Identity->GetAllUserAccounts();
	if (UserAccounts.Num() > 0)
	{
		TArray<FUniqueNetIdRef> UserIds;
		for (TSharedPtr<FUserOnlineAccount> UserAccount : UserAccounts)
		{
			UserIds.Add(UserAccount->GetUserId());
		}

		TArray<FString> StatNames;

		QueryStats(UserIds[0], UserIds, StatNames, FOnlineStatsQueryUsersStatsComplete::CreateThreadSafeSP(this, &FOnlineStatsGDK::OnQueryStatsComplete));


	}	
	// Setup delegates to query stats for all future users
	HookEvents();
}

void FOnlineStatsGDK::HookEvents()
{
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{

		UserChanged.Add(Identity->AddOnLoginStatusChangedDelegate_Handle(i, FOnLoginStatusChangedDelegate::CreateThreadSafeSP(this, &FOnlineStatsGDK::OnUserChanged)));
	}
}

void FOnlineStatsGDK::UnhookEvents()
{
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	uint8 CurrIdx = 0;
	for (FDelegateHandle DelegateHandle : UserChanged)
	{
		Identity->ClearOnLoginStatusChangedDelegate_Handle(CurrIdx++, DelegateHandle);
	}
	UserChanged.Empty();

	FCoreDelegates::OnFEngineLoopInitComplete.Remove(AppInitComplete);
}

void FOnlineStatsGDK::OnUserChanged(int32 GameUserIndex, ELoginStatus::Type PreviousLoginStatus, ELoginStatus::Type LoginStatus, const FUniqueNetId& UserId)
{
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		if (LoginStatus == ELoginStatus::LoggedIn)
		{
			FUniqueNetIdPtr LocalUserId = Identity->GetUniquePlayerId(GameUserIndex);
			QueryStats(LocalUserId.ToSharedRef(), LocalUserId.ToSharedRef(), FOnlineStatsQueryUserStatsComplete::CreateThreadSafeSP(this, &FOnlineStatsGDK::OnQueryStatsComplete));
		}
		else
		{
			FUniqueNetIdPtr LocalUserId = Identity->GetUniquePlayerId(GameUserIndex);
			StatsCache.DeleteUser(LocalUserId.ToSharedRef());
		}
	}
}

void FOnlineStatsGDK::OnQueryStatsComplete(const FOnlineError& ResultState, const TSharedPtr<const FOnlineStatsUserStats>& QueriedStats)
{
	if (QueriedStats.IsValid())
	{
		TArray<TSharedRef<const FOnlineStatsUserStats>> QueriedStatsArray;
		QueriedStatsArray.Add(QueriedStats.ToSharedRef());

		OnQueryStatsComplete(ResultState, QueriedStatsArray);
	}
}

void FOnlineStatsGDK::OnQueryStatsComplete(const FOnlineError& ResultState, const TArray<TSharedRef<const FOnlineStatsUserStats>>& QueriedStats)
{
	UE_LOG_ONLINE_STATS(Display, TEXT("Query finished with Result: %s"), *ResultState.ToLogString());

	for (TSharedRef<const FOnlineStatsUserStats> Entry : QueriedStats)
	{
		FUniqueNetIdRef EntryAccount = Entry->Account;

		StatsCache.AddUser(EntryAccount);

		const FOnlineStatsUserStats* UserStats = StatsCache.GetStatsForUser(EntryAccount);
		if (UserStats)
		{
			TArray<FString> StatNames;
			Entry->Stats.GetKeys(StatNames);
			for (const FString& StatName : StatNames)
			{
				const FOnlineStatValue* StatValue = UserStats->Stats.Find(StatName);
				if (StatValue)
				{
					FOnlineStatUpdate StatUpdate = FOnlineStatUpdate(Entry->Stats[StatName], FOnlineStatUpdate::EOnlineStatModificationType::Set);
					ApplyStatUpdate(EntryAccount, StatName, StatUpdate);
				}
				else
				{
					StatsCache.AddStatForUser(EntryAccount, StatName, Entry->Stats[StatName]);
				}
			}
		}
	}
}

void FOnlineStatsGDK::QueryStats(const FUniqueNetIdRef LocalUserId, const FUniqueNetIdRef StatsUserId, const FOnlineStatsQueryUserStatsComplete& Delegate)
{
	// Prepare async task parameters
	FGDKContextHandle GDKContext = GDKSubsystem->GetGDKContext(*LocalUserId);

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());
	int32 UserIndex = Identity->GetPlatformUserIdFromUniqueNetId(*LocalUserId);

	TUniqueNetIdMap<uint64> UserMapping;
	FGDKUserHandle User = GetUser(*StatsUserId);
	uint64 GDKUserId;
	if (User.IsValid() && SUCCEEDED(XUserGetId(User, &GDKUserId)))
	{
		UserMapping.Add(StatsUserId, GDKUserId);
	}
	else
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s."), *StatsUserId->ToDebugString());

		GDKSubsystem->ExecuteNextTick([Delegate]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStatsGDK_QueryStats_Delegate);
			Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), nullptr);
		});

		return;
	}

	auto NewTask = new FOnlineAsyncTaskGDKStatsQuery(GDKSubsystem, GDKContext, UserIndex, UserMapping, Delegate);
	GDKSubsystem->QueueAsyncTask(NewTask, false);
}

void FOnlineStatsGDK::QueryStats(const FUniqueNetIdRef LocalUserId, const TArray<FUniqueNetIdRef>& StatUsers, const TArray<FString>& StatNames, const FOnlineStatsQueryUsersStatsComplete& Delegate)
{
	// Prepare async task parameters
	FGDKContextHandle GDKContextHandle = GDKSubsystem->GetGDKContext(*LocalUserId);

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());
	int32 UserIndex = Identity->GetPlatformUserIdFromUniqueNetId(*LocalUserId);

	TUniqueNetIdMap<uint64> UserMapping;
	for (const FUniqueNetIdRef& StatsUserId : StatUsers)
	{
		FGDKUserHandle StatUser = GetUser(*StatsUserId);
		uint64 GDKUserId;
		if (StatUser.IsValid() && SUCCEEDED(XUserGetId(StatUser, &GDKUserId)))
		{
			UserMapping.Add(StatsUserId, GDKUserId);
		}
		else
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s."), *StatsUserId->ToDebugString());

			GDKSubsystem->ExecuteNextTick([Delegate]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStatsGDK_QueryStats_Delegate);
				Delegate.ExecuteIfBound(ONLINE_ERROR(EOnlineErrorResult::InvalidUser), TArray<TSharedRef<const FOnlineStatsUserStats>>());
			});

			continue;
		}
	}

	auto NewTask = new FOnlineAsyncTaskGDKStatsQuery(GDKSubsystem, GDKContextHandle, UserIndex, UserMapping, StatNames, Delegate);
	GDKSubsystem->QueueAsyncTask(NewTask, false);
}

TSharedPtr<const FOnlineStatsUserStats> FOnlineStatsGDK::GetStats(const FUniqueNetIdRef StatsUserId) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GDKStats_GetStats);

	FGDKUserHandle User = GetUser(*StatsUserId);
	if (!User.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s when getting stats."), *StatsUserId->ToDebugString());
		return nullptr;
	}

	// Get all stats for this user and fill in the results
	TSharedRef<FOnlineStatsUserStats> Result = MakeShared<FOnlineStatsUserStats>(StatsUserId);

	// Get cached stats for this user from local cache..
	const FOnlineStatsUserStats* UserStats = StatsCache.GetStatsForUser(StatsUserId);
	if (UserStats)
	{
		for (TPair<FString, FOnlineStatValue> StatPair : UserStats->Stats)
		{
			FOnlineStatValue& StatToUpdate = Result->Stats.Add(StatPair.Key);
			StatToUpdate = StatPair.Value;
		}
	}
	else
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("GetStats failed. No cached stats for user %s."), *StatsUserId->ToDebugString());
	}

	return Result;
}

void FOnlineStatsGDK::UpdateStats(const FUniqueNetIdRef LocalUserId, const TArray<FOnlineStatsUserUpdatedStats>& UpdatedUserStats, const FOnlineStatsUpdateStatsComplete& Delegate)
{
	const auto CompleteNextTick = [this,Delegate]( EOnlineErrorResult Error )
	{
		GDKSubsystem->ExecuteNextTick([Delegate,Error]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStatsGDK_UpdateStats_Delegate);
			Delegate.ExecuteIfBound(ONLINE_ERROR(Error));
		});
	};

	if (StatsMode != EGDKStatsMode::Mode2017)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("UpdateStats not supported outside Title Statistics (2017) mode."));
		CompleteNextTick(EOnlineErrorResult::NotImplemented);
		return;
	}

	// Prepare the async task
	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	check(Identity.IsValid());

	TMap<FUniqueNetIdGDKRef, uint64> GDKUserMapping;
	for (const FOnlineStatsUserUpdatedStats& UpdatedUserStat : UpdatedUserStats)
	{
		FUniqueNetIdGDKRef GDKId = FUniqueNetIdGDK::Cast(*UpdatedUserStat.Account);
		if (!GDKUserMapping.Contains(GDKId))
		{
			FGDKUserHandle User = Identity->GetUserForUniqueNetId(*GDKId);
			GDKUserMapping.Add(GDKId, GDKId->ToUint64());
		}
	}

	// Updated user stats must contain ALL user stats, not just the ones changing.
	auto NewTask = new FOnlineAsyncTaskGDKStatsUpdate(GDKSubsystem, GDKSubsystem->GetGDKContext(*LocalUserId), UpdatedUserStats, Delegate, GDKUserMapping);
	GDKSubsystem->QueueAsyncTask(NewTask, false);
}

#if !UE_BUILD_SHIPPING
void FOnlineStatsGDK::ResetStats( const FUniqueNetIdRef StatsUserId )
{
	if (StatsMode != EGDKStatsMode::Mode2017)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("ResetStats not supported outside Title Statistics (2017) mode."));
		return;
	}

	FGDKUserHandle User = GetUser(*StatsUserId);
	if (!User.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s when resetting stats."), *StatsUserId->ToDebugString());
		return;
	}

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		UE_LOG_ONLINE_STATS(Display, TEXT("\tResetting stats for user %s"), *Identity->GetPlayerNickname(User));
	}

	StatsCache.DeleteStatsForUser(StatsUserId);
}
#endif // !UE_BUILD_SHIPPING

FGDKUserHandle FOnlineStatsGDK::GetUser(const FUniqueNetId& LocalUserId) const
{
	FGDKUserHandle User;

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		FUniqueNetIdGDKRef GDKId = FUniqueNetIdGDK::Cast(LocalUserId);
		User = Identity->GetUserForUniqueNetId(*GDKId);
		if (!User.IsValid())
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("%s is not a local user"), *LocalUserId.ToDebugString());
		}
	}

	return User;
}

bool FOnlineStatsGDK::ApplyStatUpdate(const FUniqueNetIdRef& User, const FString& StatName, const FOnlineStatUpdate& StatUpdate)
{
	bool bHasUpdatedStat = false;
	bool bSuccess = false;

	// try to apply the value update
	switch (StatUpdate.GetType())
	{
	case EOnlineKeyValuePairDataType::Int32:
	case EOnlineKeyValuePairDataType::Int64:
	{
		bSuccess = StatsCache.ApplyStatModification<int64>(User, StatName, StatUpdate, bHasUpdatedStat);
	}
	break;
	case EOnlineKeyValuePairDataType::UInt32:
	case EOnlineKeyValuePairDataType::UInt64:
	{
		bSuccess = StatsCache.ApplyStatModification<uint64>(User, StatName, StatUpdate, bHasUpdatedStat);
	}
	break;

	case EOnlineKeyValuePairDataType::Float:
	case EOnlineKeyValuePairDataType::Double:
	{
		bSuccess = StatsCache.ApplyStatModification<double>(User, StatName, StatUpdate, bHasUpdatedStat);
	}
	break;

	case EOnlineKeyValuePairDataType::String:
	{
		bSuccess = StatsCache.ApplyStatModification<FString>(User, StatName, StatUpdate, bHasUpdatedStat);
	}
	break;

	case EOnlineKeyValuePairDataType::Bool:
	{
		if (StatUpdate.GetModificationType() == FOnlineStatUpdate::EOnlineStatModificationType::Set)
		{
			bool bValue = false;
			StatUpdate.GetValue().GetValue(bValue);

			FOnlineStatUpdate StatUpdateInteger(bValue ? 1 : 0, StatUpdate.GetModificationType());
			bSuccess = StatsCache.ApplyStatModification<int64>(User, StatName, StatUpdateInteger, bHasUpdatedStat);
		}
		else
		{
			UE_LOG_ONLINE_STATS(Warning, TEXT("Unsupported boolean stat modification type: %d for stat %s"), (int)StatUpdate.GetModificationType(), *StatName);
		}
	}
	break;

	default:
	{
		bSuccess = false;
		UE_LOG_ONLINE_STATS(Warning, TEXT("Unsupported stat update type: %s"), EOnlineKeyValuePairDataType::ToString(StatUpdate.GetType()));
	}
	break;
	}
	return bSuccess;
}

bool FOnlineStatsGDK::CopyStatValue(FOnlineStatValue& OutOnlineStatValue, const char* InStatisticValue, const char* InStatisticType) const
{
	//Support for types configurable via partner center

	const FString StatisticType = UTF8_TO_TCHAR(InStatisticType);
	const FString StatisticValue = UTF8_TO_TCHAR(InStatisticValue);

	if (FCString::Strcmp(*StatisticType, TEXT("Int32")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Atoi(*StatisticValue));
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("UInt32")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Strtoi(*StatisticValue, nullptr, 10)); // truncation should be fine if type is guranteed
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("Int64")) == 0 || FCString::Strcmp(*StatisticType, TEXT("Integer")) == 0) //Integers can also be read with this non-specific type
	{
		// Default integer type will be Int64
		OutOnlineStatValue.SetValue(FCString::Atoi64(*StatisticValue));
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("UInt64")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Strtoui64(*StatisticValue, nullptr, 10));
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("Float")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Atof(*StatisticValue));
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("Double")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Atod(*StatisticValue));
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("Bool")) == 0)
	{
		OutOnlineStatValue.SetValue(FCString::Strcmp(*StatisticValue, TEXT("False")) != 0);
		return true;
	}
	if (FCString::Strcmp(*StatisticType, TEXT("UnicodeString")) == 0)
	{
		OutOnlineStatValue.SetValue(*StatisticValue);
		return true;
	}

	OutOnlineStatValue.Empty();
	return false;
}

bool FOnlineStatsGDK::HandleExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING

	// Helper to get optional user for exec commands
	auto ParseLocalUserId = [&](const TCHAR*& Cmd)
	{
		FUniqueNetIdPtr LocalUserId;

		int UserIndex = FCString::Atoi(*FParse::Token(Cmd, false));

		TArray<FUniqueNetIdRef> CachedUsers = StatsCache.GetUsers();

		if (CachedUsers.IsValidIndex(UserIndex))
		{
			LocalUserId = CachedUsers[UserIndex];
		}

		if (LocalUserId.IsValid())
		{
			FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
			check(Identity.IsValid());

			UE_LOG_ONLINE_STATS(Display, TEXT("--- using user %s ---"), *Identity->GetPlayerNickname(*LocalUserId));
		}
		else
		{
			UE_LOG_ONLINE_STATS(Display, TEXT("--- invalid user ---"));
		}

		return LocalUserId;
	};

	bool bWasHandled = true;
	if (FParse::Command(&Cmd, TEXT("USERS"))) //ONLINE STATS USERS  -> show registered users and their index
	{
		HandleUsersExecCommand();
	}
	else if (FParse::Command(&Cmd, TEXT("DUMP"))) //ONLINE STATS DUMP [user index]
	{
		FUniqueNetIdPtr LocalUserId = ParseLocalUserId(Cmd);
		if(LocalUserId.IsValid())
		{
			HandleDumpStatsExecCommand(LocalUserId.ToSharedRef());
		}
	}
	else if (FParse::Command(&Cmd, TEXT("SET"))) //ONLINE STATS SET <statname> <statvalue> [user index]
	{
		FString StatName = FParse::Token(Cmd,true);
		FString Value = FParse::Token(Cmd,true);
		FUniqueNetIdPtr LocalUserId = ParseLocalUserId(Cmd);
		if(LocalUserId.IsValid() && !StatName.IsEmpty())
		{
			HandleSetStatExecCommand( LocalUserId.ToSharedRef(), StatName, Value );
		}
	}
	else if (FParse::Command(&Cmd, TEXT("DEL"))) //ONLINE STATS DEL <statname> [user index]
	{
		FString StatName = FParse::Token(Cmd,true);
		FUniqueNetIdPtr LocalUserId = ParseLocalUserId(Cmd);
		if(LocalUserId.IsValid() && !StatName.IsEmpty())
		{
			HandleDeleteStatExecCommand( LocalUserId.ToSharedRef(), StatName );
		}
	}
	else if (FParse::Command(&Cmd, TEXT("RESET"))) //ONLINE STATS RESET [user index]
	{
		FUniqueNetIdPtr LocalUserId = ParseLocalUserId(Cmd);
		if(LocalUserId.IsValid())
		{
			ResetStats(LocalUserId.ToSharedRef());
		}
	}
	else
	{
		UE_LOG_ONLINE_STATS( Warning, TEXT("Unknown ONLINE STATS command: %s"), *FParse::Token(Cmd,true) );
		bWasHandled = false;
	}

	return bWasHandled;
#else
	return false;
#endif // !UE_BUILD_SHIPPING

}



#if !UE_BUILD_SHIPPING
void FOnlineStatsGDK::HandleDumpStatsExecCommand( const FUniqueNetIdRef LocalUserId )
{
	FGDKUserHandle User = GetUser(*LocalUserId);
	if (!User.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s when dumping stats."), *LocalUserId->ToDebugString());
		return;
	}

	TSharedPtr<const FOnlineStatsUserStats> UserStats = GetStats(LocalUserId);
	if (UserStats.IsValid())
	{
		for( auto Stat : UserStats->Stats )
		{
			UE_LOG_ONLINE_STATS( Display, TEXT("\t%-32s %s"), *Stat.Key, *Stat.Value.ToString() );
		}
	}
	UE_LOG_ONLINE_STATS( Display, TEXT("--- done --- ") );
}
#endif // !UE_BUILD_SHIPPING



#if !UE_BUILD_SHIPPING
void FOnlineStatsGDK::HandleSetStatExecCommand( FUniqueNetIdRef LocalUserId, const FString& StatName, const FString& StringValue )
{
	if (StatsMode != EGDKStatsMode::Mode2017)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("DeleteStat not supported outside Title Statistics (2017) mode."));
		return;
	}

	FGDKUserHandle User = GetUser(*LocalUserId);
	if (!User.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s when setting stat %s."), *LocalUserId->ToDebugString(), *StatName);
		return;
	}

	// build boilerplate for updating stat, depending on whether the given value is a string or numeric (not ideal, but this is only for console command testing!)
	FOnlineStatsUserUpdatedStats UpdatedStats(LocalUserId);
	if (StringValue.IsNumeric())
	{
		const double NumericValue = FCString::Atod(*StringValue);
		UE_LOG_ONLINE_STATS(Display, TEXT("\tSetting \"%s\" to numeric %f   (Current value: %f)"), *StatName, NumericValue, StatsCache.GetStatValueForUser<double>(LocalUserId, StatName));
		UpdatedStats.Stats.Add(StatName, FOnlineStatUpdate(NumericValue, FOnlineStatUpdate::EOnlineStatModificationType::Set));
	}
	else
	{
		UE_LOG_ONLINE_STATS(Display, TEXT("\tSetting \"%s\" to string \"%s\"   (Current value=\"%s\")"), *StatName, *StringValue, *StatsCache.GetStatValueForUser<FString>(LocalUserId, StatName));
		UpdatedStats.Stats.Add(StatName, FOnlineStatUpdate(StringValue, FOnlineStatUpdate::EOnlineStatModificationType::Set));
	}

	TArray<FOnlineStatsUserUpdatedStats> UpdatedUserStats;
	UpdatedUserStats.Add( UpdatedStats );

	// request stat update
	UpdateStats( LocalUserId, UpdatedUserStats, FOnlineStatsUpdateStatsComplete::CreateLambda( [this,LocalUserId]( const FOnlineError& ResultState )
	{
		UE_LOG_ONLINE_STATS(Display, TEXT("\tResult state %s"), *ResultState.ToLogString());
	}));
}
#endif // !UE_BUILD_SHIPPING

#if !UE_BUILD_SHIPPING
void FOnlineStatsGDK::HandleDeleteStatExecCommand(FUniqueNetIdRef LocalUserId, const FString& StatName)
{
	if (StatsMode != EGDKStatsMode::Mode2017)
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("DeleteStat not supported outside Title Statistics (2017) mode."));
		return;
	}

	FGDKUserHandle User = GetUser(*LocalUserId);
	if (!User.IsValid())
	{
		UE_LOG_ONLINE_STATS(Warning, TEXT("Invalid user %s when deleting stat %s."), *LocalUserId->ToDebugString(), *StatName);
		return;
	}

	FOnlineIdentityGDKPtr Identity = GDKSubsystem->GetIdentityGDK();
	if (Identity.IsValid())
	{
		UE_LOG_ONLINE_STATS(Display, TEXT("\tDeleting stat %s for user %s"), *StatName, *Identity->GetPlayerNickname(User));
	}

	StatsCache.DeleteStatForUser(LocalUserId, StatName);
}
#endif // !UE_BUILD_SHIPPING

#if !UE_BUILD_SHIPPING
void FOnlineStatsGDK::HandleUsersExecCommand()
{
	TArray<FUniqueNetIdRef> CachedUsers = StatsCache.GetUsers();

	for (int Index = 0; Index < CachedUsers.Num(); Index++)
	{
		FGDKUserHandle UserHandle = GetUser(*CachedUsers[Index]);

		FString Gamertag = GDKSubsystem->GetIdentityGDK()->GetPlayerNickname(UserHandle);

		UE_LOG_ONLINE_STATS(Display, TEXT("\t%-4d %ls"), Index, *Gamertag);
	}
	UE_LOG_ONLINE_STATS(Display, TEXT("--- done ---"));
}
#endif // !UE_BUILD_SHIPPING

#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK