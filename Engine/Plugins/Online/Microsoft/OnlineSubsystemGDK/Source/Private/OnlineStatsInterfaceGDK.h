// Copyright Epic Games, Inc. All Rights Reserved.

//WMM TODO Stats do not seem to be implemented for GDK yet
#pragma once

#include "GDKHandle.h"
#include "Interfaces/OnlineStatsInterface.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGDKTypes.h"

enum class EGDKStatsMode : uint16
{
	/** 2013 stats are driven via events */
	Mode2013 = 2013,
	/** 2017 stats are title-managed */
	Mode2017 = 2017,

	/** Default mode to use if not specified */
	Default = Mode2013
};

/** Keeps a record of the value of all defined stats per logged in local user */
class FOnlineGDKStatsCache
{

private:
	/** Template helpers to savely get a stat value */
	template<typename T>
	T GetStatValue(FOnlineStatValue* StatValue);

	template<>
	double GetStatValue<double>(FOnlineStatValue* StatValue);

	template<>
	int64 GetStatValue<int64>(FOnlineStatValue* StatValue);

	template<>
	uint64 GetStatValue<uint64>(FOnlineStatValue* StatValue);

	template<>
	FString GetStatValue<FString>(FOnlineStatValue* StatValue);

public:
	/** Template to safely set a stat value for a given user */
	template<typename T>
	void SetStatValueForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr, T Value);

	/** Template to safely get a stat value for a given user */
	template<typename T>
	T GetStatValueForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr);

	/** Template to safely apply a stat modification */
	template<typename T>
	bool ApplyStatModification(const FUniqueNetIdRef UserId, const FString& StatName, const FOnlineStatUpdate& StatUpdate, bool& bOutHasUpdatedStat);

	TArray<FUniqueNetIdRef> GetUsers();
	const FOnlineStatsUserStats* GetStatsForUser(const FUniqueNetIdRef UserId) const;
	void AddUser(const FUniqueNetIdRef UserId);
	void AddStatForUser(const FUniqueNetIdRef UserId, const FString& StatName, const FOnlineStatValue& StatValue);

	void DeleteUser(const FUniqueNetIdRef UserId);
	void DeleteStatForUser(const FUniqueNetIdRef UserId, const FString& StatNameStr);
	void DeleteStatsForUser(const FUniqueNetIdRef UserId);

private:
	TUniqueNetIdMap<FOnlineStatsUserStats> CachedUserStats;
};

/**
 *	FOnlineStatsGDK - Interface class for stats (GDK implementation)
 */
class FOnlineStatsGDK : public IOnlineStats, public TSharedFromThis< FOnlineStatsGDK, ESPMode::ThreadSafe>
{
	/** The async task classes require friendship */
	friend class FOnlineAsyncTaskGDKStatsUpdate;
	friend class FOnlineAsyncTaskGDKStatsUserRegistration;
	friend class FOnlineAsyncTaskGDKStatsQuery;

private:
	bool InitFromConfig();

	/** Pointer to owning live subsystem */
	class FOnlineSubsystemGDK* GDKSubsystem;

	/** Stats mode used if no mode is specified in the config file */
	EGDKStatsMode StatsMode = EGDKStatsMode::Default;

	FOnlineGDKStatsCache StatsCache;

	FDelegateHandle AppInitComplete;
	TArray<FDelegateHandle> UserChanged;

PACKAGE_SCOPE:

public:

	//~ Begin IOnlineStats Interface
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const FUniqueNetIdRef StatsUser, const FOnlineStatsQueryUserStatsComplete& Delegate) override;
	virtual void QueryStats(const FUniqueNetIdRef LocalUserId, const TArray<FUniqueNetIdRef>& StatUsers, const TArray<FString>& StatNames, const FOnlineStatsQueryUsersStatsComplete& Delegate) override;
	virtual TSharedPtr<const FOnlineStatsUserStats> GetStats(const FUniqueNetIdRef StatsUserId) const override;
	virtual void UpdateStats(const FUniqueNetIdRef LocalUserId, const TArray<FOnlineStatsUserUpdatedStats>& UpdatedUserStats, const FOnlineStatsUpdateStatsComplete& Delegate) override;

#if !UE_BUILD_SHIPPING
	virtual void ResetStats( const FUniqueNetIdRef StatsUserId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineStats Interface

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineStatsGDK(class FOnlineSubsystemGDK* InSubsystem);
	
	/**
	 * Default destructor
	 */
	virtual ~FOnlineStatsGDK();

	bool HandleExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

private:
	bool CopyStatValue(FOnlineStatValue& OutOnlineStatValue, const char* InStatisticValue, const char* InStatisticType) const;
	bool ApplyStatUpdate(const FUniqueNetIdRef& User, const FString& StatName, const FOnlineStatUpdate& StatUpdate);

	FGDKUserHandle GetUser(const FUniqueNetId& LocalUserId) const;

	void HookEvents();
	void UnhookEvents();

	void OnEngineInitComplete();
	void OnUserChanged(int32 GameUserIndex, ELoginStatus::Type PreviousLoginStatus, ELoginStatus::Type LoginStatus, const FUniqueNetId& UserId);
	void OnQueryStatsComplete(const FOnlineError& ResultState, const TSharedPtr<const FOnlineStatsUserStats>& QueriedStats);
	void OnQueryStatsComplete(const FOnlineError& ResultState, const TArray<TSharedRef<const FOnlineStatsUserStats>>& QueriedStats);

#if !UE_BUILD_SHIPPING
	void HandleDumpStatsExecCommand( FUniqueNetIdRef LocalUserId );
	void HandleSetStatExecCommand( FUniqueNetIdRef LocalUserId, const FString& StatName, const FString& Value );
	void HandleDeleteStatExecCommand( FUniqueNetIdRef LocalUserId, const FString& StatName );
	void HandleUsersExecCommand();
#endif // !UE_BUILD_SHIPPING
};

typedef TSharedPtr<FOnlineStatsGDK, ESPMode::ThreadSafe> FOnlineStatsGDKPtr;
