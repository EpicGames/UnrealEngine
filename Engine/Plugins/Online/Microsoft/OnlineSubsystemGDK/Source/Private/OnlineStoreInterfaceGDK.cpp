// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineStoreInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "GDKRuntimeModule.h"
#include "GDKThreadCheck.h"
#include "OnlineError.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryOffers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryAllOffers.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformMisc.h"

#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.store"


FOnlineStoreGDK::FOnlineStoreGDK(FOnlineSubsystemGDK* InGDKSubsystem)
	: GDKSubsystem(InGDKSubsystem)
{
	check(GDKSubsystem);
	bBlockOnStoreIDMismatch = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bBlockOnStoreIDMismatch"), bBlockOnStoreIDMismatch, GEngineIni);
	GDKSubsystem->ExecuteNextTick([this]() {
		FOnlineIdentityGDKPtr IdentityGDK = GDKSubsystem->GetIdentityGDK();
		for (int32 LocalUserNum = 0; LocalUserNum < MAX_LOCAL_PLAYERS; LocalUserNum++)
		{
			IdentityGDK->AddOnLoginStatusChangedDelegate_Handle(LocalUserNum, FOnLoginStatusChangedDelegate::CreateThreadSafeSP(this, &FOnlineStoreGDK::OnConsoleLoginStatusChanged));
		}
	});

	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FOnlineStoreGDK::HandleAppSuspend);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FOnlineStoreGDK::HandleAppResume);
}

bool FOnlineStoreGDK::BlockMismatchedStoreUser(const FGDKUserHandle GDKUser)
{
	return bBlockOnStoreIDMismatch && !XUserIsStoreUser(GDKUser);
}

FOnlineStoreGDK::~FOnlineStoreGDK()
{
	Cleanup();

	FOnlineIdentityGDKPtr IdentityGDK = GDKSubsystem->GetIdentityGDK();
	for (int32 LocalUserNum = 0; LocalUserNum < MAX_LOCAL_PLAYERS; LocalUserNum++)
	{
		IdentityGDK->ClearOnLoginStatusChangedDelegates(LocalUserNum, this);
	}

	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
}

void FOnlineStoreGDK::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate /*= FOnQueryOnlineStoreCategoriesComplete()*/)
{
	GDKSubsystem->ExecuteNextTick([Delegate]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStoreGDK_QueryCategories_Delegate);
		Delegate.ExecuteIfBound(false, TEXT("FOnlineStoreLive::QueryCategories Not Implemented"));
	});
}

void FOnlineStoreGDK::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	OutCategories.Empty();
}

void FOnlineStoreGDK::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate /*= FOnQueryOnlineStoreOffersComplete()*/)
{
	FString ErrorStr;
	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(UserId);
	if (!UserGDKContext.IsValid())
	{
		ErrorStr = TEXT("Could not map requested user to a GDKContext");
	}

	FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(UserId));
	if (BlockMismatchedStoreUser(GDKUser))
	{
		ErrorStr = ONLINE_ERROR(EOnlineErrorResult::MismatchedUser).GetErrorMessage().ToString();
	}
	
	if (ErrorStr.IsEmpty())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryAllOffers>(GDKSubsystem, UserGDKContext, Delegate);
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([Delegate, ErrorStr]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStoreGDK_QueryOffersByFilter_Delegate);
			Delegate.ExecuteIfBound(false, TArray<FUniqueOfferId>(), ErrorStr);
		});
	}
}

void FOnlineStoreGDK::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate /*= FOnQueryOnlineStoreOffersComplete()*/)
{
	FString ErrorStr;
	FGDKContextHandle UserGDKContext = GDKSubsystem->GetGDKContext(UserId);
	if (!UserGDKContext.IsValid())
	{
		ErrorStr = TEXT("Could not map requested user to a GDKContext");
	}
	else if (OfferIds.Num() < 1)
	{
		ErrorStr = TEXT("No OfferIds requested");
	}
	FGDKUserHandle GDKUser = GDKSubsystem->GetIdentityGDK()->GetUserForUniqueNetId(static_cast<const FUniqueNetIdGDK&>(UserId));

	if (BlockMismatchedStoreUser(GDKUser))
	{
		ErrorStr = ONLINE_ERROR(EOnlineErrorResult::MismatchedUser).GetErrorMessage().ToString();
	}

	if (ErrorStr.IsEmpty())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryOffers>(GDKSubsystem, UserGDKContext, OfferIds, Delegate);
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([Delegate, OfferIds, ErrorStr]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineStoreGDK_QueryOffersById_Delegate);
			Delegate.ExecuteIfBound(false, OfferIds, ErrorStr);
		});
	}
}

void FOnlineStoreGDK::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	CachedOffers.GenerateValueArray(OutOffers);
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreGDK::GetOffer(const FUniqueOfferId& OfferId) const
{
	const FOnlineStoreOfferRef* FoundOfferPtr = CachedOffers.Find(OfferId);
	if (FoundOfferPtr == nullptr)
	{
		return nullptr;
	}

	return *FoundOfferPtr;
}

XStoreContextHandle FOnlineStoreGDK::GetStoreContextHandle(FGDKUserHandle GDKUser)
{
	if (bStoreContextHandlesInvalidated)
	{
		Cleanup();
		bStoreContextHandlesInvalidated = false;
	}

	FScopeLock ScopeLock(&StoreContextsLock);

	// XStoreCreateContext is not safe to call on a time-sensitive thread, call it from Tick() of corresponding GDK Task
	check(!IsInGameThread());

	XStoreContextHandle Result = nullptr;

	if (StoreContexts.Contains(GDKUser))
	{
		Result = StoreContexts[GDKUser];
	}
	else
	{
		if (XStoreCreateContext(GDKUser, &Result) == S_OK && Result != nullptr)
		{
			StoreContexts.Add(GDKUser, Result);
		}
	}

	return Result;
}

void FOnlineStoreGDK::Cleanup()
{
	FScopeLock ScopeLock(&StoreContextsLock);

	UE_LOG_ONLINE(Verbose, TEXT("[FOnlineStoreGDK::Cleanup] Closing all context handles"));
	for (TPair<FGDKUserHandle, XStoreContextHandle>& Entry : StoreContexts)
	{
		if (Entry.Value != nullptr)
		{
			// XStoreCloseContextHandle IS safe to call on a time-sensitive thread
			XStoreCloseContextHandle(Entry.Value);
			Entry.Value = nullptr;
		}
	}

	StoreContexts.Reset();
}

void FOnlineStoreGDK::HandleAppSuspend()
{
	SCOPED_ENTER_BACKGROUND_EVENT(STAT_FOnlineStoreGDK_HandleAppSuspend);

	UE_LOG_ONLINE(Verbose, TEXT("[FOnlineStoreGDK::HandleAppSuspend]"));

	// Instead of Cleanup() here, mark handles as invalidated to finish suspend callbacks as soon as possible.
	// It's to fix the potential dead lock when app get suspended, XStoreCreateContext could wait for IO, take 
	// a long time to complete, while FOnlineStoreGDK::HandleAppSuspend is waiting for the lock. Now by only 
	// marking handles as invalid by flag, the callback FOnlineStoreGDK::HandleAppSuspend should finish quickly 
	// to avoid the issue.
	bStoreContextHandlesInvalidated = true;
}

void FOnlineStoreGDK::HandleAppResume()
{
	UE_LOG_ONLINE(Verbose, TEXT("[FOnlineStoreGDK::HandleAppResume]"));
}

void FOnlineStoreGDK::OnConsoleLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& NewId)
{
	// Creation and closure of store context handles must follow the related user's life-cycle, not the application's
	const FPlatformUserId PlatformId = FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum);
	const FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformId);
	if (UserHandle.IsValid())
	{
		if (NewStatus == ELoginStatus::NotLoggedIn)
		{
			FScopeLock ScopeLock(&StoreContextsLock);
			if (StoreContexts.Contains(UserHandle))
			{
				// XStoreCloseContextHandle IS safe to call on a time-sensitive thread
				XStoreCloseContextHandle(StoreContexts[UserHandle]);
				StoreContexts.Remove(UserHandle);
			}
		}
	}
}
#undef ONLINE_ERROR_NAMESPACE

#endif //WITH_GRDK