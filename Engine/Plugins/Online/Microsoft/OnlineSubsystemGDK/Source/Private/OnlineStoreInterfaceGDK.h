// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemGDKPrivate.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlineSubsystemGDKPackage.h"
#include "OnlineSubsystemGDKTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class FOnlineSubsystemGDK;

class FOnlineStoreOfferGDK
	: public FOnlineStoreOffer
{
public:
	/** Signed value to pass to XBox when purchasing this offer */
	FString SignedOffer;
};

using FOnlineStoreOfferGDKRef = TSharedRef<FOnlineStoreOfferGDK>;

class FOnlineStoreGDK
	: public IOnlineStoreV2
	, public TSharedFromThis<FOnlineStoreGDK, ESPMode::ThreadSafe>
{
public:
	FOnlineStoreGDK(FOnlineSubsystemGDK* InGDKSubsystem);
	virtual ~FOnlineStoreGDK();

public:
	//~ Begin IOnlineStoreV2 Interface
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate = FOnQueryOnlineStoreCategoriesComplete()) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate = FOnQueryOnlineStoreOffersComplete()) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate = FOnQueryOnlineStoreOffersComplete()) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;
	//~ End IOnlineStoreV2 Interface

	XStoreContextHandle GetStoreContextHandle(FGDKUserHandle GDKUser);
	bool BlockMismatchedStoreUser(const FGDKUserHandle GDKUser);

PACKAGE_SCOPE:
	void Cleanup();

	void HandleAppSuspend();
	void HandleAppResume();

	void OnConsoleLoginStatusChanged(int32 LocalUserNum, ELoginStatus::Type OldStatus, ELoginStatus::Type NewStatus, const FUniqueNetId& NewId);

PACKAGE_SCOPE:
	FOnlineSubsystemGDK* GDKSubsystem;

	TMap<FUniqueOfferId, FOnlineStoreOfferRef> CachedOffers;

	TMap<FGDKUserHandle, XStoreContextHandle> StoreContexts;
	mutable FCriticalSection StoreContextsLock;

	std::atomic<bool> bStoreContextHandlesInvalidated = false;

private:
	/** Should we block store interaction if the title and MSstore IDs do not match? */
	bool bBlockOnStoreIDMismatch;
};

typedef TSharedPtr<FOnlineStoreGDK, ESPMode::ThreadSafe> FOnlineStoreGDKPtr;
