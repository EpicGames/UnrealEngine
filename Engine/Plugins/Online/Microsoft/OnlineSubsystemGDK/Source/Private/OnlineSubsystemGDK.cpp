// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineSubsystemGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "Modules/ModuleManager.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"
#include "GDKThreadCheck.h"

#include "OnlineFriendsInterfaceGDK.h"
#include "MessageSanitizerGDK.h"
// #include "OnlineUserCloudInterfaceGDK.h"
#include "OnlineLeaderboardInterfaceGDK.h"
#include "OnlineExternalUIInterfaceGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "OnlineStoreInterfaceGDK.h"
#include "OnlinePurchaseInterfaceGDK.h"
#include "OnlineAchievementsInterfaceGDK.h"
#include "OnlineAsyncTaskManagerGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort.h"
#include "OnlineSessionGDK.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "OnlinePresenceInterfaceGDK.h"
#include "OnlineDelegates.h"
#if WITH_ENGINE
#include "OnlineSubsystemUtils.h"
#include "OnlineVoiceInterfaceGDK.h"
#include "Framework/Application/SlateApplication.h"
#endif //WITH_ENGINE
#include "OnlineUserInterfaceGDK.h"
#include "OnlineStatsInterfaceGDK.h"
#include "SessionMessageRouter.h"
#include "OnlineMatchmakingInterfaceGDK.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <Xal/xal.h>
#include <winsock2.h>
#include <iphlpapi.h>

#include <GRDK.h> //for _GRDK_EDITION
#include <XGameRuntime.h>
#include <XGameRuntimeFeature.h>
#include <httpClient/httpProvider.h>

#include <xsapi-c/xbox_live_context_settings_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"
// FOnlineSubsystemGDKModule

namespace UE::OnlineSubsystemGDK
{
	bool bTreatConnectivityUnknownAsDisconnected = true;
	static FAutoConsoleVariableRef CVarTreatConnectivityUnknownAsDisconnected(
		TEXT("OnlineSubsystemGDK.TreatConnectivityUnknownAsDisconnected"),
		bTreatConnectivityUnknownAsDisconnected,
		TEXT("True (Default): Triggers OnConnectionStatusChanged delegate also for Unknown connectivity state.\n")
		TEXT("False: Does not trigger OnConnectionStatusChanged delegate for Unknown connectivity state"),
		ECVF_Default
	);
}

bool FOnlineSubsystemGDK::IsEnabled() const
{
	return FOnlineSubsystemImpl::IsEnabled() && IGDKRuntimeModule::Get().IsAvailable(); 
}

IOnlineSessionPtr FOnlineSubsystemGDK::GetSessionInterface() const
{
	return SessionInterface;
}

IOnlineFriendsPtr FOnlineSubsystemGDK::GetFriendsInterface() const
{
	return FriendInterface;
}

IMessageSanitizerPtr FOnlineSubsystemGDK::GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const
{
	bool bUseMessageSanitizer = true;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bUseMessageSanitizer"), bUseMessageSanitizer, GEngineIni);
	if (bUseMessageSanitizer)
	{
		OutAuthTypeToExclude = IdentityInterface->GetAuthType();
		return MessageSanitizer;
	}
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemGDK::GetPartyInterface() const
{
	// Xbox GDK used to support this, but no longer available
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemGDK::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemGDK::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemGDK::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemGDK::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemGDK::GetLeaderboardsInterface() const
{
	return LeaderboardsInterface;
}

IOnlineVoicePtr FOnlineSubsystemGDK::GetVoiceInterface() const
{
#if WITH_ENGINE
	return VoiceInterface;
#else //WITH_ENGINE
	return nullptr;
#endif //WITH_ENGINE
}

IOnlineExternalUIPtr FOnlineSubsystemGDK::GetExternalUIInterface() const
{
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemGDK::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemGDK::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineTitleFilePtr FOnlineSubsystemGDK::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineStoreV2Ptr FOnlineSubsystemGDK::GetStoreV2Interface() const
{
	return StoreInterface;
}

IOnlinePurchasePtr FOnlineSubsystemGDK::GetPurchaseInterface() const
{
	return PurchaseInterface;
}

IOnlineEventsPtr FOnlineSubsystemGDK::GetEventsInterface() const
{
	return EventsInterface;
}

IOnlineAchievementsPtr FOnlineSubsystemGDK::GetAchievementsInterface() const
{
	return AchievementInterface;
}

IOnlineSharingPtr FOnlineSubsystemGDK::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemGDK::GetUserInterface() const
{
	return UserInterface;
}

IOnlineMessagePtr FOnlineSubsystemGDK::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemGDK::GetPresenceInterface() const
{
	return PresenceInterface;
}

IOnlineChatPtr FOnlineSubsystemGDK::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemGDK::GetStatsInterface() const
{
	return StatsInterface;
}

IOnlineTurnBasedPtr FOnlineSubsystemGDK::GetTurnBasedInterface() const
{
	return nullptr;
}

FOnlineMatchmakingInterfaceGDKPtr FOnlineSubsystemGDK::GetMatchmakingInterface() const
{
	return MatchmakingInterfaceGDK;
}

IOnlineTournamentPtr FOnlineSubsystemGDK::GetTournamentInterface() const
{
	return nullptr;
}

/**
 *	Give the online subsystem a chance to tick its tasks
 */
bool FOnlineSubsystemGDK::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGDK_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		OnlineAsyncTaskThreadRunnable->GameTick();
	}

	if (SessionInterface.IsValid())
	{
		SessionInterface->Tick(DeltaTime);
	}

#if WITH_ENGINE
	if (VoiceInterface.IsValid())
	{
		VoiceInterface->Tick(DeltaTime);
	}
#endif //WITH_ENGINE

	if (ExternalUIInterface.IsValid())
	{
		ExternalUIInterface->Tick(DeltaTime);
	}

	if (PresenceInterface.IsValid())
	{
		PresenceInterface->Tick(DeltaTime);
	}

	if (IdentityInterface.IsValid())
	{
		IdentityInterface->Tick(DeltaTime);
	}

	return true;
}

bool FOnlineSubsystemGDK::IsXBLGoldRequired()
{
	bool bXBLGoldRequired = true;
	GConfig->GetBool(TEXT("OnlineSubsystemGDK"), TEXT("bXBLGoldRequired"), bXBLGoldRequired, GEngineIni);
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemGDK::IsXBLGoldRequired() - bXBLGoldRequired=%s"), *LexToString(bXBLGoldRequired));

	return bXBLGoldRequired;
}

bool FOnlineSubsystemGDK::Init()
{
#if WITH_EDITOR
	if(bIsInitialized && GIsEditor)
	{
		return true;
	}
#endif

	const bool bGDKInit = true;

	if (bGDKInit)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XGameGetXboxTitleId, XSystemGetXboxLiveSandboxId, XTaskQueueCreate and XblInitialize are not safe to call on time-sensitive threads (XblInitialize ultimately calls multiple unsafe XTaskQueue functions)

		// Create the online async task thread
		OnlineAsyncTaskThreadRunnable = MakeUnique<FOnlineAsyncTaskManagerGDK>(this);

		// Initialize GDK Platform

		TitleId = IGDKRuntimeModule::Get().GetTitleId();
		UE_CLOGF(TitleId==0,LogOnline, Warning, "Failed to get TitleID : XBOX LIVE SERVICES WILL NOT BE AVAILABLE. Please check your configuration.");

		ZeroMemory(SandboxId, XSystemXboxLiveSandboxIdMaxBytes);
		FPlatformString::Convert(SandboxId, XSystemXboxLiveSandboxIdMaxBytes, *IGDKRuntimeModule::Get().GetXboxSandboxId());

		auto Alloc = [](size_t Size, HCMemoryType MemoryType)
		{
			return FMemory::Malloc(Size);
		};

		auto Free = [](void* Pointer, HCMemoryType MemoryType)
		{
			FMemory::Free(Pointer);
		};

		//XblMemSetFunctions(Alloc, Free);

		XblTaskQueue.Reset( new FGDKAsyncTaskQueue() );

		XalInitArgs xalInit = {};
#ifndef HC_PLATFORM_GSDK //WMM TODO what does this define do?
		xalInit.titleId = TitleId;
		xalInit.sandbox = Sandbox;
#endif
		HRESULT Result = XalInitialize(&xalInit, XblTaskQueue->GetQueue());
		check(SUCCEEDED(Result));

		InitializeXblByConfig();

		OnlineAsyncTaskThread.Reset(FRunnableThread::Create(OnlineAsyncTaskThreadRunnable.Get(), *FString::Printf(TEXT("OnlineAsyncTaskThread %s"), *InstanceName.ToString())));
		check(OnlineAsyncTaskThread.IsValid());

		UE_LOG_ONLINE(Verbose, TEXT("Created thread (ID:%d)."), OnlineAsyncTaskThread->GetThreadID());

		SessionMessageRouterInterface = MakeShared<FSessionMessageRouter, ESPMode::ThreadSafe>(this);
		MatchmakingInterfaceGDK = MakeShared<FOnlineMatchmakingInterfaceGDK, ESPMode::ThreadSafe>(this);
		IdentityInterface = MakeShared<FOnlineIdentityGDK, ESPMode::ThreadSafe>(this);
		// CDATODO this is happening before the userManger it uses is initialised. It get repeated post init, Need to check it doest cause issues on consoles. IdentityInterface->RefreshGamepadsAndUsers();
		StoreInterface = MakeShared<FOnlineStoreGDK, ESPMode::ThreadSafe>(this);
		PurchaseInterface = MakeShared<FOnlinePurchaseGDK, ESPMode::ThreadSafe>(this);
		//PurchaseInterface->RegisterGDKPurchaseHooks();
		SessionInterface = MakeShared<FOnlineSessionGDK, ESPMode::ThreadSafe>(this);
		FriendInterface = MakeShared<FOnlineFriendsGDK, ESPMode::ThreadSafe>(this);
		MessageSanitizer = MakeShared<FMessageSanitizerGDK, ESPMode::ThreadSafe>(this);
		// 		UserCloudInterface = MakeShared<FOnlineUserCloudGDK, ESPMode::ThreadSafe>(this);
		LeaderboardsInterface = MakeShared<FOnlineLeaderboardsGDK, ESPMode::ThreadSafe>(this);
#if WITH_ENGINE
		VoiceInterface = MakeShared<FOnlineVoiceGDK, ESPMode::ThreadSafe>(this);
		if (!VoiceInterface->Init())
		{
			// Disable voice if we fail to init
			VoiceInterface.Reset();
		}
#endif //WITH_ENGINE
		ExternalUIInterface = MakeShared<FOnlineExternalUIGDK, ESPMode::ThreadSafe>(this);
		EventsInterface = MakeShared<FOnlineEventsGDK, ESPMode::ThreadSafe>(this);
		AchievementInterface = MakeShared<FOnlineAchievementsGDK, ESPMode::ThreadSafe>(this);
		StatsInterface = MakeShared<FOnlineStatsGDK, ESPMode::ThreadSafe>(this);
		PresenceInterface = MakeShared<FOnlinePresenceGDK, ESPMode::ThreadSafe>(this);
		UserInterface = MakeShared<FOnlineUserGDK, ESPMode::ThreadSafe>(this);

		bHasCalledNetworkStatusChangedAtLeastOnce = false;

		TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> LambdaWeakThis = AsShared();

		// WMM - Figure out what we do here for not RT 

		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FOnlineSubsystemGDK::HandleAppResume);


		UserLoginChanged = FCoreDelegates::OnUserLoginChangedEvent.AddRaw(this, &FOnlineSubsystemGDK::OnUserLoginChange);

		// CDA do this post init as the subsys is pre init, and engin init will set default port from the base ini that will get used in the first tick when creating the default URL. 
		// Relative travel afterwards will preserve the bad port for the first host.
		AppInitComplete = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FOnlineSubsystemGDK::RefreshNetworkConnectivityLevel);

		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
		{
			auto NetworkConnectivityHintChangedCallback =
				[](void* Context, const XNetworkingConnectivityHint* ConnectivityHint)
			{
				check(ConnectivityHint != nullptr);
				FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);
				pThis->OnNetworkConnectivityHintChanged(*ConnectivityHint);
			};

			HRESULT RegisterNetworkConnectivityChangeResult = XNetworkingRegisterConnectivityHintChanged(FGDKAsyncTaskQueue::GetGenericQueue(), this, NetworkConnectivityHintChangedCallback, &NetworkConnectivityHandle.Emplace());
			if (FAILED(RegisterNetworkConnectivityChangeResult))
			{
				// We didn't actually initialize this value so reset it here
				NetworkConnectivityHandle.Reset();
			}
		}

		OnNetworkRequestDTLSCertificate.BindLambda(
			[LambdaWeakThis](FGuid ConnectionId, const FOnDTLSCertificateReceived Delegate)
		{
			TSharedPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> StrongThis = LambdaWeakThis.Pin();
			if (StrongThis.IsValid())
			{
				StrongThis->HandleNetworkRequestDTLSCertificate(ConnectionId, Delegate);
			}
		});

		OnNetworkGeneratedDTLSCertificate.BindLambda(
			[LambdaWeakThis](FGuid ConnectionId, const TArray<uint8> Thumbprint)
		{
			TSharedPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> StrongThis = LambdaWeakThis.Pin();
			if (StrongThis.IsValid())
			{
				StrongThis->HandleNetworkGeneratedDTLSCertificate(ConnectionId, Thumbprint, true);
			}
		});

		ServiceCallRoutedHandlerContext = XblAddServiceCallRoutedHandler(
			[](XblServiceCallRoutedArgs args, void* context)
			{
				const char* HttpMethod = nullptr;
				const char* HttpUrl = nullptr;
				HCHttpCallRequestGetUrl(args.call, &HttpMethod, &HttpUrl);

				const char* HttpRequestBody = nullptr;
				HCHttpCallRequestGetRequestBodyString(args.call, &HttpRequestBody);

				UE_LOG_ONLINE(Verbose, TEXT("[URL]: %s %s"), UTF8_TO_TCHAR(HttpMethod), UTF8_TO_TCHAR(HttpUrl));
				if (HttpRequestBody)
				{
					UE_LOG_ONLINE(Verbose, TEXT("[RequestBody]: %s"), UTF8_TO_TCHAR(HttpRequestBody));
				}
				UE_LOG_ONLINE(Verbose, TEXT(""));
				UE_LOG_ONLINE(Verbose, TEXT("[Response]: %s"), UTF8_TO_TCHAR(args.fullResponseFormatted));
				UE_LOG_ONLINE(Verbose, TEXT(""));
			}
			, nullptr);

		// create contexts for all users we may have missed
		for (FGDKUserHandle& User : IGDKRuntimeModule::Get().GetAllUserHandles())
		{
			CreateGDKContext(User);
		}

#if WITH_EDITOR
		bIsInitialized = true;
#endif

#ifdef UE_PLAYFAB_MATCHMAKING
		// once we have been registered, it is safe to load our dependent modules
		OnOnlineSubsystemCreated = FOnlineSubsystemDelegates::OnOnlineSubsystemCreated.AddLambda([this](IOnlineSubsystem* NewSubsystem)
		{
			if (NewSubsystem == this)
			{
				FModuleManager::Get().LoadModule(TEXT("PlayFabParty"));
			}
		});
#endif
	}
	else
	{
		Shutdown();
	}
	return bGDKInit;
}

void FOnlineSubsystemGDK::OnNetworkConnectivityHintChanged(const XNetworkingConnectivityHint& ConnectivityHint)
{
	UE_LOG_ONLINE(Log, TEXT("ConnectivityHintChanged: %s"), LexToString(ConnectivityHint.connectivityLevel));

	if (!UE::OnlineSubsystemGDK::bTreatConnectivityUnknownAsDisconnected && ConnectivityHint.connectivityLevel == XNetworkingConnectivityLevelHint::Unknown)
	{
		return;
	}
	if (_GRDK_EDITION <= 220300) // Contexts do not need to be regenerated in later GDK versions
	{
		if (!bShouldRestoreConnectivity && ConnectivityHint.connectivityLevel != XNetworkingConnectivityLevelHint::InternetAccess)
		{
			CleanupGDKContextForNetworkConnectivityLoss();
			bShouldRestoreConnectivity = true;
		}
		else if (bShouldRestoreConnectivity && ConnectivityHint.connectivityLevel == XNetworkingConnectivityLevelHint::InternetAccess)
		{
			ReinitializeGDKContextForNetworkConnectivityRestored();
			bShouldRestoreConnectivity = false;
		}
	}

	XNetworkingConnectivityHint ModifiedConnectivityHint = ConnectivityHint;
	if (ModifiedConnectivityHint.connectivityLevel == XNetworkingConnectivityLevelHint::Unknown)
	{
		// bTreatConnectivityUnknownAsDisconnected is true, otherwise the early out in the top 
		// of this function would have run. Setting connectivityLevel to "None" so the 
		// OnConnectionStatusChanged delegate will be triggered from 
		// ApplyNetworkConnectivityLevel -> FAsyncEventConnectionStatusChanged::TriggerDelegates
		ModifiedConnectivityHint.connectivityLevel = XNetworkingConnectivityLevelHint::None;
		UE_LOG_ONLINE(Log, TEXT("Modified ConnectivityHint from: %s to: %s"), LexToString(ConnectivityHint.connectivityLevel), LexToString(ModifiedConnectivityHint.connectivityLevel));
	}

	ApplyNetworkConnectivityLevel(ModifiedConnectivityHint);
}

void FOnlineSubsystemGDK::CleanupGDKContextForNetworkConnectivityLoss()
{
	UE_LOG_ONLINE(Log, TEXT("CleanupGDKContextForNetworkConnectivityLoss"));

	for (FGDKUserHandle User : IdentityInterface->GetCachedUsers())
	{
		DeleteGDKContext(User);
	}

	AsyncGDKTask([](XAsyncBlock* AsyncBlock) { return XblCleanupAsync(AsyncBlock); });
}

void FOnlineSubsystemGDK::ReinitializeGDKContextForNetworkConnectivityRestored()
{
	UE_LOG_ONLINE(Log, TEXT("ReinitializeGDKContextForNetworkConnectivityRestored"));

	InitializeXblByConfig();

	for (FGDKUserHandle User : IdentityInterface->GetCachedUsers())
	{
		CreateGDKContext(User);
	}
}

void FOnlineSubsystemGDK::InitializeXblByConfig()
{
	UTF8CHAR ServiceConfigurationID[XBL_SCID_LENGTH];
	ZeroMemory(ServiceConfigurationID, XBL_SCID_LENGTH);
	FPlatformString::Convert(ServiceConfigurationID, XBL_SCID_LENGTH, *IGDKRuntimeModule::Get().GetPrimaryServiceConfigId());

	XblInitArgs xblInit = { XblTaskQueue->GetQueue(), (const char*)ServiceConfigurationID };
	HRESULT Result = XblInitialize(&xblInit);
	check(SUCCEEDED(Result));
}

void FOnlineSubsystemGDK::HandleNetworkRequestDTLSCertificate(FGuid ConnectionId, FOnDTLSCertificateReceived Delegate)
{
	FScopeLock Lock(&DTLSDictionariesLock);
	if (auto ThumbprintInfo = DTLSCertificateDictionary.Find(ConnectionId))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGDK_HandleNetworkRequestDTLSCertificate_Delegate);
		Delegate.ExecuteIfBound(ConnectionId, *ThumbprintInfo);
	}
	else
	{
		PendingDTLSCertificateRequests.Add(ConnectionId, Delegate);
	}
}

void FOnlineSubsystemGDK::HandleNetworkRecievedDTLSCertificate(FGuid ConnectionId, const TArray<uint8> Thumbprint)
{
	FScopeLock Lock(&DTLSDictionariesLock);
	DTLSCertificateDictionary.Add(ConnectionId, Thumbprint);

	FOnDTLSCertificateReceived Delegate;
	if (PendingDTLSCertificateRequests.RemoveAndCopyValue(ConnectionId, Delegate))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGDK_HandleNetworkRecievedDTLSCertificate_Delegate);
		Delegate.ExecuteIfBound(ConnectionId, Thumbprint);
	}
}

void FOnlineSubsystemGDK::HandleNetworkGeneratedDTLSCertificate(FGuid ConnectionId, const TArray<uint8> Thumbprint, bool WriteToService)
{
	FScopeLock Lock(&DTLSDictionariesLock);

	FOnDTLSCertificateReceived Delegate;
	if (PendingDTLSCertificateRequests.RemoveAndCopyValue(ConnectionId, Delegate))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGDK_HandleNetworkGeneratedDTLSCertificate_Delegate);
		Delegate.ExecuteIfBound(ConnectionId, Thumbprint);
	}

	DTLSCertificateDictionary.Add(ConnectionId, Thumbprint);

	// TODO: Seems this function HandleNetworkGeneratedDTLSCertificate not used

	if (WriteToService)
	{
		SessionInterface->GetMpsdImpl()->WriteDTLSCertificatesToService(DTLSCertificateDictionary);
	}
}

void FOnlineSubsystemGDK::RecreateGDKContextOnSubscriptionLost()
{
	if (bShouldRestoreConnectivity)
	{
		// Just in case there is internet access, but the XNetworkingRegisterConnectivityHintChanged is not invoked yet with InternetAccess
		return;
	}

	XNetworkingConnectivityHint ConnectivityHint;
	ConnectivityHint.connectivityLevel = XNetworkingConnectivityLevelHint::None;
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
	{
		XNetworkingGetConnectivityHint(&ConnectivityHint);
	}

	if (ConnectivityHint.connectivityLevel != XNetworkingConnectivityLevelHint::InternetAccess)
	{
		// Do nothing, leave the xbl context recreation to XNetworkingRegisterConnectivityHintChanged
		return;
	}

	UE_LOG_ONLINE(Log, TEXT("FOnlineSubsystemGDK::RecreateGDKContextOnSubscriptionLost - Recreating GDKContext..."));

	// If there is still internet access, then the game lost connection to RTA for some other reasons, recreate 
	// the context handle here to handle that
	for (const FGDKUserHandle& User : IdentityInterface->GetCachedUsers())
	{
		DeleteGDKContext(User);
		CreateGDKContext(User);
	}
}

void FOnlineSubsystemGDK::EnableSessionEventHandlers(FGDKContextHandle& GDKContext)
{
	FScopeLock ScopeLock(&GDKContextsLock);

	for (TPair<uint64,FGDKContextInfo>& GDKContextInfoPair : CachedGDKContexts)
	{
		if (GDKContextInfoPair.Value.Handle == GDKContext)
		{
			EnableSessionEventHandlers(GDKContextInfoPair.Value);
			return;
		}
	}
}

void FOnlineSubsystemGDK::EnableSessionEventHandlers(FGDKContextInfo& ContextInfo)
{
	FGDKContextHandle& GDKContext = ContextInfo.Handle;

	if (!XblMultiplayerSubscriptionsEnabled(GDKContext))
	{
		XblMultiplayerSetSubscriptionsEnabled(GDKContext, true);
	}

	if (ContextInfo.SessionChangedContext == INVALID_XBL_FUNCTION_CONTEXT)
	{
		auto SessionChanged = [](void* Context, XblMultiplayerSessionChangeEventArgs EventArgs)
		{
			FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);
			pThis->SessionMessageRouterInterface->OnMultiplayerSessionChanged((const XblMultiplayerSessionChangeEventArgs&)EventArgs);
		};

		ContextInfo.SessionChangedContext = XblMultiplayerAddSessionChangedHandler(GDKContext, SessionChanged, this);
	}

	if (ContextInfo.ConnectionIdChangedContext == INVALID_XBL_FUNCTION_CONTEXT)
	{
		auto ConnectionIdChanged = [](void* Context)
		{
			FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);

			pThis->ExecuteNextTick([pThis]()
				{
					pThis->SessionMessageRouterInterface->OnMultiplayerConnectionIdChanged();
				});
		};

		ContextInfo.ConnectionIdChangedContext = XblMultiplayerAddConnectionIdChangedHandler(GDKContext, ConnectionIdChanged, this);
	}

	if (ContextInfo.SubscriptionLostContext == INVALID_XBL_FUNCTION_CONTEXT)
	{
		auto SubscriptionLost = [](void* Context)
		{
			FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);
			pThis->ExecuteNextTick([pThis]()
				{
					pThis->SessionMessageRouterInterface->OnMultiplayerSubscriptionLost();
				});
		};

		ContextInfo.SubscriptionLostContext = XblMultiplayerAddSubscriptionLostHandler(GDKContext, SubscriptionLost, this);
	}
}

void FOnlineSubsystemGDK::DisableSessionEventHandlers(FGDKContextInfo& ContextInfo) const
{
	if (ContextInfo.SessionChangedContext != INVALID_XBL_FUNCTION_CONTEXT)
	{
		XblMultiplayerRemoveSessionChangedHandler(ContextInfo.Handle, ContextInfo.SessionChangedContext);
	}

	if (ContextInfo.ConnectionIdChangedContext != INVALID_XBL_FUNCTION_CONTEXT)
	{
		XblMultiplayerRemoveConnectionIdChangedHandler(ContextInfo.Handle, ContextInfo.ConnectionIdChangedContext);
	}

	if (ContextInfo.SubscriptionLostContext != INVALID_XBL_FUNCTION_CONTEXT)
	{
		XblMultiplayerRemoveSubscriptionLostHandler(ContextInfo.Handle, ContextInfo.SubscriptionLostContext);
	}

	if (XblMultiplayerSubscriptionsEnabled(ContextInfo.Handle))
	{
		XblMultiplayerSetSubscriptionsEnabled(ContextInfo.Handle, false);
	}
}

TMap<FGuid, TArray<uint8>> FOnlineSubsystemGDK::GetCertificateDictionary() const
{
	FScopeLock Lock(&DTLSDictionariesLock);
	return DTLSCertificateDictionary;
}

void FOnlineSubsystemGDK::DeleteGDKContext(FGDKUserHandle GDKUser)
{
	FScopeLock ScopeLock(&GDKContextsLock);

	uint64_t Xuid;
	if( FAILED(XalUserGetId(GDKUser, &Xuid)) )
	{
		UE_LOG_ONLINE(Error, TEXT("FOnlineSubsystemGDK::DeleteGDKContext - can't get xal for user"));
		return;
	}

	for (UserContextWrapperPtr UserContextWrapper : UserContextWrappers)
	{
		if (UserContextWrapper->UserId == Xuid)
		{
			UserContextWrappers.Remove(UserContextWrapper);
			break;
		}
	}

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(Xuid);
	if (ensure(GDKContextInfoPtr))
	{
		DeleteGDKContextInternal(Xuid, *GDKContextInfoPtr);
		CachedGDKContexts.Remove(Xuid);
	}
}

void FOnlineSubsystemGDK::DeleteGDKContextInternal(uint64 Xuid, FGDKContextInfo& ContextInfo) const
{
	// We'll unsubscribe from all presence updates for this user (context) before we unbind the corresponding delegates
	const FOnlinePresenceGDKPtr PresencePtr = GetPresenceGDK();
	if (PresencePtr.IsValid())
	{
		PresencePtr->UnsubscribeFromAllPresenceUpdatesForUser(ContextInfo.Handle);
		PresencePtr->ClearAllStatUpdateSubscriptionsForUser(FUniqueNetIdGDK::Create(Xuid));
	}
	XblUserStatisticsRemoveStatisticChangedHandler(ContextInfo.Handle, ContextInfo.StatisticChangedContext);
	XblPresenceRemoveTitlePresenceChangedHandler(ContextInfo.Handle, ContextInfo.TitlePresenceChangedContext);
	XblPresenceRemoveDevicePresenceChangedHandler(ContextInfo.Handle, ContextInfo.DevicePresenceChangedContext);
	XblSocialRemoveSocialRelationshipChangedHandler(ContextInfo.Handle, ContextInfo.RelationshipChangedContext);

	DisableSessionEventHandlers(ContextInfo);

	ContextInfo.Handle.Clear();
}

void FOnlineSubsystemGDK::OnUserLoginChange(bool bIsSignIn, int32 PlatformUserId, int32 UserIndex )
{
	if (ensure(IdentityInterface.IsValid()))
	{
		const FGDKUserHandle UserHandle = IdentityInterface->GetUserForPlatformUserId(PlatformUserId);
		if (ensure(UserHandle.IsValid()))
		{
			if (bIsSignIn)
			{
				CreateGDKContext(UserHandle);
			}
			else
			{
				DeleteGDKContext(UserHandle);
			}
		}

		IdentityInterface->OnUserLoginChange(bIsSignIn, PlatformUserId, UserIndex);
	}
}


void FOnlineSubsystemGDK::HandleAppResume()
{
	// After resuming from suspend, the cached XboxLiveContexts are invalid. Clear them,
	// and they will be re-created on demand in GetLiveContext().
	// WMM Add this in: FreeLiveContextCache();
	RefreshNetworkConnectivityLevel();
}

bool FOnlineSubsystemGDK::Shutdown()
{
#if WITH_EDITOR
	if (!bIsInitialized && GIsEditor)
	{
		return true;
	}
	bIsInitialized = false;
#endif
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XNetworkingUnregisterConnectivityHintChanged and XalUserUnregisterChangeEventHandler(calls XUserUnregisterForChangeEvent) are not safe to call on time-sensitive threads

	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemGDK::Shutdown()"));

	XblRemoveServiceCallRoutedHandler(ServiceCallRoutedHandlerContext);
	ServiceCallRoutedHandlerContext = INVALID_XBL_FUNCTION_CONTEXT;


	IGDKRuntimeModule* GDKRuntimeModule = IGDKRuntimeModule::TryGet();
	bool bGRDKAvailable = GDKRuntimeModule && GDKRuntimeModule->IsAvailable();

	if (bGRDKAvailable && XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
	{
		if (NetworkConnectivityHandle.IsSet())
		{
			bool bWait = true;
			XNetworkingUnregisterConnectivityHintChanged(NetworkConnectivityHandle.GetValue(), bWait);
			NetworkConnectivityHandle.Reset();
		}
	}

	OnNetworkRequestDTLSCertificate.Unbind();
	OnNetworkGeneratedDTLSCertificate.Unbind();

	FCoreDelegates::OnUserLoginChangedEvent.Remove(UserLoginChanged);
	FCoreDelegates::OnFEngineLoopInitComplete.Remove(AppInitComplete);

	if (bGRDKAvailable && XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
	{
		// WMM TODO: Why doesn't this compile?
		//CancelMibChangeNotify2(NetworkConnectivityChangedHandle);
	}

	if (bGRDKAvailable && XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XUser))
	{
		XalUserUnregisterChangeEventHandler(OnUserStateChanged);
	}

	FOnlineSubsystemImpl::Shutdown();

	if (OnlineAsyncTaskThread.IsValid())
	{
		// Destroy the online async task thread
		OnlineAsyncTaskThread->Kill(true);
		OnlineAsyncTaskThread.Reset();
	}

	if (OnlineAsyncTaskThreadRunnable.IsValid())
	{
		OnlineAsyncTaskThreadRunnable.Reset();
	}

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = nullptr; \
	}

	// Destruct the interfaces (in opposite order they were created)
	DESTRUCT_INTERFACE(UserInterface);
	DESTRUCT_INTERFACE(PresenceInterface);
	DESTRUCT_INTERFACE(AchievementInterface);
	DESTRUCT_INTERFACE(StatsInterface);
	DESTRUCT_INTERFACE(EventsInterface);
	DESTRUCT_INTERFACE(ExternalUIInterface);
	DESTRUCT_INTERFACE(VoiceInterface);
	DESTRUCT_INTERFACE(LeaderboardsInterface);
	DESTRUCT_INTERFACE(FriendInterface);
	DESTRUCT_INTERFACE(MessageSanitizer);
	DESTRUCT_INTERFACE(SessionInterface);
	DESTRUCT_INTERFACE(PurchaseInterface);
	DESTRUCT_INTERFACE(StoreInterface);
	DESTRUCT_INTERFACE(IdentityInterface);
	DESTRUCT_INTERFACE(MatchmakingInterfaceGDK);
	DESTRUCT_INTERFACE(SessionMessageRouterInterface);

#undef DESTRUCT_INTERFACE

	for (TPair<uint64,FGDKContextInfo>& GDKContextInfo : CachedGDKContexts)
	{
		DeleteGDKContextInternal(GDKContextInfo.Key, GDKContextInfo.Value);
	}
	CachedGDKContexts.Reset();

	if (bGRDKAvailable && XblTaskQueue.IsValid() && XblTaskQueue->GetQueue() != nullptr)
	{
		XAsyncBlock AsyncBlockXbl = {0};
		AsyncBlockXbl.queue = XblTaskQueue->GetQueue();
		XblCleanupAsync(&AsyncBlockXbl);
		XblTaskQueue->BlockUntilComplete(AsyncBlockXbl);

		XAsyncBlock AsyncBlockXal = {0};
		AsyncBlockXal.queue = XblTaskQueue->GetQueue();
		XalCleanupAsync(&AsyncBlockXal);
		XblTaskQueue->BlockUntilComplete(AsyncBlockXal);

		XblTaskQueue->CancelPendingTasksAndDestroyQueue();
	}

#ifdef UE_PLAYFAB_MATCHMAKING
	FOnlineSubsystemDelegates::OnOnlineSubsystemCreated.Remove(OnOnlineSubsystemCreated);
#endif

	return true;
}

FString FOnlineSubsystemGDK::GetAppId() const
{
	static const FString TitleIdStr(FString::Printf(TEXT("%d"), TitleId));
	return TitleIdStr;
}

bool FOnlineSubsystemGDK::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}

	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("STATS")))
	{
		if (StatsInterface.IsValid())
		{
			bWasHandled = StatsInterface->HandleExec(InWorld, Cmd, Ar);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("SANITIZESTRING")))
	{
		bWasHandled = HandleSanitizeStringExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("SANITIZESTRINGS")))
	{
		bWasHandled = HandleSanitizeStringsExecCommands(InWorld, Cmd, Ar);
	}

	return bWasHandled;
}

bool FOnlineSubsystemGDK::HandleSanitizeStringExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	FString StringToSanitize(Cmd);
	if (!StringToSanitize.IsEmpty())
	{
		MessageSanitizer->SanitizeDisplayName(StringToSanitize, FOnMessageProcessed::CreateLambda([this, StringToSanitize](bool Success, const FString& SanitizedMessage)
		{
			UE_LOG_ONLINE(Verbose, TEXT(" Sanitized String: %s ---> %s"), *StringToSanitize, *SanitizedMessage);
		}));
		bWasHandled = true;
	}
	return bWasHandled;
}

bool FOnlineSubsystemGDK::HandleSanitizeStringsExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	TArray<FString> ArrayToSanitize;
	for (FString StringToSanitize = FParse::Token(Cmd, false); !StringToSanitize.IsEmpty(); StringToSanitize = FParse::Token(Cmd, false))
	{
		ArrayToSanitize.Add(StringToSanitize);
	}
	if (ArrayToSanitize.Num() > 0)
	{
		MessageSanitizer->SanitizeDisplayNames(ArrayToSanitize, FOnMessageArrayProcessed::CreateLambda([this, ArrayToSanitize](bool Success, const TArray<FString>& SanitizedArrayMessage)
		{
			for (int i=0; i < SanitizedArrayMessage.Num(); i++)
			{
				UE_LOG_ONLINE(Verbose, TEXT(" Sanitized String: %s ----> %s"), *ArrayToSanitize[i] , *SanitizedArrayMessage[i]);
			}
		}));
		bWasHandled = true;
	}
	return bWasHandled;
}

FText FOnlineSubsystemGDK::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemGDK", "OnlineServiceName", "Xbox network");
}

FText FOnlineSubsystemGDK::GetSocialPlatformName() const
{
	return NSLOCTEXT("OnlineSubsystemGDK", "SocialPlatformName", "Xbox network");
}

const FString& FOnlineSubsystemGDK::GetTitleProductId() const
{
	static FString TitleProductId;
	if (TitleProductId.IsEmpty())
	{
		GConfig->GetString(TEXT("/Script/GDKPlatformEditor.GDKTargetSettings"), TEXT("ProductId"), TitleProductId, GEngineIni); //??
	}

	return TitleProductId;
}

FOnlineAsyncTaskManagerGDK* FOnlineSubsystemGDK::GetAsyncTaskManager()
{
	check(OnlineAsyncTaskThreadRunnable != nullptr);

	return OnlineAsyncTaskThreadRunnable.Get();
}

void FOnlineSubsystemGDK::QueueAsyncTask(FOnlineAsyncTask* const AsyncTask, const bool bCanRunInParallel)
{
	check(OnlineAsyncTaskThreadRunnable);

	if (bCanRunInParallel)
	{
		OnlineAsyncTaskThreadRunnable->AddToParallelTasks(AsyncTask);
	}
	else
	{
		OnlineAsyncTaskThreadRunnable->AddToInQueue(AsyncTask);
	}
}

void FOnlineSubsystemGDK::QueueAsyncEvent(FOnlineAsyncEvent<FOnlineSubsystemGDK>* const AsyncEvent)
{
	check(OnlineAsyncTaskThreadRunnable);

	OnlineAsyncTaskThreadRunnable->AddToOutQueue(AsyncEvent);
}

/* // WMM - determine where this is used
Platform::String& FOnlineSubsystemGDK::RemoveBracesFromGuidString( __in Platform::String& guid )
{
	std::wstring strGuid = guid->ToString()->Data();

	if(strGuid.length() > 0 && strGuid[0] == L'{')
	{
		// Remove the {
		strGuid.erase(0, 1);
	}
	if(strGuid.length() > 0 && strGuid[strGuid.length() - 1] == L'}')
	{
		// Remove the }
		strGuid.erase(strGuid.end() - 1, strGuid.end());
	}

	return ref new Platform::String(strGuid.c_str());
}
*/

FGDKContextHandle FOnlineSubsystemGDK::GetGDKContext(int32 LocalUserNum)
{
	if (!IdentityInterface.IsValid())
	{
		return FGDKContextHandle();
	}

	auto GDKUser = IdentityInterface->GetUserForPlatformUserId(LocalUserNum);
	if (!GDKUser)
	{
		return FGDKContextHandle();
	}

	return GetGDKContext(GDKUser);
}

FGDKContextHandle FOnlineSubsystemGDK::GetGDKContext(uint64 GDKUserId)
{
	FScopeLock ScopeLock(&GDKContextsLock);

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(GDKUserId);
	if (GDKContextInfoPtr)
	{
		check(GDKContextInfoPtr->Handle.IsValid());
		return GDKContextInfoPtr->Handle;
	}

	UE_LOG_ONLINE(Warning, TEXT("There is no GDKContext for user xuid passed into FOnlineSubsystemGDK::GetGDKContext - maybe user has already signed out"));
	return FGDKContextHandle();
}

FGDKContextHandle FOnlineSubsystemGDK::GetGDKContext(const FUniqueNetId& UserId)
{
	uint64 XUserId = static_cast<const FUniqueNetIdGDK&>(UserId).ToUint64();
	return GetGDKContext(XUserId);
}

FGDKContextHandle FOnlineSubsystemGDK::GetGDKContext(FGDKMultiplayerSessionHandle Session)
{
	const XblMultiplayerSessionMember* CurrentUser = XblMultiplayerSessionCurrentUser(Session);
	if(!CurrentUser)
	{
		return FGDKContextHandle();
	}

	return GetGDKContext(CurrentUser->Xuid);
}

FGDKContextHandle FOnlineSubsystemGDK::GetGDKContext(FGDKUserHandle GDKUser)
{
	if (!GDKUser.IsValid())
	{
		UE_LOG_ONLINE(Error, TEXT("Invalid GDKUser passed into FOnlineSubsystemGDK::GetGDKContext"));
		return FGDKContextHandle();
	}

	uint64 UserId = 0;
	HRESULT Result = XUserGetId(GDKUser, &UserId);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Cannot get xuid for GDKUser passed into FOnlineSubsystemGDK::GetGDKContext"));
		return FGDKContextHandle();
	}

	return GetGDKContext(UserId);
}

FGDKContextHandle FOnlineSubsystemGDK::CreateGDKContext(FGDKUserHandle GDKUser)
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XblContextCreateHandle & XblMultiplayerSetSubscriptionsEnabled ultimately call several XTaskQueue functions which are not safe to call on time-sensitive threads

	checkf(XblTaskQueue->GetQueue() != nullptr, TEXT("attempting to create GDK context after shutdown"));

	FScopeLock ScopeLock(&GDKContextsLock);

	uint64 Xuid = 0;
	HRESULT Result = XUserGetId(GDKUser, &Xuid);
	if (Result != S_OK)
	{
		UE_LOG_ONLINE(Error, TEXT("Invalid GDKUser passed into FOnlineSubsystemGDK::CreateGDKContext"));
		return FGDKContextHandle();
	}

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(Xuid);
	if (ensure(GDKContextInfoPtr == nullptr))
	{
		FGDKContextHandle GDKContext;
		TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> LambdaWeakThis = AsShared();

		Result = XblContextCreateHandle(GDKUser, GDKContext.GetInitReference());
		if (GDKContext)
		{
			FGDKContextInfo& ContextInfo = CachedGDKContexts.Emplace(Xuid, GDKContext);
			
			// Register for friends updates
			auto SocialRelationshipsChanged = [](const XblSocialRelationshipChangeEventArgs* EventArgs, void* Context)
			{
				FUniqueNetIdGDKRef GDKNetId = FUniqueNetIdGDK::Create(EventArgs->callerXboxUserId);
				UE_LOG_ONLINE(Verbose, TEXT("Received SocialRelationshipChange event for player %s"), *GDKNetId->ToString());

				FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);

				// Call on the game thread
				pThis->ExecuteNextTick([pThis, GDKNetId]()
				{
					const FOnlineIdentityGDKPtr IdentityPtr = pThis->GetIdentityGDK();
					if (!IdentityPtr.IsValid())
					{
						UE_LOG_ONLINE(Warning, TEXT("Received unhandleable SocialRelationshipChange event for player %s"), *GDKNetId->ToString());
						return;
					}

					const int32 LocalUserNum = IdentityPtr->GetPlatformUserIdFromUniqueNetId(*GDKNetId);
					if (LocalUserNum == -1)
					{
						UE_LOG_ONLINE(Warning, TEXT("Received SocialRelationshipChange event for unknown player %s"), *GDKNetId->ToString());
						return;
					}

					// Requery our friends list
					const FOnlineFriendsGDKPtr FriendsPtr = pThis->GetFriendsGDK();
					if (FriendsPtr.IsValid())
					{
						if (!FriendsPtr->ReadFriendsList(LocalUserNum, EFriendsLists::ToString(EFriendsLists::Default), FOnReadFriendsListComplete()))
						{
							UE_LOG_ONLINE(Warning, TEXT("Failed to requery friends list for player %s after SocialRelationshipChange"), *GDKNetId->ToString());
						}
					}
				});
			};

			// Register for friends updates
			ContextInfo.RelationshipChangedContext = XblSocialAddSocialRelationshipChangedHandler(GDKContext, SocialRelationshipsChanged, this);



			auto DevicePresenceChangedHandler = [](void* Context, uint64_t xuid, XblPresenceDeviceType deviceType, bool isUserLoggedOnDevice)
			{
				FUniqueNetIdGDKRef GDKNetId = FUniqueNetIdGDK::Create(xuid);
				UE_LOG_ONLINE(Verbose, TEXT("Received DevicePresenceChanged Event Player:%s DeviceType:%d IsUserLoggedIn:%d"), *GDKNetId->ToString(), EnumToUnderlyingType(deviceType), isUserLoggedOnDevice);

				FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(Context);
				pThis->ExecuteNextTick([pThis, xuid, deviceType, isUserLoggedOnDevice, GDKNetId]()
				{
					const FOnlinePresenceGDKPtr PresencePtr = pThis->GetPresenceGDK();
					if (PresencePtr.IsValid())
					{
						PresencePtr->OnPresenceDeviceChanged(*GDKNetId, deviceType, isUserLoggedOnDevice);
					}
				});
			};

			ContextInfo.DevicePresenceChangedContext = XblPresenceAddDevicePresenceChangedHandler(GDKContext, DevicePresenceChangedHandler, this);
		
			auto TitlePresenceChangedHandler = [](void* InContext, uint64 InXuid, uint32 InTitleId, XblPresenceTitleState InTitleState)
			{
				FUniqueNetIdGDKRef GDKNetId = FUniqueNetIdGDK::Create(InXuid);
				UE_LOG_ONLINE(Verbose, TEXT("Received TitlePresenceChanged Event Player: %s TitleId: %u TitleState:%d"), *GDKNetId->ToString(), InTitleId, EnumToUnderlyingType(InTitleState));

				FOnlineSubsystemGDK* pThis = reinterpret_cast<FOnlineSubsystemGDK*>(InContext);
				pThis->ExecuteNextTick([pThis, InXuid, InTitleId, InTitleState, GDKNetId]()
				{
					const FOnlinePresenceGDKPtr PresencePtr = pThis->GetPresenceGDK();
					if (PresencePtr.IsValid())
					{
						PresencePtr->OnPresenceTitleChanged(*GDKNetId, InTitleId, InTitleState);
					}
				});
			};

			ContextInfo.TitlePresenceChangedContext = XblPresenceAddTitlePresenceChangedHandler(GDKContext, TitlePresenceChangedHandler, this);

			// Binding a handler to the stat update event, to process stat updates in the presence interface
			TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> WeakThisLocal = AsShared();

			int32 Idx = UserContextWrappers.Add(MakeShared<UserContextWrapper, ESPMode::ThreadSafe>(WeakThisLocal, Xuid));

			ContextInfo.StatisticChangedContext = XblUserStatisticsAddStatisticChangedHandler(
				GDKContext,
				[](XblStatisticChangeEventArgs statisticChangeEventArgs, void* Context)
				{
					FUniqueNetIdGDKRef GDKNetId = FUniqueNetIdGDK::Create(statisticChangeEventArgs.xboxUserId);
					const FString StatName(UTF8_TO_TCHAR(statisticChangeEventArgs.latestStatistic.statisticName));
					const FString StatValue(UTF8_TO_TCHAR(statisticChangeEventArgs.latestStatistic.value));
					const FString StatType(UTF8_TO_TCHAR(statisticChangeEventArgs.latestStatistic.statisticType));
					UE_LOG_ONLINE(VeryVerbose, TEXT("Received StatisticChanged Event Player:%s StatName: %ls NewValue: %ls"), *GDKNetId->ToString(), *StatName, *StatValue);
					// This delegate will be unbound in DeleteGDKContext before the corresponding entry for UserContextWrappers is deleted, so pThis should always be valid
					UserContextWrapper* pThis = reinterpret_cast<UserContextWrapper*>(Context);

					TWeakPtr<UserContextWrapper, ESPMode::ThreadSafe> WeakThis = pThis->AsShared();

					TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> StrongThis = WeakThis.Pin();

					if (!StrongThis.IsValid() || !StrongThis->Sys.IsValid())
					{
						UE_LOG_ONLINE(Verbose, TEXT("Ignoring StatisticChanged as we went away"));
						return;
					}

					TSharedPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> Subsystem = StrongThis->Sys.Pin();

					Subsystem->ExecuteNextTick([WeakThis, GDKNetId, StatName, StatValue, StatType]()
						{
							TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> StrongThis = WeakThis.Pin();
							if (!StrongThis.IsValid() || !StrongThis->Sys.IsValid())
							{
								UE_LOG_ONLINE(Verbose, TEXT("Ignoring StatisticChanged as we went away"));
								return;
							}

							TSharedPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> Subsystem = StrongThis->Sys.Pin();
							if (FOnlinePresenceGDKPtr PresencePtr = Subsystem->GetPresenceGDK())
							{
								// Sending the source net ID as a string and creating here (instead of creating the shared ref up top and capturing that down)
								// as ID shared refs aren't thread safe and this callback in particular is prone to issues with a null SourceNetId if captured by shared ref.
								FUniqueNetIdGDKRef SourceNetId = FUniqueNetIdGDK::Create(StrongThis->UserId);
								PresencePtr->ProcessStatUpdate(SourceNetId, GDKNetId, StatName, StatValue, StatType);
							}

							// If we were to trigger functionality on the Stats Interface, this would be a good place
						});

				},
				UserContextWrappers[Idx].Get());

			return GDKContext;
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("GDKContext creation failed. Error: %d."), Result);
			return FGDKContextHandle();
		}
	}
	else
	{
		check(GDKContextInfoPtr->Handle.IsValid());
		return GDKContextInfoPtr->Handle;
	}
}

FGDKContextHandle FOnlineSubsystemGDK::GetFirstValidContext()
{
	FScopeLock ScopeLock(&GDKContextsLock);
	for (auto Pair : CachedGDKContexts)
	{
		if (Pair.Value.Handle.IsValid())
		{
			return Pair.Value.Handle;
		}
	}

	return FGDKContextHandle();
}

void FOnlineSubsystemGDK::CacheGDKSession(const FName& SessionName, FGDKMultiplayerSessionHandle LatestSession)
{
	check(IsInGameThread());

	bool bFoundSession = false;

	FNamedOnlineSessionPtr NamedSession = SessionInterface->GetNamedSessionPtr(SessionName);
	if (NamedSession.IsValid())
	{
		UE_LOG_ONLINE(Log, TEXT("CacheGDKSession() Named Session is valid."));
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		if (GDKInfo.IsValid())
		{
			GDKInfo->RefreshGDKInfo(LatestSession);
			UE_LOG_ONLINE(Log, TEXT("CacheGDKSession() GDK Info is valid."));
			bFoundSession = true;
		}
	}

	if (MatchmakingInterfaceGDK.IsValid())
	{
		FOnlineMatchTicketInfoPtr MatchTicket;
		MatchmakingInterfaceGDK->GetMatchmakingTicket(SessionName, MatchTicket);
		if (MatchTicket.IsValid())
		{
			MatchTicket->RefreshGDKInfo(LatestSession);
			bFoundSession = true;
		}
	}

	UE_CLOGF(!bFoundSession, LogOnline, Log, "Attempted to update GDK Session for session %ls, but found none.", *SessionName.ToString());
}

void FOnlineSubsystemGDK::SetLastDiffedSession(const FName& SessionName, FGDKMultiplayerSessionHandle LatestSession)
{
	// Access interfaces directly through this object
	if (SessionInterface.IsValid())
	{
		FNamedOnlineSessionPtr NamedSession = SessionInterface->GetNamedSessionPtr(SessionName);
		if (NamedSession.IsValid())
		{
			FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
			if (GDKInfo.IsValid())
			{
				GDKInfo->SetLastDiffedMultiplayerSession(LatestSession);
			}
		}
	}

	if (MatchmakingInterfaceGDK.IsValid())
	{
		FOnlineMatchTicketInfoPtr MatchTicket;
		MatchmakingInterfaceGDK->GetMatchmakingTicket(SessionName, MatchTicket);
		if (MatchTicket.IsValid())
		{
			MatchTicket->SetLastDiffedSession(LatestSession);
		}
	}
}

FGDKMultiplayerSessionHandle FOnlineSubsystemGDK::GetLastDiffedSession(const FName& SessionName)
{
	if (SessionInterface.IsValid())
	{
		if (FNamedOnlineSession* NamedSession = SessionInterface->GetNamedSession(SessionName))
		{
			FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
			if (GDKInfo.IsValid())
			{
				return GDKInfo->GetLastDiffedMultiplayerSession();
			}
		}
	}

	if (MatchmakingInterfaceGDK.IsValid())
	{
		FOnlineMatchTicketInfoPtr MatchTicket;
		MatchmakingInterfaceGDK->GetMatchmakingTicket(SessionName, MatchTicket);
		if (MatchTicket.IsValid())
		{
			return MatchTicket->GetLastDiffedSession();
		}
	}

	return FGDKMultiplayerSessionHandle();
}

bool FOnlineSubsystemGDK::AreSessionReferencesEqual(const XblMultiplayerSessionReference* First, const XblMultiplayerSessionReference* Second)
{
	return (FCStringAnsi::Stricmp(First->Scid, Second->Scid) == 0 &&
		FCStringAnsi::Stricmp(First->SessionTemplateName, Second->SessionTemplateName) == 0 &&
		FCStringAnsi::Stricmp(First->SessionName, Second->SessionName) == 0);
}

class FAsyncEventConnectionStatusChanged : public FOnlineAsyncEvent<FOnlineSubsystemGDK>
{
private:
	XNetworkingConnectivityLevelHint ConnectivityLevel;

public:
	FAsyncEventConnectionStatusChanged(FOnlineSubsystemGDK* InGDKSubsystem, XNetworkingConnectivityLevelHint InConnectivityLevel) :
		FOnlineAsyncEvent(InGDKSubsystem),
		ConnectivityLevel(InConnectivityLevel)
	{
	}

	virtual void Finalize() override
	{

	}

	virtual FString ToString() const override
	{
		return FString("FAsyncEventConnectionStatusChanged");
	}

	virtual void TriggerDelegates() override
	{
		EOnlineServerConnectionStatus::Type	ConvertedNetworkConnectivityLevelOnStack = EOnlineServerConnectionStatus::ServiceUnavailable;
		if (ConnectivityLevel == XNetworkingConnectivityLevelHint::InternetAccess)
		{
			ConvertedNetworkConnectivityLevelOnStack = EOnlineServerConnectionStatus::Connected;
		}

		//Must reset Default port, before Network Ready it may be bogus
		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
		{
			FName SubsysemInstanceName = Subsystem->GetInstanceName();
			Subsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort>(Subsystem,
				FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate::CreateLambda([SubsysemInstanceName](bool bSuccess, uint16_t GDKPort)
					{
						if (!bSuccess)
						{
							UE_LOG_ONLINE_SESSION(Warning, TEXT("A CreateAndDispatchAsyncTaskParallel call failed."));
						}
						else
						{
#if WITH_ENGINE							
							FURL::UrlConfig.DefaultPort = GDKPort;

							if (UWorld* World = GetWorldForOnline(SubsysemInstanceName))
							{
								FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(World);
								WorldContext.LastURL.Port = GDKPort;
							}
							UE_LOG_ONLINE(Log, TEXT("Preferred Local Udp Multiplayer Port set to %d. WorldContext LastURL Port updated"), GDKPort);
#endif //WITH_ENGINE
						}
					}));
		}

		const EOnlineServerConnectionStatus::Type OldStatus = Subsystem->ConvertedNetworkConnectivityLevel;

		UE_LOG_ONLINE(Log, TEXT("NetworkStatusChangedEvent: OldConverted: %s, Converted: %s"), EOnlineServerConnectionStatus::ToString(OldStatus), EOnlineServerConnectionStatus::ToString(ConvertedNetworkConnectivityLevelOnStack));

		if (!Subsystem->bHasCalledNetworkStatusChangedAtLeastOnce || ConvertedNetworkConnectivityLevelOnStack != Subsystem->ConvertedNetworkConnectivityLevel)
		{
			Subsystem->bHasCalledNetworkStatusChangedAtLeastOnce = true;
			Subsystem->ConvertedNetworkConnectivityLevel = ConvertedNetworkConnectivityLevelOnStack;

			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGDK_FAsyncEventConnectionStatusChanged_TriggerDelegates);
			Subsystem->TriggerOnConnectionStatusChangedDelegates(Subsystem->GetSubsystemName().ToString(), OldStatus, ConvertedNetworkConnectivityLevelOnStack);
		}
	}
};


void FOnlineSubsystemGDK::RefreshNetworkConnectivityLevel()
{
	XNetworkingConnectivityHint ConnectivityHint;
	ConnectivityHint.connectivityLevel = XNetworkingConnectivityLevelHint::None;
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
	{
		XNetworkingGetConnectivityHint(&ConnectivityHint);
	}

	ApplyNetworkConnectivityLevel(ConnectivityHint);
}


void FOnlineSubsystemGDK::ApplyNetworkConnectivityLevel(const XNetworkingConnectivityHint& ConnectivityHint)
{
	UE_LOG_ONLINE(Log, TEXT("ApplyNetworkConnectivityLevel"));

	if (ConnectivityHint.connectivityLevel != XNetworkingConnectivityLevelHint::Unknown)
	{
		CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort>(this,
			FOnlineAsyncTaskGDKQueryPreferredLocalUdpMultiplayerPort::FOnQueryPreferredLocalUdpMultiplayerPortCompleteDelegate::CreateLambda([&](bool bSuccess, uint16_t GDKPort)
		{
			if (!bSuccess)
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("A CreateAndDispatchAsyncTaskParallel call failed."));
			}
			else
			{
#if WITH_ENGINE
				FURL::UrlConfig.DefaultPort = GDKPort;

				if (UWorld* World = GetWorldForOnline(GetInstanceName()))
				{
					FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(World);
					WorldContext.LastURL.Port = GDKPort;
				}
				UE_LOG_ONLINE(Log, TEXT("Preferred Local Udp Multiplayer Port set to %d. WorldContext LastURL Port updated"), GDKPort);				
#endif //WITH_ENGINE
			}
		}));

		CreateAndDispatchAsyncEvent<FAsyncEventConnectionStatusChanged>(this, ConnectivityHint.connectivityLevel);
	}
	else
	{
		ConvertedNetworkConnectivityLevel = EOnlineServerConnectionStatus::ServiceUnavailable;
	}
}


EOnlineEnvironment::Type FOnlineSubsystemGDK::GetOnlineEnvironment() const
{
	FString SandboxIdString(UTF8_TO_TCHAR(SandboxId));
	TArray<FString> RetailSandboxes;
	GConfig->GetArray(TEXT("OnlineSubsystemGDK"), TEXT("RetailSandboxes"), RetailSandboxes, GEngineIni);
	if (SandboxIdString.Equals(TEXT("RETAIL")) || RetailSandboxes.Contains(SandboxIdString))
	{
		return EOnlineEnvironment::Production;
	}

	TArray<FString> CertSandboxes;
	GConfig->GetArray(TEXT("OnlineSubsystemGDK"), TEXT("CertSandboxes"), CertSandboxes, GEngineIni);
	if (SandboxIdString.StartsWith(TEXT("CERT")) || SandboxIdString.EndsWith(TEXT(".99")) || SandboxIdString.EndsWith(TEXT(".98")) || CertSandboxes.Contains(SandboxIdString))
	{
		return EOnlineEnvironment::Certification;
	}

	return EOnlineEnvironment::Development;
}

#endif //WITH_GRDK
