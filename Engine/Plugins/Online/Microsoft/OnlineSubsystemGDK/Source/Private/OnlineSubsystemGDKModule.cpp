// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_GRDK
#include "HCTraceHandler.h"
#include "OnlineSubsystemGDK.h"
#endif

/**
 * Online subsystem module class  (GDK Implementation)
 * Code related to the loading of the GDK module
 */
class FOnlineSubsystemGDKModule : public IModuleInterface
{
private:
#if WITH_GRDK
	/** Class responsible for creating instance(s) of the subsystem */
	TUniquePtr<IOnlineFactory> GDKFactory;

	// Redirects HC logging to UE logger.
	FHCTraceHandler HCTraceHandler;
#endif //WITH_GRDK

public:
	FOnlineSubsystemGDKModule() = default;
	virtual ~FOnlineSubsystemGDKModule() = default;
	FOnlineSubsystemGDKModule(const FOnlineSubsystemGDKModule& Other) = delete;

	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};

/**
 * Class responsible for creating instance(s) of the subsystem
 */
#if WITH_GRDK
class FOnlineFactoryGDK : public IOnlineFactory
{
private:
	/** Single instantiation of the GDK interface */
	FOnlineSubsystemGDKPtr& GetSingleton() const;

	virtual void DestroySubsystem();

public:
	FOnlineFactoryGDK() {};
	virtual ~FOnlineFactoryGDK();

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName) override;
};






#if WITH_EDITOR

typedef TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> FOnlineSubsystemGDKWeakPtr;

// this is returned when we are in the editor and the caller requests the default instance of the GDK OSS - this is typically the one for the editor itself
// this is necessary because if we are the DefaultPlatformService and return null, it won't try to create us again and fall back to OSS NULL.
// the real GDK OSS is reserved for PIE
class FMockOnlineSubsystemGDK : public FOnlineSubsystemImpl
{
public:
	FMockOnlineSubsystemGDK( FName InstanceName) : FOnlineSubsystemImpl(GDK_SUBSYSTEM, InstanceName)
	{
	}

	virtual FText GetOnlineServiceName() const override
	{
		return FText::GetEmpty();
	}

	virtual FString GetAppId() const override
	{
		return TEXT("");
	}

	IOnlineSessionPtr GetSessionInterface() const
	{
		return nullptr;
	}

	IOnlineFriendsPtr GetFriendsInterface() const
	{
		return nullptr;
	}
	bool Init()
	{
		return true;
	}
};

// this factory is used in the editor so ensure our only-available GDK OSS instance is used for PIE sessions and not by the editor itself
class FOnlineFactoryGDK_ForPIE : public IOnlineFactory
{
private:
	/** Single instantiation of the GDK interface. Using a weak pointer because it can be destroyed by the online subsystem module when launching & stopping PIE and we don't want to hold an instance to keep it alive  */
	FOnlineSubsystemGDKWeakPtr& GetWeakSingleton() const;

	virtual void DestroySubsystem();

	void OnInitForPIE();
	void OnTeardownForPIE();

public:
	FOnlineFactoryGDK_ForPIE();
	virtual ~FOnlineFactoryGDK_ForPIE();

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName) override;
};

#endif //WITH_EDITOR







#endif //WITH_GRDK

/**
 * Called right after the module DLL has been loaded and the module object has been created
 * Registers the actual implementation of the GDK online subsystem with the engine
 */
void FOnlineSubsystemGDKModule::StartupModule()
{
#if WITH_GRDK
#if WITH_EDITOR
	if (GIsEditor)
	{
		GDKFactory = MakeUnique<FOnlineFactoryGDK_ForPIE>();
	}
	else
#endif //WITH_EDITOR
	{
		GDKFactory = MakeUnique<FOnlineFactoryGDK>();
	}

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(GDK_SUBSYSTEM, GDKFactory.Get());
#endif
}

/**
 * Called before the module is unloaded, right before the module object is destroyed.
 * Overloaded to shut down all loaded online subsystems
 */
void FOnlineSubsystemGDKModule::ShutdownModule()
{
#if WITH_GRDK
	if (GDKFactory.IsValid())
	{
		FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
		OSS.UnregisterPlatformService(GDK_SUBSYSTEM);

		GDKFactory.Reset();
	}
#endif //WITH_GRDK
}

#if WITH_GRDK
FOnlineSubsystemGDKPtr& FOnlineFactoryGDK::GetSingleton() const
{
	static FOnlineSubsystemGDKPtr GDKSingleton;
	return GDKSingleton;
}

void FOnlineFactoryGDK::DestroySubsystem()
{
	FOnlineSubsystemGDKPtr& GDKSingleton = GetSingleton();
	if (GDKSingleton.IsValid())
	{
		GDKSingleton->Shutdown();
		GDKSingleton.Reset();
	}
}

FOnlineFactoryGDK::~FOnlineFactoryGDK()
{
	DestroySubsystem();
}

IOnlineSubsystemPtr FOnlineFactoryGDK::CreateSubsystem(FName InstanceName)
{
	FOnlineSubsystemGDKPtr& GDKSingleton = GetSingleton();
	if (GDKSingleton.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of GDK online subsystem!"));
		return nullptr;
	}

	GDKSingleton = MakeShared<FOnlineSubsystemGDK, ESPMode::ThreadSafe>(InstanceName);
	if (GDKSingleton->IsEnabled())
	{
		if (!GDKSingleton->Init())
		{
			UE_LOG_ONLINE(Warning, TEXT("GDK API failed to initialize!"));
			DestroySubsystem();
			return nullptr;
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("GDK API disabled!"));
		DestroySubsystem();
		return nullptr;
	}

	return GDKSingleton;
}




#if WITH_EDITOR

FOnlineSubsystemGDKWeakPtr& FOnlineFactoryGDK_ForPIE::GetWeakSingleton() const
{
	static FOnlineSubsystemGDKWeakPtr GDKSingleton;
	return GDKSingleton;
}

void FOnlineFactoryGDK_ForPIE::DestroySubsystem()
{
	FOnlineSubsystemGDKWeakPtr& GDKSingleton = GetWeakSingleton();
	if (GDKSingleton.IsValid())
	{
		GDKSingleton.Pin()->Shutdown();
		GDKSingleton.Reset();
	}
}

FOnlineFactoryGDK_ForPIE::FOnlineFactoryGDK_ForPIE()
{
#if WITH_GRDK
	// NB. these are only called when using the MSGamingRuntime plugin with bRestartRuntimeForPIE enabled
	IGDKRuntimeModule::Get().GetOnInitForPIE().AddRaw( this, &FOnlineFactoryGDK_ForPIE::OnInitForPIE );
	IGDKRuntimeModule::Get().GetOnTeardownForPIE().AddRaw( this, &FOnlineFactoryGDK_ForPIE::OnTeardownForPIE );
#endif // WITH_GRDK
}

FOnlineFactoryGDK_ForPIE::~FOnlineFactoryGDK_ForPIE()
{
	DestroySubsystem();

#if WITH_GRDK
	if (IGDKRuntimeModule* GDKRuntime = IGDKRuntimeModule::TryGet())
	{
		GDKRuntime->GetOnInitForPIE().RemoveAll(this);
		GDKRuntime->GetOnTeardownForPIE().RemoveAll(this);
	}
#endif // WITH_GRDK
}

IOnlineSubsystemPtr FOnlineFactoryGDK_ForPIE::CreateSubsystem(FName InstanceName)
{
#if WITH_GRDK
	if (!IGDKRuntimeModule::Get().IsAvailable())
	{
		UE_LOG_ONLINE(Warning, TEXT("GDK runtime module is not available - returning mock OSS instead"));
		return MakeShared<FMockOnlineSubsystemGDK, ESPMode::ThreadSafe>(InstanceName);
	}
#endif // WITH_GRDK

	if (InstanceName == FOnlineSubsystemImpl::DefaultInstanceName)
	{
		UE_LOG_ONLINE(Warning, TEXT("GDK OSS only available during PIE when running in the editor - returning mock OSS instead"));
		return MakeShared<FMockOnlineSubsystemGDK, ESPMode::ThreadSafe>(InstanceName);
	}

	FOnlineSubsystemGDKWeakPtr& GDKSingleton = GetWeakSingleton();
	if (GDKSingleton.IsValid())
	{
		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of GDK online subsystem!"));
		return nullptr;
	}

	FOnlineSubsystemGDKPtr NewInstance = MakeShared<FOnlineSubsystemGDK, ESPMode::ThreadSafe>(InstanceName);
	GDKSingleton = NewInstance;

	if (NewInstance->IsEnabled())
	{
		if (!NewInstance->Init())
		{
			UE_LOG_ONLINE(Warning, TEXT("GDK API failed to initialize!"));
			DestroySubsystem();
			return nullptr;
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("GDK API disabled!"));
		DestroySubsystem();
		return nullptr;
	}

	return NewInstance;
}

void FOnlineFactoryGDK_ForPIE::OnInitForPIE()
{
	FOnlineSubsystemGDKWeakPtr& GDKSingleton = GetWeakSingleton();
	if (GDKSingleton.IsValid() && GDKSingleton.Pin()->IsEnabled())
	{
		if (!GDKSingleton.Pin()->Init())
		{
			UE_LOG_ONLINE(Warning, TEXT("GDK API failed to initialize for PIE!"));
			DestroySubsystem();
		}
	}
}

void FOnlineFactoryGDK_ForPIE::OnTeardownForPIE()
{
	FOnlineSubsystemGDKWeakPtr& GDKSingleton = GetWeakSingleton();
	if (GDKSingleton.IsValid())
	{
		GDKSingleton.Pin()->Shutdown();
	}
}
#endif //WITH_EDITOR





#endif //WITH_GRDK

IMPLEMENT_MODULE(FOnlineSubsystemGDKModule, OnlineSubsystemGDK);