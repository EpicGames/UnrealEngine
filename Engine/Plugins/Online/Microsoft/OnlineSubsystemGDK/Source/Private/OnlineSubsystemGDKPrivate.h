// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define INVALID_INDEX -1

/** URL Prefix when using Live socket connection */
#define GDK_URL_PREFIX TEXT("GDK.")

/** pre-pended to all Live logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("GDK: ")

/** global SCID used for non-title-specific queries (e.g. user reputation) */
#define GDK_GLOBAL_SCID TEXT("7492baca-c1b4-440d-a391-b7ef364a8d40")

/** Attribute to use to read the IsBadReputation attribute off a local-user */
#define BAD_REPUTATION_ATTRIBUTE TEXT("BadReputation")
