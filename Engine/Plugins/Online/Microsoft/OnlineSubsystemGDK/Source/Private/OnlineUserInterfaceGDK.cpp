// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineUserInterfaceGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryUsers.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryUserPrivacyPermissions.h"
#include "AsyncTasks/OnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions.h"
#include "OnlineError.h"

bool FOnlineUserGDK::QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& UserIds)
{
	FString Error;
	FGDKContextHandle UserContext = GDKSubsystem->GetGDKContext(LocalUserNum);
	if (!UserContext.IsValid())
	{
		Error = TEXT("Could not find user context for user");
	}
	if (UserIds.Num() < 1)
	{
		Error = TEXT("No users provided to QueryUserInfo");
	}
	if (!Error.IsEmpty())
	{
		GDKSubsystem->ExecuteNextTick([this, LocalUserNum, UserIds, Error = MoveTemp(Error)]()
		{
			const constexpr bool bWasSuccessful = false;
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_QueryUserInfo_Delegate);
			TriggerOnQueryUserInfoCompleteDelegates(LocalUserNum, bWasSuccessful, UserIds, Error);
		});
		return false;
	}

	// Shouldn't happen if UserContext is not nullptr
	check(LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS);

	GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryUsers>(GDKSubsystem, UserContext, UserIds, LocalUserNum, OnQueryUserInfoCompleteDelegates[LocalUserNum]);
	return true;
}

bool FOnlineUserGDK::GetAllUserInfo(int32 LocalUserNum, TArray<TSharedRef<FOnlineUser>>& OutUsers)
{
	UNREFERENCED_PARAMETER(LocalUserNum);

	OutUsers.Empty(UsersMap.Num());
	for (const FOnlineUserListGDKMap::ElementType& Pair : UsersMap)
	{
		OutUsers.Emplace(Pair.Value);
	}

	return true;
}

TSharedPtr<FOnlineUser> FOnlineUserGDK::GetUserInfo(int32 LocalUserNum, const FUniqueNetId& UserId)
{
	UNREFERENCED_PARAMETER(LocalUserNum);

	TSharedRef<FOnlineUserInfoGDK>* FoundUserPtr = UsersMap.Find(UserId.AsShared());
	return FoundUserPtr ? *FoundUserPtr : TSharedPtr<FOnlineUserInfoGDK>();
}

bool FOnlineUserGDK::QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate /*= FOnQueryUserMappingComplete()*/)
{
	FUniqueNetIdGDKRef UserIdGDK = FUniqueNetIdGDK::Cast(UserId);
	GDKSubsystem->ExecuteNextTick([Delegate, UserIdGDK, DisplayNameOrEmail]()
	{
		const constexpr bool bWasSuccessful = false;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_QueryUserIdMapping_Delegate);
		Delegate.ExecuteIfBound(bWasSuccessful, *UserIdGDK, DisplayNameOrEmail, *FUniqueNetIdGDK::EmptyId(), TEXT("Querying user id mapping is not supported"));
	});

	return false;
}

bool FOnlineUserGDK::QueryExternalIdMappings(const FUniqueNetId& UserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate /*= FOnQueryExternalIdMappingsComplete()*/)
{
	FUniqueNetIdGDKRef UserIdGDK = FUniqueNetIdGDK::Cast(UserId);
	GDKSubsystem->ExecuteNextTick([Delegate, QueryOptions, UserIdGDK]()
	{
		const constexpr bool bWasSuccessful = false;
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_QueryExternalIdMappings_Delegate);
		Delegate.ExecuteIfBound(bWasSuccessful, *UserIdGDK, QueryOptions, TArray<FString>(), TEXT("Querying external user id mapping is not supported"));
	});

	return false;
}

void FOnlineUserGDK::GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds)
{
	OutIds.Empty(ExternalIds.Num());
	OutIds.AddDefaulted(ExternalIds.Num());
}

FUniqueNetIdPtr FOnlineUserGDK::GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId)
{
	return nullptr;
}

void FOnlineUserGDK::QueryUserCommunicationPermissions(const FUniqueNetId& UserId, const TArray<FUniqueNetIdRef >& InUsersToQuery, const TArray<XblPermission>& PermissionsToQuery, const FOnGDKCommunicationPermissionsQueryComplete& CompletionDelegate)
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);

	// Get an GDK context
	FGDKContextHandle UserContext = GDKSubsystem->GetGDKContext(*GDKUserId);
	if (UserContext.IsValid())
	{
		// Fire off our task
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryUserPrivacyPermissions>(
			GDKSubsystem,
			UserContext,
			InUsersToQuery,
			PermissionsToQuery,
			FOnGDKUserPrivacyPermissionsQueryComplete::CreateThreadSafeSP(this, &FOnlineUserGDK::OnQueryUserCommunicationPermissionsComplete, CompletionDelegate, PermissionsToQuery)
			);
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([CompletionDelegate, GDKUserId]() 
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_QueryUserCommunicationPermissions_Delegate);
			CompletionDelegate.ExecuteIfBound(OnlineUserGDK::Errors::NoGDKContext(), GDKUserId, FCommunicationPermissionResultsMap());
		});
	}
}

void FOnlineUserGDK::OnQueryUserCommunicationPermissionsComplete(const FOnlineError& RequestStatus, const FUniqueNetIdRef& RequestingUser, const FPrivacyPermissionsResultsMap& Results, const FOnGDKCommunicationPermissionsQueryComplete CompletionDelegate, const TArray<XblPermission> PermissionsToQuery)
{
	// Build summarized results Map
	FCommunicationPermissionResultsMap OutResults;
	OutResults.Empty(Results.Num());
	for (const TPair<FUniqueNetIdRef, TMap<XblPermission, bool> >& UserPermissionsMapPair : Results)
	{
		bool bAllPermissionsAllowed = true;
		for (const XblPermission Permission : PermissionsToQuery)
		{
			const bool* CommunicationAllowedPtr = UserPermissionsMapPair.Value.Find(Permission);
			bool bCommunicationAllowed = CommunicationAllowedPtr && *CommunicationAllowedPtr;
			if (!bCommunicationAllowed)
			{
				bAllPermissionsAllowed = false;
				break;
			}
		}

		OutResults.Add(UserPermissionsMapPair.Key, bAllPermissionsAllowed);
	}

	// Fire off our results
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_OnQueryUserCommunicationPermissionsComplete_Delegate);
	CompletionDelegate.ExecuteIfBound(RequestStatus, RequestingUser, OutResults);
}

void FOnlineUserGDK::QueryAnonymousUserCommunicationPermissions(const FUniqueNetId& UserId, const TArray<XblAnonymousUserType>& UserTypesToQuery, const TArray<XblPermission>& PermissionsToQuery, const FOnGDKAnonymousUserCommunicationPermissionsQueryComplete& CompletionDelegate)
{
	FUniqueNetIdGDKRef GDKUserId = FUniqueNetIdGDK::Cast(UserId);

	// Get an GDK context
	FGDKContextHandle UserContext = GDKSubsystem->GetGDKContext(*GDKUserId);
	if (UserContext.IsValid())
	{
		// Fire off our task
		GDKSubsystem->CreateAndDispatchAsyncTaskParallel<FOnlineAsyncTaskGDKQueryAnonymousUserPrivacyPermissions>(
			GDKSubsystem,
			UserContext,
			UserTypesToQuery,
			PermissionsToQuery,
			FOnGDKAnonymousUserPrivacyPermissionsQueryComplete::CreateThreadSafeSP(this, &FOnlineUserGDK::OnQueryAnonymousUserCommunicationPermissionsComplete, CompletionDelegate, PermissionsToQuery)
			);
	}
	else
	{
		GDKSubsystem->ExecuteNextTick([CompletionDelegate, GDKUserId]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_QueryAnonymousUserCommunicationPermissions_Delegate);
			CompletionDelegate.ExecuteIfBound(OnlineUserGDK::Errors::NoGDKContext(), FAnonymousUserCommunicationPermissionResultsMap());
		});
	}
}

void FOnlineUserGDK::OnQueryAnonymousUserCommunicationPermissionsComplete(const FOnlineError& RequestStatus, const FAnonymousUserPrivacyPermissionsResultsMap& Results, const FOnGDKAnonymousUserCommunicationPermissionsQueryComplete CompletionDelegate, const TArray<XblPermission> PermissionsToQuery)
{
	// Build summarized results Map
	FAnonymousUserCommunicationPermissionResultsMap OutResults;
	OutResults.Empty(Results.Num());
	for (const TPair<XblAnonymousUserType, TMap<XblPermission, bool> >& UserPermissionsMapPair : Results)
	{
		bool bAllPermissionsAllowed = true;
		for (const XblPermission Permission : PermissionsToQuery)
		{
			const bool* CommunicationAllowedPtr = UserPermissionsMapPair.Value.Find(Permission);
			bool bCommunicationAllowed = CommunicationAllowedPtr && *CommunicationAllowedPtr;
			if (!bCommunicationAllowed)
			{
				bAllPermissionsAllowed = false;
				break;
			}
		}

		OutResults.Add(UserPermissionsMapPair.Key, bAllPermissionsAllowed);
	}

	// Fire off our results
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineUserGDK_OnAnonymousUserCommunicationPermissionsQueryComplete_Delegate);
	CompletionDelegate.ExecuteIfBound(RequestStatus, OutResults);
}

#undef COMMUNICATE_USING_TEXT
#endif //WITH_GRDK
