// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "OnlineVoiceInterfaceGDK.h"
#if WITH_ENGINE
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineIdentityInterfaceGDK.h"
#include "GDKThreadCheck.h"
#include "Voice.h"
#include "OnlineAsyncTaskManager.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/Engine.h"
#include "Misc/Crc.h"
#include "HAL/PlatformAffinity.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable: 5043) // exception specification does not match previous declaration
#include <GameChat2.h>
#include <GameChat2Impl.h>
#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

using xbox::services::game_chat_2::chat_manager;
using xbox::services::game_chat_2::chat_user;
using xbox::services::game_chat_2::chat_user_array;
using xbox::services::game_chat_2::c_communicationRelationshipSendAndReceiveAll;
using xbox::services::game_chat_2::game_chat_audio_encoding_bitrate;
using xbox::services::game_chat_2::game_chat_communication_relationship_flags;
using xbox::services::game_chat_2::game_chat_data_frame;
using xbox::services::game_chat_2::game_chat_data_frame_array;
using xbox::services::game_chat_2::game_chat_data_transport_requirement;
using xbox::services::game_chat_2::game_chat_shared_device_communication_relationship_resolution_mode;
using xbox::services::game_chat_2::game_chat_speech_to_text_conversion_mode;
using xbox::services::game_chat_2::game_chat_thread_id;
using xbox::services::game_chat_2::game_chat_user_chat_indicator;

namespace
{
	void* GDKChatManagerAlloc(size_t Size, uint32_t MemoryTypeId)
	{
		return FMemory::Malloc(Size);
	}

	void GDKChatManagerFree(void* PointerToMemory, uint32_t MemoryTypeId)
	{
		FMemory::Free(PointerToMemory);
	}
}

/** Limit for the number of voice packets to keep in the buffer in case they can't be sent immediately. */
static const int32 MaxBufferedVoicePackets = 128;

FOnlineVoiceGDK::FOnlineVoiceGDK(class FOnlineSubsystemGDK* const InGDKSubsystem)
	: GDKSubsystem(InGDKSubsystem)
	, ChatManager(chat_manager::singleton_instance())
	, MaxLocalTalkers(0)
	, MaxRemoteTalkers(0)
	, VoiceNotificationDelta(0.0f)
#if !UE_BUILD_SHIPPING
	, DebugDisplayEnabled(false)
#endif // !UE_BUILD_SHIPPING
	, LocalPacketIndex(0)
	, bChatManagerInitialized(false)
{
}

FOnlineVoiceGDK::~FOnlineVoiceGDK()
{
	CleanupChatManager();

	// Unregister for suspend/resume callbacks
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
}

bool FOnlineVoiceGDK::Init()
{
	bool bHasVoiceEnabled = true;
	if (GConfig->GetBool(TEXT("OnlineSubsystem"),TEXT("bHasVoiceEnabled"), bHasVoiceEnabled, GEngineIni) && !bHasVoiceEnabled)
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Voice Chat is Disabled"));
		return false;
	}

	// Initialize Local Talkers
	{
		if (!GConfig->GetInt(TEXT("OnlineSubsystem"),TEXT("MaxLocalTalkers"), MaxLocalTalkers, GEngineIni))
		{
			MaxLocalTalkers = MAX_SPLITSCREEN_TALKERS;
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxLocalTalkers key in OnlineSubsystem of DefaultEngine.ini"));
		}

		if (MaxLocalTalkers < 0)
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid MaxLocalTalkers value of %d, setting to 0"), MaxLocalTalkers);
			MaxLocalTalkers = 0;
		}
		else if (MaxLocalTalkers > MAX_SPLITSCREEN_TALKERS)
		{
			MaxLocalTalkers = MAX_SPLITSCREEN_TALKERS;
		}
		LocalTalkers.Empty(MaxLocalTalkers);
	}

	// Initialize Remote Talkers
	{
		if (!GConfig->GetInt(TEXT("OnlineSubsystem"),TEXT("MaxRemoteTalkers"), MaxRemoteTalkers, GEngineIni))
		{
			MaxRemoteTalkers = MAX_REMOTE_TALKERS;
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing MaxRemoteTalkers key in OnlineSubsystem of DefaultEngine.ini"));
		}

		if (MaxRemoteTalkers < 0)
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid MaxRemoteTalkers value of %d, setting to 0"), MaxRemoteTalkers);
			MaxRemoteTalkers = 0;
		}
		else if (MaxRemoteTalkers > MAX_REMOTE_TALKERS)
		{
			MaxRemoteTalkers = MAX_REMOTE_TALKERS;
		}

		RemoteTalkers.Empty(MaxRemoteTalkers);
	}

	if (!GConfig->GetFloat(TEXT("OnlineSubsystem"),TEXT("VoiceNotificationDelta"), VoiceNotificationDelta, GEngineIni))
	{
		VoiceNotificationDelta = 0.2f;
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Missing VoiceNotificationDelta key in OnlineSubsystem of DefaultEngine.ini"));
	}

#if !UE_BUILD_SHIPPING
	// If we have Debug enabled on the commandline or config file, register an on-screen display lambda
	GConfig->GetBool(TEXT("OnlineSubsystem"), TEXT("VoiceDebugDisplay"), DebugDisplayEnabled, GEngineIni);
	DebugDisplayEnabled |= FParse::Param(FCommandLine::Get(), TEXT("VoiceDebugDisplay"));

	if (DebugDisplayEnabled)
	{
		TWeakPtr<FOnlineVoiceGDK, ESPMode::ThreadSafe> LambdaWeakThis = AsShared();
		FCoreDelegates::OnGetOnScreenMessages.AddLambda([LambdaWeakThis](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
		{
			FOnlineVoiceGDKPtr StrongThis = LambdaWeakThis.Pin();
			if (StrongThis.IsValid())
			{
				StrongThis->DisplayDebugText(OutMessages);
			}
		});
	}
#endif // !UE_BUILD_SHIPPING

	// Setup our chat manager
	InitializeChatManager();

	// Register for suspend/resume callbacks
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FOnlineVoiceGDK::OnApplicationWillEnterBackground);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FOnlineVoiceGDK::OnApplicationHasEnteredForeground);

	return bChatManagerInitialized;
}

void FOnlineVoiceGDK::OnApplicationWillEnterBackground()
{
	if (bChatManagerInitialized)
	{
		CleanupChatManager();
	}
}

void FOnlineVoiceGDK::OnApplicationHasEnteredForeground()
{
	if (!bChatManagerInitialized)
	{
		InitializeChatManager();
	}
}


TSharedPtr<FLocalTalkerGDK> FOnlineVoiceGDK::GetLocalTalker(const int32 LocalUserNum) const
{
	FOnlineIdentityGDKPtr IdentityInt = GDKSubsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	FUniqueNetIdPtr LocalUserId = IdentityInt->GetUniquePlayerId(LocalUserNum);
	if (LocalUserId.IsValid())
	{
		return GetLocalTalker(*LocalUserId);
	}

	return nullptr;
}

TSharedPtr<FLocalTalkerGDK> FOnlineVoiceGDK::GetLocalTalker(const FUniqueNetId& NetId) const
{
	for (TSharedPtr<FLocalTalkerGDK> LocalTalker : LocalTalkers)
	{
		if (LocalTalker.IsValid() && *LocalTalker->LocalTalkerId == NetId)
		{
			return LocalTalker;
		}
	}

	return nullptr;
}

TSharedPtr<FRemoteTalkerGDK> FOnlineVoiceGDK::GetRemoteTalker(const FUniqueNetId& NetId) const
{
	for (TSharedPtr<FRemoteTalkerGDK> RemoteTalker : RemoteTalkers)
	{
		if (RemoteTalker.IsValid() && *RemoteTalker->TalkerId == NetId)
		{
			return RemoteTalker;
		}
	}

	return nullptr;
}

void FOnlineVoiceGDK::ClearVoicePackets()
{
	// this is called once per frame and deletes any packets not yet sent
	// this is bad for reliable packets, so we make sure they don't get deleted
	const FScopeLock PacketLock(&LocalPacketsCS);

	for (int32 i = (LocalPackets.Num() - 1); i >= 0; --i)
	{
		if (!LocalPackets[i]->IsReliable())
		{
			LocalPackets.RemoveAt(i, EAllowShrinking::No);
		}
	}

	LocalPackets.Append(PendingLocalPackets);
	PendingLocalPackets.Reset();
}

void FOnlineVoiceGDK::Tick(float DeltaTime)
{
	// Submit voice packets from library to network
	ProcessLocalVoicePackets();
	// Submit packets from network to audio system
	ProcessRemoteVoicePackets();
	// Fire off any talking notifications for hud display
	ProcessTalkingDelegates(DeltaTime);
}

void FOnlineVoiceGDK::StartNetworkedVoice(uint8 LocalUserNum)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid() && LocalTalker->ChatUser)
	{
		LocalTalker->ChatUser->local()->set_microphone_muted(false);
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid user specified in StartNetworkedVoice(%d)"), static_cast<uint32>(LocalUserNum));
	}
}

void FOnlineVoiceGDK::StopNetworkedVoice(uint8 LocalUserNum)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid() && LocalTalker->ChatUser)
	{
		LocalTalker->ChatUser->local()->set_microphone_muted(true);
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("Invalid user specified in StopNetworkedVoice(%d)"), static_cast<uint32>(LocalUserNum));
	}
}

bool FOnlineVoiceGDK::RegisterLocalTalker(uint32 LocalUserNum)
{
	FOnlineIdentityGDKPtr IdentityInt = GDKSubsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
	if (!UniqueId.IsValid())
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("RegisterLocalTalker: Unable to register local talker %u, unable to get unique player id"), LocalUserNum);
		return false;
	}

	return RegisterLocalTalker(*UniqueId);
}

bool FOnlineVoiceGDK::RegisterLocalTalker(const FUniqueNetId& LocalPlayer)
{
	// Check if user already registered
	{
		TSharedPtr<FLocalTalkerGDK> ExistingTalker = GetLocalTalker(LocalPlayer);
		if (ExistingTalker.IsValid())
		{
			// If we're already registered, don't re-register.  Count as a "success" to the user by returning true
			UE_LOG_ONLINE_VOICE(Log, TEXT("Ignoring already registered local talker %s"), *LocalPlayer.ToDebugString());
			return true;
		}
	}

	// Ensure we can have this many local talkers
	if (LocalTalkers.Num() >= MaxLocalTalkers)
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("RegisterLocalTalker: Adding local talker %s would exceed MaxLocalTalkers of %d, it may not be configured correctly."), *LocalPlayer.ToDebugString(), MaxLocalTalkers);
		return false;
	}

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // add_local_user calls XUserFindUserById which is safe to call on a time-sensitive thread

	// Register local talker
	FUniqueNetIdGDKRef GDKUniqueNetId = StaticCastSharedRef<const FUniqueNetIdGDK>(LocalPlayer.AsShared());
	chat_user* const ChatUser = ChatManager.add_local_user(*GDKUniqueNetId->ToString());
	if (!ChatUser)
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("RegisterLocalTalker: Failed to register local talker object with ChatManager"));
		return false;
	}

	// Don't mute the player by default. The call to StartNetworkedVoice occurs before the call to RegisterLocalTalker on clients,
	// so if the user starts out muted clients will never get the StartNetworkedVoice call that unmutes them. This was the behavior
	// in 4.19, and while this won't work with push-to-talk, push-to-talk hasn't been supported on consoles anyway.
	ChatUser->local()->set_microphone_muted(false);

	// Place our fully created talker in the local talkers array
	LocalTalkers.Emplace(MakeShared<FLocalTalkerGDK>(GDKUniqueNetId, ChatUser));

	UE_LOG_ONLINE_VOICE(Log, TEXT("Registered Local talker %s"), *GDKUniqueNetId->ToDebugString());
	return true;
}

void FOnlineVoiceGDK::RegisterLocalTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Registering all local talkers"));

	const IOnlineIdentityPtr IdentityInt = GDKSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		// Loop through the available players and register them
		const TArray<TSharedPtr<FUserOnlineAccount> > OnlineUsers = IdentityInt->GetAllUserAccounts();
		for (const TSharedPtr<FUserOnlineAccount>& User : OnlineUsers)
		{
			if (User.IsValid())
			{
				// Register the local player as a local talker
				RegisterLocalTalker(*User->GetUserId());
			}
		}
	}
}

bool FOnlineVoiceGDK::UnregisterLocalTalkerByUniqueId(const FUniqueNetId& UniqueId)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(UniqueId);
	if (!LocalTalker.IsValid())
	{
		UE_LOG_ONLINE_VOICE(Log, TEXT("Invalid user specified in UnregisterLocalTalkerByUniqueId(%s)"), *UniqueId.ToDebugString());
		return false;
	}

	if (OnPlayerTalkingStateChangedDelegates.IsBound() && (LocalTalker->bIsTalking || LocalTalker->bWasTalking))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineVoiceGDK_UnregisterLocalTalkerByUniqueId_Delegate);
		OnPlayerTalkingStateChangedDelegates.Broadcast(LocalTalker->LocalTalkerId, false);
	}

	TryRemoveChatUserFromChatManager(LocalTalker->ChatUser);

	ensure(LocalTalkers.Remove(LocalTalker.ToSharedRef()) > 0);

	UE_LOG_ONLINE_VOICE(Log, TEXT("Local talker %s unregistered"), *UniqueId.ToDebugString());

	return true;
}

bool FOnlineVoiceGDK::UnregisterLocalTalker(uint32 LocalUserNum)
{
	FOnlineIdentityGDKPtr IdentityInt = GDKSubsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(LocalUserNum);
	if (!UniqueId.IsValid())
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("UnregisterLocalTalker: Unable to unregister local talker %u, unable to get unique player id."), LocalUserNum);
		return false;
	}

	return UnregisterLocalTalkerByUniqueId(*UniqueId);
}

void FOnlineVoiceGDK::UnregisterLocalTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Unregistering all local talkers"));

	TArray<TSharedRef<FLocalTalkerGDK> > LocalTalkersCopy = LocalTalkers;
	for (const TSharedRef<FLocalTalkerGDK>& Talker : LocalTalkersCopy)
	{
		// Unregister the local player as a local talker
		UnregisterLocalTalkerByUniqueId(*Talker->LocalTalkerId);
	}
}

bool FOnlineVoiceGDK::RegisterRemoteTalker(const FUniqueNetId& UniqueId)
{
	// See if this talker has already been registered or not
	TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(UniqueId);
	if (RemoteTalker.IsValid())
	{
		UE_LOG_ONLINE_VOICE(VeryVerbose, TEXT("Remote talker %s is already registered"), *UniqueId.ToDebugString());
		return true;
	}

	// Ensure we can have this many remote talkers
	if (RemoteTalkers.Num() >= MaxRemoteTalkers)
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("RegisterLocalTalker: Adding remote talker %s would exceed MaxRemoteTalkers of %d, it may not be configured correctly."), *UniqueId.ToDebugString(), MaxRemoteTalkers);
		return false;
	}

	const FUniqueNetIdGDK& GDKNetId = static_cast<const FUniqueNetIdGDK&>(UniqueId);
	const uint64 ConsoleId = GDKNetId.ToUint64();
	chat_user* const ChatUser = ChatManager.add_remote_user(*GDKNetId.ToString(), ConsoleId);

	RemoteTalker = MakeShared<FRemoteTalkerGDK>(GDKNetId.AsShared(), ChatUser, ConsoleId);

	RemoteTalkers.Add(RemoteTalker.ToSharedRef());

	UE_LOG_ONLINE_VOICE(Log, TEXT("Registered Remote talker %s"), *UniqueId.ToDebugString());

	return true;
}

bool FOnlineVoiceGDK::UnregisterRemoteTalker(const FUniqueNetId& UniqueId)
{
	TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(UniqueId);
	if (RemoteTalker.IsValid())
	{
		if (OnPlayerTalkingStateChangedDelegates.IsBound() && (RemoteTalker->bIsTalking || RemoteTalker->bWasTalking))
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineVoiceGDK_UnregisterRemoteTalker_Delegate);
			OnPlayerTalkingStateChangedDelegates.Broadcast(RemoteTalker->TalkerId.ToSharedRef(), false);
		}

		TryRemoveChatUserFromChatManager(RemoteTalker->ChatUser);

		ensure(RemoteTalkers.Remove(RemoteTalker.ToSharedRef()) > 0);

		UE_LOG_ONLINE_VOICE(Log, TEXT("Remote talker %s unregistered"), *UniqueId.ToDebugString());
		return true;
	}

	return false;
}

void FOnlineVoiceGDK::RemoveAllRemoteTalkers()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Removing all remote talkers"));

	TArray<TSharedRef<FRemoteTalkerGDK> > RemoteTalkersCopy = RemoteTalkers;
	for (const TSharedRef<FRemoteTalkerGDK>& Talker : RemoteTalkersCopy)
	{
		// Unregister the remote player as a remote talker
		UnregisterRemoteTalker(*Talker->TalkerId);
	}

	RemoteTalkers.Reset();
}

bool FOnlineVoiceGDK::IsHeadsetPresent(uint32 ControllerIndex)
{
	/*
	FOnlineIdentityGDKPtr IdentityInt = GDKSubsystem->GetIdentityGDK();
	check(IdentityInt.IsValid());

	FGDKUserHandle GDKUser = IdentityInt->GetUserForPlatformControllerIndex(ControllerIndex);
	if (GDKUser == nullptr)
	{
		UE_LOGF(LogVoice, Warning, "IsHeadsetPresent: %u, unable to get unique player id", ControllerIndex);
		return false;
	}
	*/
	// XSAPI_C TODO [5/3/2019 claytonv]
// 	for (auto AudioDevice : GDKUser->AudioDevices)
// 	{
// 		if (AudioDevice->DeviceCategory == Communications ||
// 			AudioDevice->DeviceCategory == Voice)
// 		{
// 			return true;
// 		}
// 	}
	return false;
}

bool FOnlineVoiceGDK::IsLocalPlayerTalking(uint32 LocalUserNum)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid())
	{
		return LocalTalker->bIsTalking;
	}

	return false;
}

bool FOnlineVoiceGDK::IsRemotePlayerTalking(const FUniqueNetId& UniqueId)
{
	TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(UniqueId);
	if (RemoteTalker.IsValid())
	{
		return RemoteTalker->bIsTalking;
	}

	return false;
}

bool FOnlineVoiceGDK::IsMuted(uint32 LocalUserNum, const FUniqueNetId& UniqueId) const
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid() && LocalTalker->ChatUser)
	{
		TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(UniqueId);
		if (RemoteTalker.IsValid() && RemoteTalker->ChatUser)
		{
			return LocalTalker->ChatUser->local()->remote_user_muted(RemoteTalker->ChatUser);
		}
	}

	return false;
}

bool FOnlineVoiceGDK::MuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid())
	{
		const bool bShouldBeMuted = true;
		return MuteRemoteTalkerGDK(*LocalTalker->LocalTalkerId, PlayerId, bShouldBeMuted);
	}

	return false;
}

bool FOnlineVoiceGDK::UnmuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalUserNum);
	if (LocalTalker.IsValid())
	{
		const bool bShouldBeMuted = false;
		return MuteRemoteTalkerGDK(*LocalTalker->LocalTalkerId, PlayerId, bShouldBeMuted);
	}

	return false;
}

void FOnlineVoiceGDK::ProcessMuteChangeNotification()
{
	// irrelevant to GDK
}

TSharedPtr<FVoicePacket> FOnlineVoiceGDK::SerializeRemotePacket(FArchive& Ar)
{
	TSharedRef<FVoicePacketGDK> NewPacket = MakeShared<FVoicePacketGDK>();
	NewPacket->Serialize(Ar);

	const FScopeLock PacketLock(&RemotePacketsCS);

	if (Ar.IsError() == false && NewPacket->GetBufferSize() > 0)
	{
		RemotePackets.Add(NewPacket);

		return NewPacket;
	}

	return nullptr;
}

TSharedPtr<FVoicePacket> FOnlineVoiceGDK::GetLocalPacket(uint32 LocalUserNum)
{
	TSharedPtr<FVoicePacket> ReturnVoicePacket;
	const FScopeLock PacketLock(&LocalPacketsCS);

	// duplicate the local copy of the data and set it on a shared pointer for destruction elsewhere
	if (LocalPackets.Num() > 0)
	{
		// GDK doesn't have the same ownership of packets, so we just grab the first packet from the queue to send
		TSharedPtr<FVoicePacket> VoicePacket = LocalPackets[0];
		if (VoicePacket->GetBufferSize() > 0)
		{
			ReturnVoicePacket = VoicePacket;
		}

		LocalPackets.RemoveAt(0, EAllowShrinking::No);
	}

	return ReturnVoicePacket;
}

void FOnlineVoiceGDK::ProcessTalkingDelegates(float DeltaTime)
{
	// Fire off any talker notification delegates for local talkers
	for (TSharedRef<FLocalTalkerGDK>& Talker : LocalTalkers)
	{
		// Only check players with voice
		if (Talker->bIsRegistered)
		{
			// If the talker was not previously talking, but now is trigger the event
			bool bShouldNotify = !Talker->bWasTalking && Talker->bIsTalking;
			// If the talker was previously talking, but now isn't time delay the event
			if (!bShouldNotify && Talker->bWasTalking)
			{
				Talker->LastNotificationTime -= DeltaTime;
				if (Talker->LastNotificationTime <= 0.f)
				{
					// Clear the flag so it only activates when needed
					Talker->bIsTalking = false;
					Talker->LastNotificationTime = VoiceNotificationDelta;
					bShouldNotify = true;
				}
			}

			if (bShouldNotify)
			{
				// Skip all delegate handling if none are registered
				if (OnPlayerTalkingStateChangedDelegates.IsBound())
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineVoiceGDK_ProcessTalkingDelegates_Delegate);
					OnPlayerTalkingStateChangedDelegates.Broadcast(Talker->LocalTalkerId, Talker->bIsTalking);
				}

				Talker->bWasTalking = Talker->bIsTalking;
				UE_LOG_ONLINE_VOICE(VeryVerbose, TEXT("Trigger is %sTALKING"), Talker->bIsTalking ? TEXT("") : TEXT("NOT"));
			}
		}
	}

	// Now check all remote talkers
	for (TSharedRef<FRemoteTalkerGDK>& RemoteTalker : RemoteTalkers)
	{
		// If the talker was not previously talking, but now is trigger the event
		bool bShouldNotify = !RemoteTalker->bWasTalking && RemoteTalker->bIsTalking;
		// If the talker was previously talking, but now isn't time delay the event
		if (!bShouldNotify && RemoteTalker->bWasTalking && !RemoteTalker->bIsTalking)
		{
			RemoteTalker->LastNotificationTime -= DeltaTime;
			if (RemoteTalker->LastNotificationTime <= 0.f)
			{
				bShouldNotify = true;
			}
		}

		if (bShouldNotify)
		{
			// Skip all delegate handling if none are registered
			if (OnPlayerTalkingStateChangedDelegates.IsBound())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineVoiceGDK_ProcessTalkingDelegates_Delegate);
				OnPlayerTalkingStateChangedDelegates.Broadcast(RemoteTalker->TalkerId.ToSharedRef(), RemoteTalker->bIsTalking);
			}

			UE_LOG_ONLINE_VOICE(VeryVerbose, TEXT("Trigger %sTALKING"), RemoteTalker->bIsTalking ? TEXT("") : TEXT("NOT"));

			// Clear the flag so it only activates when needed
			RemoteTalker->bWasTalking = RemoteTalker->bIsTalking;
			RemoteTalker->LastNotificationTime = VoiceNotificationDelta;
		}
	}
}

void FOnlineVoiceGDK::ProcessRemoteVoicePackets()
{
	const FScopeLock PacketLock(&RemotePacketsCS);

	// Clear the talking state for remote players
	for (TSharedRef<FRemoteTalkerGDK>& RemoteTalker : RemoteTalkers)
	{
		RemoteTalker->bIsTalking = false;
	}

	// Now process all pending packets from the server
	for (int32 Index = 0; Index < RemotePackets.Num(); Index++)
	{
		TSharedRef<FVoicePacketGDK> VoicePacket = StaticCastSharedRef<FVoicePacketGDK>(RemotePackets[Index]);
		if (!ensure(VoicePacket->Sender.IsValid()))
		{
			UE_LOG_ONLINE_VOICE(Warning, TEXT("Dropped message with bogus sender"));
			continue;
		}

		ensureMsgf(!GDKSubsystem->IsLocalPlayer(*VoicePacket->Sender), TEXT("Receieved remote message from local player %s"), *VoicePacket->Sender->ToDebugString());

		// check to see if this packet was meant for us
		bool bIsPacketForUs = false;
		for (FUniqueNetIdRef Target : VoicePacket->TargetList)
		{
			if (GDKSubsystem->IsLocalPlayer(*Target))
			{
				bIsPacketForUs = true;
				break;
			}
		}

		if (bIsPacketForUs)
		{
			TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(*VoicePacket->Sender);
			if (!RemoteTalker.IsValid())
			{
				// if we don't know about this user then add them
				ensure(RegisterRemoteTalker(*VoicePacket->Sender));
				RemoteTalker = GetRemoteTalker(*VoicePacket->Sender);
			}

			if (RemoteTalker.IsValid())
			{
				// ignore this code if this is our first packet, or if the voice packet is non-indexed (likely due to this being a control message)
				if (RemoteTalker->LastPacketIndex != 0 && VoicePacket->PacketIndex != 0)
				{
					if (VoicePacket->PacketIndex != RemoteTalker->LastPacketIndex + 1)
					{
						// this is all UDP, so some packet loss is expected, this printout is just designed to help us spot massive loss/mis-ordering
						if (RemoteTalker->LastPacketIndex < VoicePacket->PacketIndex)
						{
							UE_LOG_ONLINE_VOICE(Log, TEXT("Missed %d packets, expected %d got %d"), VoicePacket->PacketIndex - RemoteTalker->LastPacketIndex - 1, RemoteTalker->LastPacketIndex + 1, VoicePacket->PacketIndex);
						}
						else
						{
							UE_LOG_ONLINE_VOICE(Log, TEXT("Received out of order packet, expected %d got %d"), RemoteTalker->LastPacketIndex + 1, VoicePacket->PacketIndex);
						}
					}
				}
				RemoteTalker->LastPacketIndex = VoicePacket->PacketIndex;

				// Submit this packet to the voice engine
				const uint64 ConsoleId = RemoteTalker->ConsoleId;
				const uint32 VoiceBufferSize = VoicePacket->GetBufferSize();
				const void* const VoiceBuffer = VoicePacket->Buffer.GetData();
				ChatManager.process_incoming_data(ConsoleId, VoiceBufferSize, VoiceBuffer);

				RemoteTalker->bIsTalking = true;
				RemoteTalker->LastNotificationTime = VoiceNotificationDelta;
				RemoteTalker->bReceivedData = true;
			}
			else
			{
				UE_LOG_ONLINE_VOICE(Warning, TEXT("Dropped message as we can't find sender %s"), *VoicePacket->Sender->ToDebugString());
			}
		}
		else
		{
			UE_LOG_ONLINE_VOICE(VeryVerbose, TEXT("Rejecting message not meant for us"));
		}
	}
	// Zero the list without causing a free/realloc
	RemotePackets.Reset();
}

FString FOnlineVoiceGDK::GetVoiceDebugState() const
{
	FString Output;

	Output += TEXT("Local Talkers:\n");
	for (const TSharedRef<FLocalTalkerGDK>& LocalTalker : LocalTalkers)
	{
		Output += FString::Printf(TEXT("ID: %s\n Registered: %d\n Networked: %d\n Talking: %d\n "),
			*LocalTalker->LocalTalkerId->ToDebugString(),
			LocalTalker->bIsRegistered,
			LocalTalker->bHasNetworkedVoice,
			LocalTalker->bIsTalking);
	}

	Output += TEXT("Remote Talkers:\n");
	for (const TSharedRef<FRemoteTalkerGDK>& RemoteTalker : RemoteTalkers)
	{
		Output += FString::Printf(TEXT("ID: %s\n IsTalking: %d\n Muted: %s\n"),
			RemoteTalker->TalkerId.IsValid() ? *RemoteTalker->TalkerId->ToDebugString() : TEXT("NULL"),
			RemoteTalker->bIsTalking,
			(RemoteTalker->TalkerId.IsValid() && IsMuted(0, *RemoteTalker->TalkerId)) ? TEXT("1") : TEXT("0"));

	}

	return Output;
}

IVoiceEnginePtr FOnlineVoiceGDK::CreateVoiceEngine()
{
	return IVoiceEnginePtr();
}

void FOnlineVoiceGDK::ProcessLocalVoicePackets()
{
	uint32 NumberOfFrames = 0;
	game_chat_data_frame_array DataArray = nullptr;

	// Process pending voice packets waiting to be sent over the network
	ChatManager.start_processing_data_frames(&NumberOfFrames, &DataArray);
	for (uint32 Index = 0; Index < NumberOfFrames; ++Index)
	{
		const game_chat_data_frame* const Frame = DataArray[Index];
		if (Frame)
		{
			ProcessLocalVoicePacket(*Frame);
		}
	}
	ChatManager.finish_processing_data_frames(DataArray);
}

void FOnlineVoiceGDK::ProcessLocalVoicePacket(const game_chat_data_frame& VoiceFrame)
{
	UE_LOG_ONLINE_VOICE(VeryVerbose, TEXT("ProcessLocalVoicePacket: Packet ready to send"));

	const FUniqueNetIdPtr RegisteredTalkerId = GetFirstRegisteredLocalPlayer();
	if (!RegisteredTalkerId.IsValid())
	{
		// We have no registered talkers, so drop this packet?
		return;
	}

	const TSharedRef<FVoicePacketGDK> NewPacket = MakeShared<FVoicePacketGDK>();

	const uint32 PacketLength = VoiceFrame.packet_byte_count;
	check(PacketLength <= MAX_VOICE_DATA_SIZE);

	NewPacket->Buffer.AddUninitialized(PacketLength);
	FMemory::Memcpy(NewPacket->Buffer.GetData(), VoiceFrame.packet_buffer, PacketLength);
	NewPacket->Length = PacketLength;

	// Don't use the args->ChatUser for the sender, because GDK doesn't always give us one,
	// and for splitscreen to work the Sender needs to be consistent - we use it as the
	// unique console identifier for the GameChat library.
	// Just use the first registered local talker
	// @todo: This probably breaks per-user muting in splitscreen
	NewPacket->Sender = RegisteredTalkerId;

	SetLocalPlayerIsTalking(NewPacket->Sender);

	TArray<FUniqueNetIdRef > UsersToSendDataTo;
	for (uint32 Index = 0; Index < VoiceFrame.target_endpoint_identifier_count; ++Index)
	{
		const uint64_t XuidInt = VoiceFrame.target_endpoint_identifiers[Index];
		UsersToSendDataTo.Emplace(FUniqueNetIdGDK::Create(FString::Printf(TEXT("%llu"), XuidInt)));
	}

	NewPacket->TargetList = MoveTemp(UsersToSendDataTo);
	NewPacket->bIsReliable = (VoiceFrame.transport_requirement == game_chat_data_transport_requirement::guaranteed);
	// TODO: this was 0 in the previous ChatManager if we weren't sending a broadcast; maybe we should get rid of it?
	NewPacket->PacketIndex = 0;

	const FScopeLock PacketLock(&LocalPacketsCS);
	{
		// Packets are only cleared when a net driver calls ClearVoicePackets.
		// If this isn't happening, packets can stack up indefinitely, so clear
		// out an old null or unreliable one if possible.
		if (PendingLocalPackets.Num() >= MaxBufferedVoicePackets)
		{
			// Find a packet to remove to make room for this new one
			const int32 IndexToRemove = PendingLocalPackets.IndexOfByPredicate([](const TSharedPtr<FVoicePacketGDK>& Packet)
			{
				return !Packet.IsValid() || !Packet->IsReliable();
			});

			if (IndexToRemove != INDEX_NONE)
			{
				UE_LOG_ONLINE_VOICE(Verbose, TEXT("FOnlineVoiceGDK: PendingLocalPackets buffer overflow - dropping the oldest packet"));
				PendingLocalPackets.RemoveAt(IndexToRemove, EAllowShrinking::No);
			}
			else
			{
				// No unreliable packets found, the buffer is full of reliable packets. Warn that one will be dropped.
				UE_LOG_ONLINE_VOICE(Warning, TEXT("FOnlineVoiceGDK: PendingLocalPackets buffer reliable overflow - has MaxBufferedVoicePackets (%d) reliable packets queued. Reliable packet will be dropped!"), MaxBufferedVoicePackets);

				PendingLocalPackets.RemoveAt(0, EAllowShrinking::No);
			}
		}

		ensure(PendingLocalPackets.Num() < MaxBufferedVoicePackets);
		PendingLocalPackets.Add(NewPacket);
	}
}

void FOnlineVoiceGDK::InitializeChatManager()
{
	UE_LOG_ONLINE_VOICE(Log, TEXT("Attempting ChatManager2 initialization..."));

	//ChatManager.set_memory_callbacks(GDKChatManagerAlloc, GDKChatManagerFree);

	const uint32 AudioThreadId = FMath::CountTrailingZeros64(FPlatformAffinity::GetAudioRenderThreadMask());
	ChatManager.set_thread_processor(game_chat_thread_id::audio, AudioThreadId);
	// TODO Swap the above for the below when GDK minver is new enough.
	//ChatManager.set_thread_affinity_mask(game_chat_thread_id::audio, FPlatformAffinity::GetAudioRenderThreadMask());

	const int32 MaxTalkers = MaxLocalTalkers + MaxRemoteTalkers;
	const float DefaultAudioRenderVolume = 1.00f;
	const game_chat_communication_relationship_flags DefaultCommunicationRelation = game_chat_communication_relationship_flags::receive_audio | game_chat_communication_relationship_flags::send_microphone_audio;
	const game_chat_shared_device_communication_relationship_resolution_mode DefaultSharedDeviceCommunicationRelationResolutionMode = game_chat_shared_device_communication_relationship_resolution_mode::restrictive;
	const game_chat_speech_to_text_conversion_mode DefaultSpeechToTextMode = game_chat_speech_to_text_conversion_mode::automatic;
	ChatManager.initialize(MaxTalkers, DefaultAudioRenderVolume, DefaultCommunicationRelation, DefaultSharedDeviceCommunicationRelationResolutionMode, DefaultSpeechToTextMode);

	ChatManager.set_audio_encoding_bitrate(game_chat_audio_encoding_bitrate::kilobits_per_second_24);

	// Recreate chat users when resume
	for (TSharedRef<FLocalTalkerGDK>& LocalTalker : LocalTalkers)
	{
		chat_user* ChatUser = ChatManager.add_local_user(*LocalTalker->LocalTalkerId->ToString());
		check(ChatUser);
		LocalTalker->ChatUser = ChatUser;
	}

	for (TSharedRef<FRemoteTalkerGDK>& RemoteTalker : RemoteTalkers)
	{
		chat_user* ChatUser = ChatManager.add_remote_user(*RemoteTalker->TalkerId->ToString(), RemoteTalker->ConsoleId);
		check(ChatUser);
		RemoteTalker->ChatUser = ChatUser;
	}

	bChatManagerInitialized = true;

	UE_LOG_ONLINE_VOICE(Verbose, TEXT("ChatManager2 Initialized successfully"));
}

void FOnlineVoiceGDK::CleanupChatManager()
{
	if (bChatManagerInitialized)
	{
		bChatManagerInitialized = false;

		for (TSharedRef<FLocalTalkerGDK>& LocalTalker : LocalTalkers)
		{
			LocalTalker->ChatUser = nullptr;
		}

		for (TSharedRef<FRemoteTalkerGDK>& RemoteTalker : RemoteTalkers)
		{
			RemoteTalker->ChatUser = nullptr;
		}

		ChatManager.cleanup();
	}
}

void FOnlineVoiceGDK::TryRemoveChatUserFromChatManager(xbox::services::game_chat_2::chat_user* ChatUser)
{
	if (!ChatUser)
	{
		return;
	}

	uint32_t RetrievedChatUserCount;
	chat_user_array RetrievedChatUsers;
	ChatManager.get_chat_users(&RetrievedChatUserCount, &RetrievedChatUsers);

	UE_LOG_ONLINE_VOICE(Verbose, TEXT("[FOnlineVoiceGDK::TryRemoveChatUserFromChatManager] Checking if chat user is part of the chat manager's %d retrieved users."), RetrievedChatUserCount);

	bool bRemoveChatUser = false;
	for (uint32_t ChatUserIndex = 0; ChatUserIndex < RetrievedChatUserCount; ++ChatUserIndex)
	{
		chat_user* RetrievedChatUser = RetrievedChatUsers[ChatUserIndex];
		if (RetrievedChatUser == ChatUser)
		{
			bRemoveChatUser = true;
			break;
		}
	}

	if (bRemoveChatUser)
	{
		UE_LOG_ONLINE_VOICE(Verbose, TEXT("[FOnlineVoiceGDK::TryRemoveChatUserFromChatManager] Trying to remove chat user from chat manager."));

		ChatManager.remove_user(ChatUser);
	}
	else
	{
		UE_LOG_ONLINE_VOICE(Warning, TEXT("[FOnlineVoiceGDK::TryRemoveChatUserFromChatManager] Chat user was not part of the chat manager and we didn't try to remove it."));
	}
}

bool FOnlineVoiceGDK::MuteRemoteTalkerGDK(const FUniqueNetId& LocalPlayerId, const FUniqueNetId& RemotePlayerId, const bool bNewMutedState)
{
	TSharedPtr<FLocalTalkerGDK> LocalTalker = GetLocalTalker(LocalPlayerId);
	if (LocalTalker.IsValid() && LocalTalker->ChatUser)
	{
		TSharedPtr<FRemoteTalkerGDK> RemoteTalker = GetRemoteTalker(RemotePlayerId);
		if (RemoteTalker.IsValid() && RemoteTalker->ChatUser)
		{
			LocalTalker->ChatUser->local()->set_remote_user_muted(RemoteTalker->ChatUser, bNewMutedState);
			return true;
		}
	}

	return false;
}

void FOnlineVoiceGDK::SetLocalPlayerIsTalking(FUniqueNetIdPtr NewTalkerId)
{
	// Find the local talker and mark them as talking
	if (NewTalkerId.IsValid())
	{
		for (TSharedRef<FLocalTalkerGDK>& Talker : LocalTalkers)
		{
			// Compare the ids
			if (*NewTalkerId == *Talker->LocalTalkerId)
			{
				Talker->bIsTalking = true;
				Talker->LastNotificationTime = VoiceNotificationDelta;

				return;
			}
		}
	}
}

FUniqueNetIdPtr FOnlineVoiceGDK::GetFirstRegisteredLocalPlayer()
{
	for (const TSharedRef<FLocalTalkerGDK>& Talker : LocalTalkers)
	{
		if (Talker->bIsRegistered)
		{
			return Talker->LocalTalkerId;
		}
	}

	UE_LOG_ONLINE_VOICE(Warning, TEXT("Found no local talkers"));

	return nullptr;
}

int32 FOnlineVoiceGDK::GetNumLocalTalkers()
{
	return LocalTalkers.Num();
}

#if !UE_BUILD_SHIPPING
void FOnlineVoiceGDK::DisplayDebugText(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
{
	uint32 PendingLocalPacketsNum = 0u;
	uint32 LocalPacketsNum = 0u;
	{
		const FScopeLock PacketLock(&LocalPacketsCS);
		PendingLocalPacketsNum = PendingLocalPackets.Num();
		LocalPacketsNum = LocalPackets.Num();
	}

	uint32 RemotePacketsNum = 0u;
	{
		const FScopeLock PacketLock(&RemotePacketsCS);
		RemotePacketsNum = RemotePackets.Num();
	}

	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("PendingLocalPackets: %u, LocalPackets: %u, RemotePackets: %u"), PendingLocalPacketsNum, LocalPacketsNum, RemotePacketsNum)));
	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("")));
	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("Local GDK Talkers:")));
	for (const TSharedRef<FLocalTalkerGDK>& Talker : LocalTalkers)
	{
		DisplayUserStatus(Talker->ChatUser, OutMessages);
	}

	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("")));
	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("Remote GDK Talkers:")));
	for (const TSharedRef<FRemoteTalkerGDK>& Talker : RemoteTalkers)
	{
		DisplayUserStatus(Talker->ChatUser, OutMessages);
	}
}

void FOnlineVoiceGDK::DisplayUserStatus(const chat_user* ChatUser, TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
{
	const TCHAR* ChatStatus = TEXT("Unknown");
	if (ChatUser)
	{
		switch (ChatUser->chat_indicator())
		{
		case game_chat_user_chat_indicator::incoming_communications_muted:
			ChatStatus = TEXT("Muted");
			break;
		case game_chat_user_chat_indicator::local_microphone_muted:
			ChatStatus = TEXT("Microphone Muted");
			break;
		case game_chat_user_chat_indicator::no_chat_focus:
			ChatStatus = TEXT("Away");
			break;
		case game_chat_user_chat_indicator::no_microphone:
			ChatStatus = TEXT("No Microphone");
			break;
		case game_chat_user_chat_indicator::platform_restricted:
			ChatStatus = TEXT("Platform Muted");
			break;
		case game_chat_user_chat_indicator::reputation_restricted:
			ChatStatus = TEXT("Reputation Muted");
			break;
		case game_chat_user_chat_indicator::silent:
			ChatStatus = TEXT("Silent");
			break;
		case game_chat_user_chat_indicator::talking:
			ChatStatus = TEXT("Talking");
			break;
		}
		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("  User: %ls - %s"), ChatUser->xbox_user_id(), ChatStatus)));
	}
	else
	{
		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("  User: Unknown - %s"), ChatStatus)));
	}
}
#endif //!UE_BUILD_SHIPPING

#endif //WITH_ENGINE

#endif //WITH_GRDK