// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_ENGINE
#include "CoreMinimal.h"
#include "Interfaces/VoiceInterface.h"
#include "VoicePacketGDK.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDKPackage.h"
#include "Misc/CoreDelegates.h"

//WMM TODO - This is shared with the live implementation.. Find a way to use the same class?
typedef TArray< TSharedRef<FVoicePacketGDK> > FGDKVoicePacketArray;

/** Forward declare all of the xbox chat manager classes we use */
namespace xbox
{
	namespace services
	{
		namespace game_chat_2
		{
			class chat_manager;
			class chat_user;
			struct game_chat_data_frame;
		}
	}
}

class FOnlineSubsystemGDK;

struct FRemoteTalkerGDK : public FRemoteTalker
{
public:
	xbox::services::game_chat_2::chat_user* ChatUser;
	uint64 ConsoleId;
	bool bReceivedData;
	uint32 LastPacketIndex;

public:
	/** Constructor to take reference to remote chat user */
	FRemoteTalkerGDK(const FUniqueNetIdRef InGDKId, xbox::services::game_chat_2::chat_user* InRemoteChatUser, const uint64 InConsoleId)
		: ChatUser(InRemoteChatUser)
		, ConsoleId(InConsoleId)
		, bReceivedData(false)
		, LastPacketIndex(0u)
	{
		TalkerId = InGDKId;
	}

private:
	FRemoteTalkerGDK() = delete;
};

struct FLocalTalkerGDK : public FLocalTalker
{
public:
	/** GDK Net ID of this local user */
	const FUniqueNetIdRef LocalTalkerId;
	/** Microsoft Chat User object */
	xbox::services::game_chat_2::chat_user* ChatUser;

public:
	/** Constructor */
	FLocalTalkerGDK(const FUniqueNetIdRef& InTalkerId, xbox::services::game_chat_2::chat_user* InChatUser)
		: LocalTalkerId(InTalkerId)
		, ChatUser(InChatUser)
	{
		// All talkers, if they exist, are registered
		bIsRegistered = true;
	}
};

/**
 * The GDK implementation of the voice interface
 */
class FOnlineVoiceGDK
	: public IOnlineVoice
	, public TSharedFromThis<FOnlineVoiceGDK, ESPMode::ThreadSafe>
{
	/** Reference to the main GDK subsystem */
	class FOnlineSubsystemGDK* const GDKSubsystem;

	/** Reference to chat manager */
	xbox::services::game_chat_2::chat_manager& ChatManager;

	/** Maximum permitted local talkers */
	int32 MaxLocalTalkers;
	/** Maximum permitted remote talkers */
	int32 MaxRemoteTalkers;

	/** State of all possible local talkers */
	TArray<TSharedRef<FLocalTalkerGDK> > LocalTalkers;
	/** State of all possible remote talkers */
	TArray<TSharedRef<FRemoteTalkerGDK> > RemoteTalkers;

	/** Time to wait for new data before triggering "not talking" */
	float VoiceNotificationDelta;

	/** Buffered voice data I/O */
	FGDKVoicePacketArray	PendingLocalPackets;
	FGDKVoicePacketArray	LocalPackets;
	FGDKVoicePacketArray	RemotePackets;

	/** Critical sections for threadsafeness */
	FCriticalSection		LocalPacketsCS;
	FCriticalSection		RemotePacketsCS;

#if !UE_BUILD_SHIPPING
	/** Turn on or off debug display/verbose output **/
	bool					DebugDisplayEnabled;
#endif // !UE_BUILD_SHIPPING

	/** Next packet index to send. Only used to track dropped/misordered packets **/
	uint32					LocalPacketIndex;

	/** Is the chat manager currently initialized? */
	bool bChatManagerInitialized;

	/**
	 * Hook up the XBL callbacks
	 *
	 */
	void InitializeChatManager();

	/**
	 * Clean up the XBL Callbacks
	 *
	 */
	void CleanupChatManager();

	/*
	* Check if the ChatUser is part of the ChatManager, then remove it
	*/
	void TryRemoveChatUserFromChatManager(xbox::services::game_chat_2::chat_user* ChatUser);

	/**
	 * Tell GDK to mute/unmute a remote player
	 *
	 * @param LocalPlayerId Local player to mute for
	 * @param RemotePlayerId Remote player to mute
	 * @param bNewMutedState true to mute, false to unmute
	 */
	bool MuteRemoteTalkerGDK(const FUniqueNetId& LocalPlayerId, const FUniqueNetId& RemotePlayerId, const bool bNewMutedState);

	/**
	 * Mark a local player as being talking
	 *
	 * @param NewTalkerId player who is talking
	 */
	void SetLocalPlayerIsTalking(FUniqueNetIdPtr talker);

	/**
	 * Get the first registered local player
	 *
	 * @return First registered local player
	 */
	FUniqueNetIdPtr GetFirstRegisteredLocalPlayer();

	/**
	* Unregister a local talker using their unique net id
	*
	* @param UniqueId player to unregister
	*
	* @return True if the player was unregistered
	*/
	bool UnregisterLocalTalkerByUniqueId(const FUniqueNetId& UniqueId);

#if !UE_BUILD_SHIPPING
	/**
	 * Display all talkers and their status
	 */
	void DisplayDebugText(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages);

	/**
	 * Display an individual user
	 */
	void DisplayUserStatus(const xbox::services::game_chat_2::chat_user* ChatUser, TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages);
#endif // !UE_BUILD_SHIPPING

PACKAGE_SCOPE:
	/**
	 * Processes any talking delegates that need to be fired off
	 *
	 * @param DeltaTime the amount of time that has elapsed since the last tick
	 */
	void ProcessTalkingDelegates(float DeltaTime);

	/** Submits voice packets from the local users to the network */
	void ProcessLocalVoicePackets();

	/** Process a specific voice packet */
	void ProcessLocalVoicePacket(const xbox::services::game_chat_2::game_chat_data_frame& VoiceFrame);

	/**
	 * Submits network packets to audio system for playback
	 */
	void ProcessRemoteVoicePackets();

	/**
	 * Re-evaluates the muting list for all local talkers
	 */
	void ProcessMuteChangeNotification() override;

	/**
	 * Initialize the voice interface
	 */
	virtual bool Init() override;

	/** Get our cached local talker if there is one for this user */
	TSharedPtr<FLocalTalkerGDK> GetLocalTalker(const int32 LocalUserNum) const;
	TSharedPtr<FLocalTalkerGDK> GetLocalTalker(const FUniqueNetId& NetId) const;

	/** Get our cached remote talker if this is one for this user */
	TSharedPtr<FRemoteTalkerGDK> GetRemoteTalker(const FUniqueNetId& NetId) const;

public:
	/** Constructors */
	FOnlineVoiceGDK() = delete;
	FOnlineVoiceGDK(class FOnlineSubsystemGDK* const InGDKSubsystem);

	/** Virtual destructor to force proper child cleanup */
	virtual ~FOnlineVoiceGDK();

	virtual void StartNetworkedVoice(uint8 LocalUserNum) override;
	virtual void StopNetworkedVoice(uint8 LocalUserNum) override;
	virtual bool RegisterLocalTalker(uint32 LocalUserNum) override;
	bool RegisterLocalTalker(const FUniqueNetId& LocalPlayer);
	virtual void RegisterLocalTalkers() override;
	virtual bool UnregisterLocalTalker(uint32 LocalUserNum) override;
	virtual void UnregisterLocalTalkers() override;
	virtual bool RegisterRemoteTalker(const FUniqueNetId& UniqueId) override;
	virtual bool UnregisterRemoteTalker(const FUniqueNetId& UniqueId) override;
	virtual void RemoveAllRemoteTalkers() override;
	virtual bool IsHeadsetPresent(uint32 LocalUserNum) override;
	virtual bool IsLocalPlayerTalking(uint32 LocalUserNum) override;
	virtual bool IsRemotePlayerTalking(const FUniqueNetId& UniqueId) override;
	bool IsMuted(uint32 LocalUserNum, const FUniqueNetId& UniqueId) const override;
	bool MuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) override;
	bool UnmuteRemoteTalker(uint8 LocalUserNum, const FUniqueNetId& PlayerId, bool bIsSystemWide) override;
	virtual TSharedPtr<class FVoicePacket> SerializeRemotePacket(FArchive& Ar) override;
	virtual TSharedPtr<class FVoicePacket> GetLocalPacket(uint32 LocalUserNum) override;
	virtual int32 GetNumLocalTalkers() override;
	virtual void ClearVoicePackets() override;
	virtual void Tick(float DeltaTime) override;
	virtual FString GetVoiceDebugState() const override;
protected:
	void OnApplicationWillEnterBackground();
	void OnApplicationHasEnteredForeground();

	virtual IVoiceEnginePtr CreateVoiceEngine() override;

};

typedef TSharedPtr<FOnlineVoiceGDK, ESPMode::ThreadSafe> FOnlineVoiceGDKPtr;

#endif //WITH_ENGINE