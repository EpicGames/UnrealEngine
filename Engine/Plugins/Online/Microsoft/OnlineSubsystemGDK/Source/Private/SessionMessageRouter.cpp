// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "SessionMessageRouter.h"
#include "OnlineSubsystemGDKPrivate.h"

#include "OnlineAsyncTaskManagerGDK.h"
#include "OnlineSubsystemGDK.h"
#include "OnlineMatchmakingInterfaceGDK.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionInterfaceMpsdGDK.h"
#include "AsyncTasks/OnlineAsyncTaskGDKFindSessionById.h"
#include "Misc/ScopeLock.h"

bool FSessionMessageRouter::FSessionChangedDelegatePair::Equals(const FSessionMessageRouter::FSessionChangedDelegatePair& Other) const
{
	if (FOnlineSubsystemGDK::AreSessionReferencesEqual(&SessionReference, &Other.SessionReference) == false)
	{
		return false;
	}

	//Compare delegate target
	if (Delegate.GetUObject() != Other.Delegate.GetUObject())
	{
		return false;
	}

	return &Delegate == &Other.Delegate || (!Delegate.IsBound() && !Other.Delegate.IsBound());
}

bool FSessionMessageRouter::FSessionChangedDelegatePair::BoundTo(const XblMultiplayerSessionReference& InSessionReference) const
{
	return FOnlineSubsystemGDK::AreSessionReferencesEqual(&InSessionReference, &SessionReference);
}

void FSessionMessageRouter::TriggerOnSessionChangedDelegates(
	const XblMultiplayerSessionReference* SessionReference,
	FName SessionName,
	XblMultiplayerSessionChangeTypes Diff) const
{
	if (!SessionReference)
	{
		return;
	}

	FScopeLock Lock(&DelegateLock);
	TArray<FOnSessionChangedDelegate> DelegatesToRun;

	for (auto Pair : RegisteredDelegates)
	{
		if (Pair.BoundTo(*SessionReference))
		{
			DelegatesToRun.Add(Pair.Delegate);
		}
	}

	// These need to be invoked after traversing RegisteredDelegates, since they may modify it
	for (auto DelegateToRun : DelegatesToRun)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionMessageRouter_OnSessionChanged_Delegate);
		DelegateToRun.ExecuteIfBound(SessionName, Diff);
	}
}

void FSessionMessageRouter::AddOnSessionChangedDelegate(const FOnSessionChangedDelegate& Delegate, const XblMultiplayerSessionReference* SessionReference)
{
	if (SessionReference)
	{
		FScopeLock Lock(&DelegateLock);
		FSessionChangedDelegatePair DelegatePair(Delegate, *SessionReference);
		if (RegisteredDelegates.FindNode(DelegatePair) == nullptr)
		{
			RegisteredDelegates.AddTail(DelegatePair);
		}
	}
}

void FSessionMessageRouter::ClearOnSessionChangedDelegate(const FOnSessionChangedDelegate& Delegate, const XblMultiplayerSessionReference* SessionReference)
{
	if (SessionReference)
	{
		FScopeLock Lock(&DelegateLock);
		if (auto Node = RegisteredDelegates.FindNode(FSessionChangedDelegatePair(Delegate, *SessionReference)))
		{
			RegisteredDelegates.RemoveNode(Node);
		}
	}
}

FSessionMessageRouter::FSessionMessageRouter(FOnlineSubsystemGDK* InSubsystem)
	: GDKSubsystem(InSubsystem)
{

}

const XblMultiplayerSessionReference* FSessionMessageRouter::GetGDKSessionRefForSessionName(const FName& SessionName) const
{
	FNamedOnlineSession* NamedSession = GDKSubsystem->GetSessionInterfaceGDK()->GetNamedSession(SessionName);
	if (NamedSession)
	{
		FOnlineSessionInfoMpsdGDKPtr GDKInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(NamedSession->SessionInfo);
		if (GDKInfo.IsValid())
		{
			return GDKInfo->GetGDKMultiplayerSessionRef();
		}
	}

	FOnlineMatchTicketInfoPtr MatchTicket;
	GDKSubsystem->GetMatchmakingInterfaceGDK()->GetMatchmakingTicket(SessionName, MatchTicket);
	if (MatchTicket.IsValid())
	{
		return MatchTicket->GetGDKSessionRef();
	}

	return nullptr;
}

FName FSessionMessageRouter::GetSessionNameForGDKSessionRef(const XblMultiplayerSessionReference& GDKSessionRef)
{
	TOptional<FName> Name = GDKSubsystem->GetSessionInterfaceGDK()->GetMpsdImpl()->GetNamedSessionNameForGDKSessionRef(&GDKSessionRef);
	if (Name.IsSet())
	{
		return Name.GetValue();
	}

	FOnlineMatchTicketInfoPtr MatchTicket = GDKSubsystem->GetMatchmakingInterfaceGDK()->GetMatchTicketForGDKSessionRef(&GDKSessionRef);
	if (MatchTicket.IsValid())
	{
		return MatchTicket->SessionName;
	}

	return NAME_None;
}

void FSessionMessageRouter::GetUpdatedSessionAndCompare(const XblMultiplayerSessionChangeEventArgs& EventArgs)
{
	FName SessionName = GetSessionNameForGDKSessionRef(EventArgs.SessionReference);

	FGDKContextHandle Context = GDKSubsystem->GetFirstValidContext();

	if (Context.IsValid())
	{
		GDKSubsystem->CreateAndDispatchAsyncTaskSerial<FOnlineAsyncTaskGDKFindSessionById>(
			GDKSubsystem,
			Context,
			0,
			&EventArgs.SessionReference,
			FOnSingleSessionResultCompleteDelegate::CreateThreadSafeSP(this, &FSessionMessageRouter::OnGetSessionForCompareComplete, EventArgs, SessionName));
	}
}

void FSessionMessageRouter::OnGetSessionForCompareComplete(int32 UserIndex, bool bSuccessful, const FOnlineSessionSearchResult& SearchResult, const XblMultiplayerSessionChangeEventArgs EventArgs, FName SessionName)
{
	if (!bSuccessful || !SearchResult.Session.SessionInfo.IsValid())
	{
		return;
	}
	
	FOnlineSessionInfoMpsdGDKPtr SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoMpsdGDK>(SearchResult.Session.SessionInfo);
	if (!SessionInfo.IsValid())
	{
		return;
	}

	FGDKMultiplayerSessionHandle UpdatedSession = SessionInfo->GetGDKMultiplayerSession();

	if (auto GDKSessionInfo = XblMultiplayerSessionGetInfo(UpdatedSession))
	{
		FString Branch(GDKSessionInfo->Branch);
		SetLastProcessedChangeNumber(Branch, GDKSessionInfo->ChangeNumber);

		auto GDKSessionReference = GetGDKSessionRefForSessionName(SessionName);

		if (XblMultiplayerSessionReferenceIsValid(GDKSessionReference))
		{
			auto UpdatedSessionReference = XblMultiplayerSessionSessionReference(UpdatedSession);
			if (FOnlineSubsystemGDK::AreSessionReferencesEqual(GDKSessionReference, UpdatedSessionReference))
			{
				auto OldSession = GDKSubsystem->GetLastDiffedSession(SessionName);

				auto Diff = XblMultiplayerSessionCompare(UpdatedSession, OldSession);

				GDKSubsystem->CacheGDKSession(SessionName, UpdatedSession);
				GDKSubsystem->SetLastDiffedSession(SessionName, UpdatedSession);

				GDKSubsystem->ExecuteNextTick([this, SessionName, Diff]
				{
					// Session Reference went out of scope when captured, retrieving it here is safer
					const XblMultiplayerSessionReference* GDKSessionReference = GetGDKSessionRefForSessionName(SessionName);
					
					if (GDKSessionReference != nullptr)
					{
						TriggerOnSessionChangedDelegates(GDKSessionReference, SessionName, Diff);
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("FSessionMessageRouter::OnGetSessionForCompareComplete - Session Reference could not be found for session with name: %s"), *SessionName.ToString());
					}
				});
			}
		}
	}
}

void FSessionMessageRouter::OnMultiplayerConnectionIdChanged()
{
	UE_LOG_ONLINE(Log, TEXT("FSessionMessageRouter::OnMultiplayerConnectionIdChanged"));

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionMessageRouter_OnConnectionIdChanged_Delegate);
	TriggerOnConnectionIdChangedDelegates();
}

void FSessionMessageRouter::OnMultiplayerSubscriptionLost()
{
	UE_LOG_ONLINE(Log, TEXT("FSessionMessageRouter::OnMultiplayerSubscriptionLost"));

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionMessageRouter_OnSubscriptionLost_Delegate);
	TriggerOnSubscriptionsLostDelegates();
}

void FSessionMessageRouter::OnMultiplayerSessionChanged(const XblMultiplayerSessionChangeEventArgs& EventArgs)
{
	UE_LOG_ONLINE(Log, TEXT("FSessionMessageRouter::OnMultiplayerSessionChanged - Session: %hs, Branch: %hs, ChangeNumber: %llu"), EventArgs.SessionReference.SessionName, EventArgs.Branch, EventArgs.ChangeNumber);

	// If there are multiple local users, we'll get multiple events for each session change. Only process the
	// event if we haven't seen this change yet.
	FScopeLock Lock(&LastSeenChangeNumberMapLock);

	if (GetLastProcessedChangeNumber(EventArgs.Branch) < EventArgs.ChangeNumber)
	{
		LastSeenChangeNumberMap.Add(FString(EventArgs.Branch), EventArgs.ChangeNumber);
		UE_LOG_ONLINE(Log, TEXT("FSessionMessageRouter::OnMultiplayerSessionChanged - adding async task"));

		GetUpdatedSessionAndCompare(EventArgs);
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("FSessionMessageRouter: MultiplayerSessionChanged - duplicate event, ignoring"));
	}
}

void FSessionMessageRouter::SyncInitialSessionState(FName SessionName, FGDKMultiplayerSessionHandle Session)
{
	check(IsInGameThread());

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionMessageRouter_SyncInitialSessionState_Delegate);
		TriggerOnSessionNeedsInitialStateDelegates(SessionName);
	}

	// Set baseline for future change notifications. Session should be the most recent MultiplayerSession object for the new
	// game or match session.
	GDKSubsystem->SetLastDiffedSession(SessionName, Session);
	auto SessionInfo = XblMultiplayerSessionGetInfo(Session);

	SetLastProcessedChangeNumber(SessionInfo->Branch, SessionInfo->ChangeNumber);
}

uint64 FSessionMessageRouter::GetLastProcessedChangeNumber(const FString& Branch)
{
	FScopeLock Lock(&LastProcessedChangeNumberMapLock);
	uint64* lastProcessedChangeNumber = LastProcessedChangeNumberMap.Find(Branch);

	return (lastProcessedChangeNumber == nullptr) ? 0 : *lastProcessedChangeNumber;
}

void FSessionMessageRouter::SetLastProcessedChangeNumber(const FString& Branch, uint64 ChangeNumber)
{
	FScopeLock Lock(&LastProcessedChangeNumberMapLock);

	if(GetLastProcessedChangeNumber(Branch) < ChangeNumber)
	{
		LastProcessedChangeNumberMap.Add(Branch, ChangeNumber);
	}
}

#endif //WITH_GRDK