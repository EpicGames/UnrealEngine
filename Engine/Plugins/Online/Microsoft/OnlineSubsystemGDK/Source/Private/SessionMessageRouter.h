// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "OnlineDelegateMacros.h"
#include "OnlineSubsystemGDKPackage.h"

#include "OnlineSubsystemGDKTypes.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/multiplayer_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSessionNeedsInitialState, FName);
typedef FOnSessionNeedsInitialState::FDelegate FOnSessionNeedsInitialStateDelegate;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSessionChanged, FName, XblMultiplayerSessionChangeTypes);
typedef FOnSessionChanged::FDelegate FOnSessionChangedDelegate;

DECLARE_MULTICAST_DELEGATE(FOnConnectionIdChanged);
typedef FOnConnectionIdChanged::FDelegate FOnConnectionIdChangedDelegate;

DECLARE_MULTICAST_DELEGATE(FOnSubscriptionsLost);
typedef FOnSubscriptionsLost::FDelegate FOnSubscriptionsLostDelegate;

class FSessionMessageRouter : public TSharedFromThis<FSessionMessageRouter, ESPMode::ThreadSafe>
{
	struct FSessionChangedDelegatePair
	{
		XblMultiplayerSessionReference SessionReference;
		const FOnSessionChangedDelegate& Delegate;

		FSessionChangedDelegatePair(const FOnSessionChangedDelegate& InDelegate,
									const XblMultiplayerSessionReference& InSessionReference)
			: SessionReference(InSessionReference)
			, Delegate(InDelegate)
		{
		}

		bool BoundTo(const XblMultiplayerSessionReference& SessionReference) const;

		bool Equals(const FSessionChangedDelegatePair& Other) const;

		bool operator== (const FSessionChangedDelegatePair& Other) const
		{
			return Equals(Other);
		}
	};

	typedef TDoubleLinkedList<FSessionChangedDelegatePair> DelegateList;
	DelegateList RegisteredDelegates;
	mutable FCriticalSection DelegateLock;

public:
	FSessionMessageRouter(class FOnlineSubsystemGDK* InSubsystem);
	virtual ~FSessionMessageRouter() = default;

	void TriggerOnSessionChangedDelegates(
		const XblMultiplayerSessionReference* SessionReference,
		FName SessionName,
		XblMultiplayerSessionChangeTypes Diff) const;

	void AddOnSessionChangedDelegate(const FOnSessionChangedDelegate& Delegate, const XblMultiplayerSessionReference* SessionReference);
	void ClearOnSessionChangedDelegate(const FOnSessionChangedDelegate& Delegate, const XblMultiplayerSessionReference* SessionReference);


	//Delegates
	DEFINE_ONLINE_DELEGATE_ONE_PARAM(OnSessionNeedsInitialState, FName);
	DEFINE_ONLINE_DELEGATE(OnConnectionIdChanged);
	DEFINE_ONLINE_DELEGATE(OnSubscriptionsLost);

	void SyncInitialSessionState(FName SessionName, FGDKMultiplayerSessionHandle Session);

PACKAGE_SCOPE:
	uint64 GetLastProcessedChangeNumber(const FString& Branch);
	void SetLastProcessedChangeNumber(const FString& Branch, uint64 ChangeNumber);

	/** Handle changes to the game session */
	void OnMultiplayerSessionChanged(const XblMultiplayerSessionChangeEventArgs& EventArgs);
	void GetUpdatedSessionAndCompare(const XblMultiplayerSessionChangeEventArgs& EventArgs);
	void OnGetSessionForCompareComplete(int32 UserIndex, bool bSuccessful, const FOnlineSessionSearchResult& SearchResult, const XblMultiplayerSessionChangeEventArgs EventArgs, FName SessionName);

	void OnMultiplayerConnectionIdChanged();
	void OnMultiplayerSubscriptionLost();
private:

	const XblMultiplayerSessionReference* GetGDKSessionRefForSessionName(const FName& SessionName) const;
	FName GetSessionNameForGDKSessionRef(const XblMultiplayerSessionReference& LiveSessionRef);

	//Prevent copies
	FSessionMessageRouter(const FSessionMessageRouter&) = delete;
	FSessionMessageRouter& operator=(const FSessionMessageRouter&) = delete;

	/** Detect a loss of connection to the subscription service and exit multiplayer. */
	void OnMultiplayerSubscriptionsLost(FGDKUserHandle User);

	/** Handle changes to the game session */
	void OnMultiplayerSessionChanged(XblMultiplayerSessionChangeEventArgs& EventArgs);

private:
	class FOnlineSubsystemGDK* GDKSubsystem;

	TMap<FString, uint64> LastSeenChangeNumberMap, LastProcessedChangeNumberMap;
	FCriticalSection LastSeenChangeNumberMapLock, LastProcessedChangeNumberMapLock;

};