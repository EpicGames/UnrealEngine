// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "SimpleTokenParser.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDK.h" // For LogOnline

FSimpleTokenParser::FSimpleTokenParser()
{
}

void FSimpleTokenParser::ResetParser(const TCHAR* SourceBuffer, int32 StartingLineNumber)
{
	Input		= SourceBuffer;
	InputLen	= FCString::Strlen( Input );
	InputPos	= 0;
	PrevPos		= 0;
	PrevLine	= 1;
	InputLine	= StartingLineNumber;
}

// Clears out the stored comment.
void FSimpleTokenParser::ClearComment()
{
	// Can't call Reset as FString uses protected inheritance
	PrevComment.Empty( PrevComment.Len() );
}

//
// Look at a single character from the input stream and return it, or 0=end.
// Has no effect on the input stream.
//
TCHAR FSimpleTokenParser::PeekChar()
{
	return ( InputPos < InputLen ) ? Input[InputPos] : 0;
}

//
// Unget the previous character retrieved with GetChar().
//
void FSimpleTokenParser::UngetChar()
{
	InputPos = PrevPos;
	InputLine = PrevLine;
}

bool FSimpleTokenParser::IsEOL( TCHAR c )
{
	return c==TEXT('\n') || c==TEXT('\r') || c==0;
}

bool FSimpleTokenParser::IsWhitespace( TCHAR c )
{
	return c==TEXT(' ') || c==TEXT('\t') || c==TEXT('\r') || c==TEXT('\n');
}

//
// Get a single character from the input stream and return it, or 0=end.
//
TCHAR FSimpleTokenParser::GetChar(bool bLiteral)
{
	int32 CommentCount = 0;

	PrevPos = InputPos;
	PrevLine = InputLine;

Loop:
	const TCHAR c = Input[InputPos++];
	if ( CommentCount > 0 )
	{
		// Record the character as a comment.
		PrevComment += c;
	}

	if (c == TEXT('\n'))
	{
		InputLine++;
	}
	else if (!bLiteral)
	{
		const TCHAR NextChar = PeekChar();
		if ( c==TEXT('/') && NextChar==TEXT('*') )
		{
			if ( CommentCount == 0 )
			{
				ClearComment();
				// Record the slash and star.
				PrevComment += c;
				PrevComment += NextChar;
			}
			CommentCount++;
			InputPos++;
			goto Loop;
		}
		else if( c==TEXT('*') && NextChar==TEXT('/') )
		{
			if (--CommentCount < 0)
			{
				ClearComment();
				UE_LOG_ONLINE(Error, TEXT("Unexpected '*/' outside of comment"));
			}
			// Star already recorded; record the slash.
			PrevComment += Input[InputPos];

			InputPos++;
			goto Loop;
		}
	}

	if (CommentCount > 0)
	{
		if (c == 0)
		{
			ClearComment();
			UE_LOG_ONLINE(Error, TEXT("End of class header encountered inside comment"));
		}
		goto Loop;
	}
	return c;
}

//
// Skip past all spaces and tabs in the input stream.
//
TCHAR FSimpleTokenParser::GetLeadingChar()
{
	TCHAR TrailingCommentNewline = 0;

	for (;;)
	{
		bool MultipleNewlines = false;

		TCHAR c;

		// Skip blanks.
		do
		{
			c = GetChar();

			// Check if we've encountered another newline since the last one
			if (c == TrailingCommentNewline)
			{
				MultipleNewlines = true;
			}
		} while (IsWhitespace(c));

		if (c != TEXT('/') || PeekChar() != TEXT('/'))
		{
			return c;
		}

		// Clear the comment if we've encountered newlines since the last comment
		if (MultipleNewlines)
		{
			ClearComment();
		}

		// Record the first slash.  The first iteration of the loop will get the second slash.
		PrevComment += c;

		do
		{
			c = GetChar(true);
			if (c == 0)
			{
				return c;
			}
			PrevComment += c;
		} while (!IsEOL(c));

		TrailingCommentNewline = c;
	}
}

// Gets the next token from the input stream, advancing the variables which keep track of the current input position and line.
bool FSimpleTokenParser::GetToken( FSimpleToken & Token, bool bNoConsts/*=false*/ )
{
	TCHAR c = GetLeadingChar();
	TCHAR p = PeekChar();
	if( c == 0 )
	{
		UngetChar();
		return 0;
	}
	Token.StartPos		= PrevPos;
	Token.StartLine		= PrevLine;
	if( (c>='A' && c<='Z') || (c>='a' && c<='z') || (c=='_') )
	{
		// Alphanumeric token.
		int32 Length=0;
		do
		{
			Token.Identifier[Length++] = c;
			if( Length >= NAME_SIZE )
			{
				UE_LOG_ONLINE(Error, TEXT("Identifer length exceeds maximum of %i"), (int32)NAME_SIZE);
				Length = ((int32)NAME_SIZE) - 1;
				break;
			}
			c = GetChar();
		} while( ((c>='A')&&(c<='Z')) || ((c>='a')&&(c<='z')) || ((c>='0')&&(c<='9')) || (c=='_') );
		UngetChar();
		Token.Identifier[Length]=0;

		// Assume this is an identifier unless we find otherwise.
		Token.TokenType = TOKEN_Identifier;

		// If const values are allowed, determine whether the identifier represents a constant
		if ( !bNoConsts )
		{
			// See if the identifier is part of a vector, rotation or other struct constant.
			// boolean true/false
			if( Token.Matches(TEXT("true")) )
			{
				Token.SetConstBool(true);
				return true;
			}
			else if( Token.Matches(TEXT("false")) )
			{
				Token.SetConstBool(false);
				return true;
			}
		}
		return true;
	}

	// if const values are allowed, determine whether the non-identifier token represents a const
	else if ( !bNoConsts && ((c>='0' && c<='9') || ((c=='+' || c=='-') && (p>='0' && p<='9'))) )
	{
		// Integer or floating point constant.
		bool  bIsFloat = 0;
		int32 Length   = 0;
		bool  bIsHex   = 0;
		do
		{
			if( c==TEXT('.') )
			{
				bIsFloat = true;
			}
			if( c==TEXT('X') || c == TEXT('x') )
			{
				bIsHex = true;
			}

			Token.Identifier[Length++] = c;
			if( Length >= NAME_SIZE )
			{
				UE_LOG_ONLINE(Error, TEXT("Number length exceeds maximum of %i "), (int32)NAME_SIZE );
				Length = ((int32)NAME_SIZE) - 1;
				break;
			}
			c = FChar::ToUpper(GetChar());
		} while ((c >= TEXT('0') && c <= TEXT('9')) || (!bIsFloat && c == TEXT('.')) || (!bIsHex && c == TEXT('X')) || (bIsHex && c >= TEXT('A') && c <= TEXT('F')));

		Token.Identifier[Length]=0;
		if (!bIsFloat || c != 'F')
		{
			UngetChar();
		}

		if (bIsFloat)
		{
			Token.SetConstFloat( FCString::Atof(Token.Identifier) );
		}
		else if (bIsHex)
		{
			TCHAR* End = Token.Identifier + FCString::Strlen(Token.Identifier);
			Token.SetConstInt( FCString::Strtoi(Token.Identifier,&End,0) );
		}
		else
		{
			Token.SetConstInt( FCString::Atoi(Token.Identifier) );
		}
		return true;
	}
	else if( c=='"' )
	{
		// String constant.
		TCHAR Temp[MAX_STRING_CONST_SIZE];
		int32 Length=0;
		c = GetChar(1);
		while( (c!='"') && !IsEOL(c) )
		{
			if( c=='\\' )
			{
				c = GetChar(1);
				if( IsEOL(c) )
				{
					break;
				}
				else if(c == 'n')
				{
					// Newline escape sequence.
					c = '\n';
				}
			}
			Temp[Length++] = c;
			if( Length >= MAX_STRING_CONST_SIZE )
			{
				UE_LOG_ONLINE(Error, TEXT("String constant exceeds maximum of %i characters"), (int32)MAX_STRING_CONST_SIZE );
				c = TEXT('\"');
				Length = ((int32)MAX_STRING_CONST_SIZE) - 1;
				break;
			}
			c = GetChar(1);
		}
		Temp[Length]=0;

		if( c != '"' )
		{
			UE_LOG_ONLINE(Error, TEXT("Unterminated string constant: %s"), Temp);
			UngetChar();
		}

		Token.SetConstString(Temp);
		return true;
	}
	else
	{
		// Symbol.
		int32 Length=0;
		Token.Identifier[Length++] = c;

		// Handle special 2-character symbols.
#define PAIR(cc,dd) ((c==cc)&&(d==dd)) /* Comparison macro for convenience */
		TCHAR d = GetChar();
		if
			(	PAIR('<','<')
			||	PAIR('>','>')
			||	PAIR('!','=')
			||	PAIR('<','=')
			||	PAIR('>','=')
			||	PAIR('+','+')
			||	PAIR('-','-')
			||	PAIR('+','=')
			||	PAIR('-','=')
			||	PAIR('*','=')
			||	PAIR('/','=')
			||	PAIR('&','&')
			||	PAIR('|','|')
			||	PAIR('^','^')
			||	PAIR('=','=')
			||	PAIR('*','*')
			||	PAIR('~','=')
			||	PAIR(':',':')
			)
		{
			Token.Identifier[Length++] = d;
			if( c=='>' && d=='>' )
			{
				if( GetChar()=='>' )
					Token.Identifier[Length++] = '>';
				else
					UngetChar();
			}
		}
		else UngetChar();
#undef PAIR

		Token.Identifier[Length] = 0;
		Token.TokenType = TOKEN_Symbol;

		return true;
	}
}

bool FSimpleTokenParser::ExpectToken( FSimpleToken & Token, TCHAR* Expected )
{
	if ( !GetToken( Token ) )
	{
		return false;
	}

	if ( !Token.Matches( Expected ) )
	{
		return false;
	}

	return true;
}

#endif //WITH_GRDK