// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Stack.h"

/**
 *	ESimpleTokenType
 */
enum ESimpleTokenType
{
	TOKEN_None				= 0x00,		// No token.
	TOKEN_Identifier		= 0x01,		// Alphanumeric identifier.
	TOKEN_Symbol			= 0x02,		// Symbol.
	TOKEN_Const				= 0x03,		// A constant.
	TOKEN_Max				= 0x0D
};

/**
 *	FSimpleToken - Information about a token that was just parsed.
 */
class FSimpleToken
{
public:
	/** PropertyType of token. */
	EPropertyType       PropertyType;

	/** Type of token. */
	ESimpleTokenType	TokenType;

	/** Starting position in script where this token came from. */
	int32				StartPos;

	/** Starting line in script. */

	int32				StartLine;

	/** Always valid. */
	TCHAR				Identifier[NAME_SIZE];

	union
	{
		// TOKEN_Const values.
		int32	Int;							// If CPT_Int.
		bool	NativeBool;						// if CPT_Bool
		float	Float;							// If CPT_Float.
		TCHAR	String[MAX_STRING_CONST_SIZE];	// If CPT_String
	};

	// Constructors.
	FSimpleToken() { InitToken( CPT_None ); }

	// Inlines.
	void InitToken( EPropertyType InType )
	{
		PropertyType	= InType;
		TokenType		= TOKEN_None;
		StartPos		= 0;
		StartLine		= 0;
		*Identifier		= 0;
		FMemory::Memzero(String, sizeof(Identifier));
	}
	bool Matches( const TCHAR* Str, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase ) const
	{
		return ( TokenType==TOKEN_Identifier || TokenType==TOKEN_Symbol ) && ( ( SearchCase == ESearchCase::CaseSensitive ) ? !FCString::Strcmp( Identifier, Str ) : !FCString::Stricmp( Identifier, Str ) );
	}
	bool StartsWith( const TCHAR* Str, bool bCaseSensitive = false ) const
	{
		const int32 StrLength = FCString::Strlen( Str );
		return ( TokenType==TOKEN_Identifier || TokenType==TOKEN_Symbol ) && ( bCaseSensitive ? ( !FCString::Strncmp( Identifier, Str, StrLength ) ) : ( !FCString::Strnicmp( Identifier, Str, StrLength ) ) );
	}
	bool IsBool() const
	{
		return PropertyType == CPT_Bool || PropertyType == CPT_Bool8 || PropertyType == CPT_Bool16 || PropertyType == CPT_Bool32 || PropertyType == CPT_Bool64;
	}

	// Setters.

	void SetConstInt( int32 InInt )
	{
		PropertyType	= CPT_Int;
		Int				= InInt;
		TokenType		= TOKEN_Const;
	}

	void SetConstBool( bool InBool )
	{
		PropertyType	= CPT_Bool;
		NativeBool 		= InBool;
		TokenType		= TOKEN_Const;
	}

	void SetConstFloat( float InFloat )
	{
		PropertyType	= CPT_Float;
		Float			= InFloat;
		TokenType		= TOKEN_Const;
	}

	void SetConstString( TCHAR* InString, int32 MaxLength=MAX_STRING_CONST_SIZE )
	{
		check(MaxLength>0);
		PropertyType = CPT_String;
		if( InString != String )
		{
			FCString::Strncpy( String, InString, MaxLength );
		}
		TokenType = TOKEN_Const;
	}
};

/**
 *	FSimpleTokenParser - Parser logic ripped from BaseParser.cpp
 */
class FSimpleTokenParser
{
public:
	FSimpleTokenParser();

	void	ResetParser( const TCHAR* SourceBuffer, int32 StartingLineNumber );
	TCHAR	PeekChar();
	void	UngetChar();
	bool	IsEOL( TCHAR c );
	bool	IsWhitespace( TCHAR c );
	TCHAR	GetChar( bool bLiteral = false );
	TCHAR	GetLeadingChar();
	void	ClearComment();
	bool	GetToken( FSimpleToken & Token, bool bNoConsts = false );
	bool	ExpectToken( FSimpleToken & Token, TCHAR* Expected );

	// Input text.
	const TCHAR* Input;

	// Length of input text.
	int32 InputLen;

	// Current position in text.
	int32 InputPos;

	// Current line in text.
	int32 InputLine;

	// Position previous to last GetChar() call.
	int32 PrevPos;

	// Line previous to last GetChar() call.
	int32 PrevLine;

	// Previous comment parsed by GetChar() call.
	FString PrevComment;
};
