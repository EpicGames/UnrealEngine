// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#if WITH_ENGINE
#include "VoicePacketGDK.h"
#include "OnlineSubsystemGDKPrivate.h"
#include "OnlineSubsystemGDKTypes.h"
#include "Interfaces/OnlineIdentityInterface.h"

FVoicePacketGDK::FVoicePacketGDK(const FVoicePacketGDK& Other)
	: FVoicePacket(Other)
	, Sender(Other.Sender)
	, TargetList(Other.TargetList)
	, Length(Other.Length)
	, PacketIndex(Other.PacketIndex)
	, bIsReliable(Other.bIsReliable)
{
	// Copy the contents of the voice packet
	Buffer.Empty(Other.Length);
	Buffer.AddUninitialized(Other.Length);
	FMemory::Memcpy(Buffer.GetData(), Other.Buffer.GetData(), Other.Length);
}

uint16 FVoicePacketGDK::GetTotalPacketSize()
{
	uint16 TargetListSize = sizeof(uint8);
	for (const FUniqueNetIdRef& Target : TargetList)
	{
		TargetListSize += Target->GetSize();
	}
	return sizeof(bIsReliable) + TargetListSize + Sender->GetSize() + sizeof(Length) + Length + sizeof(PacketIndex);
}

uint16 FVoicePacketGDK::GetBufferSize()
{
	return Length;
}

FUniqueNetIdPtr FVoicePacketGDK::GetSender()
{
	return Sender;
}

void FVoicePacketGDK::Serialize(FArchive& Ar)
{
	// Make sure not to overflow the buffer by reading an invalid amount
	if (Ar.IsLoading())
	{
		Ar << bIsReliable;

		FString SenderUID;
		Ar << SenderUID;
		Sender = FUniqueNetIdGDK::Create(MoveTemp(SenderUID));

		uint8 TargetLength;
		Ar << TargetLength;
		for (uint8 Index = 0u; Index < TargetLength; ++Index)
		{
			FString TargetUID;
			Ar << TargetUID;
			TargetList.Emplace(FUniqueNetIdGDK::Create(MoveTemp(TargetUID)));
		}

		Ar << Length;

		// Verify the packet is a valid size
		check(Length <= MAX_VOICE_DATA_SIZE);

		// Initialize our buffer with exact space for our data
		Buffer.Empty(Length);
		Buffer.AddUninitialized(Length);

		Ar.Serialize(Buffer.GetData(), Length);

		Ar << PacketIndex;
	}
	else
	{
		Ar << bIsReliable;

		FString SenderString = Sender->ToString();
		Ar << SenderString;

		// Get length of targetlist
		uint32 TargetLength32 = TargetList.Num();

		// Verify the targetlist will fit in our uint8
		check(TargetLength32 <= TNumericLimits<uint8>::Max());

		// Add our targets
		uint8 TargetLength = static_cast<uint8>(TargetLength32);
		Ar << TargetLength;
		for (const FUniqueNetIdRef& Target : TargetList)
		{
			FString TargetString = Target->ToString();
			Ar << TargetString;
		}

		Ar << Length;

		// Always safe to save the data as the voice code prevents overwrites
		Ar.Serialize(Buffer.GetData(), Length);

		Ar << PacketIndex;
	}
}

#endif //WITH_ENGINE
#endif //WITH_GRDK