// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_ENGINE
#include "Net/VoiceDataCommon.h"
#include "Online/CoreOnlineFwd.h"
#include "OnlineSubsystemGDKPackage.h"

/** Defines the data involved in a Live voice packet */
class FVoicePacketGDK : public FVoicePacket
{
PACKAGE_SCOPE:
	/** The unique net id of the talker sending the data */
	FUniqueNetIdPtr Sender;
	/** The unique net id of the machine we're sending the data to, empty if a broadcast message */
	TArray<FUniqueNetIdRef > TargetList;
	/** The data that is to be sent/processed */
	TArray<uint8> Buffer;
	/** The current amount of space used in the buffer for this packet */
	uint16 Length;
	/** What index is this packet? */
	uint32 PacketIndex;
	/** Is this packet reliable (MUST be sent)? */
	bool bIsReliable;

public:
	/** Zeros members and validates the assumptions */
	FVoicePacketGDK()
		: Length(0u)
		, PacketIndex(0u)
		, bIsReliable(false)
	{
		Buffer.Empty(MAX_VOICE_DATA_SIZE);
	}

	/** Should only be used by TSharedPtr and FVoiceData */
	virtual ~FVoicePacketGDK() = default;

	/**
	 * Copies another packet and inits the ref count
	 *
	 * @param Other packet to copy
	 * @param InRefCount the starting ref count to use
	 */
	FVoicePacketGDK(const FVoicePacketGDK& Other);

	/** Returns the amount of space this packet will consume in a buffer */
	virtual uint16 GetTotalPacketSize() override;

	/** @return the amount of space used by the internal voice buffer */
	virtual uint16 GetBufferSize() override;

	/** @return the sender of this voice packet */
	virtual FUniqueNetIdPtr GetSender() override;

	virtual bool IsReliable() override { return bIsReliable; }

	/**
	 * Serialize the voice packet data to a buffer
	 *
	 * @param Ar buffer to write into
	 */
	virtual void Serialize(class FArchive& Ar) override;
};
#endif //WITH_ENGINE
