// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemGDKPackage.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineAsyncTaskManager.h"

#include "OnlineError.h"
#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.identity"

namespace OnlineIdentityGDK
{
#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError HResultError(int32 InCode) { return ONLINE_ERROR(EOnlineErrorResult::FailExtended, FString::Printf(TEXT("0x%08X"), InCode)); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

class FUserOnlineAccountGDK;
enum class ECheckForPackageUpdateResult : uint8;

class FOnlineIdentityGDK
	: public IOnlineIdentity
	, public TSharedFromThis<FOnlineIdentityGDK, ESPMode::ThreadSafe>
{
PACKAGE_SCOPE:

	/** Constructor
	 *
	 * @param InSubsystem The owner of this identity interface.
	 */
	explicit FOnlineIdentityGDK(class FOnlineSubsystemGDK* InSubsystem);

	/** Reference to the owning subsystem */
	class FOnlineSubsystemGDK* GDKSubsystem;

public:

	virtual ~FOnlineIdentityGDK();

	// IOnlineIdentity

	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	//virtual FUniqueNetIdPtr GetSponsorUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI = EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	virtual void GetLinkedAccountAuthToken(int32 LocalUserNum, const FString& TokenType, const FOnGetLinkedAccountAuthTokenCompleteDelegate& Delegate) const override;

	/**
	* Sets a user's XSTS token (adding the user to the internal map if they're not already in it)
	*/
	void SetUserXSTSToken(FGDKUserHandle User, const FString& AuthToken);

	void ResolvePrivilegeWithUIComplete(bool bWasSuccessful, const FUniqueNetIdGDKRef GDKUserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate Delegate);
	void CheckForGDKPackageUpdate(XUserPrivilegeDenyReason DenyReason, const FUniqueNetIdGDKRef& GDKUserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate);
	void PatchCheckCompletion(const FOnlineError& ErrorResult, const TOptional<ECheckForPackageUpdateResult> OptionalPatchCheckResult, const FUniqueNetIdGDKRef GDKUserId, EUserPrivileges::Type Privilege, XUserPrivilegeDenyReason DenyReason, const FOnGetUserPrivilegeCompleteDelegate Delegate);

	float GetPrivilegeCheckDelay() { return PendingPrivilegeCheckDelay; };
	void Tick(const float DeltaTime);


private:

	/**
	 * Sets up event handlers for Xbox system events about users changing so that the
	 * user cache is updated accordingly
	 */
	void HookGDKEvents();

	/**
	 * Removes the event handlers that were set up in HookGDKEvents()
	 */
	void UnhookGDKEvents();

	/**
	 * Delegate called when app resumes from suspend.
	 */
	void HandleAppResume();

public:

	/**
	 * Searches the cache of Users for one that matches the UniqueId and returns it if found.
	 *
	 * @param UniqueId The unique net id to look for.
	 * @return A FGDKUserHandle* matching UniqueId if found, nullptr if a user was not found.
	 */
	FGDKUserHandle GetUserForUniqueNetId(const FUniqueNetIdGDK& UniqueId) const;

	/**
	 * Helper method to translate Controller Index request to User
	 *
	 * @param ControllerIndex the controller index to use
	 *
	 * @return The FGDKUserHandle* associated with the controller index, or nullptr if no users are found
	 */
	FGDKUserHandle GetUserForPlatformUserId(int32 ControllerIndex) const;

	/**
	 * Helper method to translate an GDK User to PlatformUserId
	 *
	 * @param InUser the user to look up
	 *
	 * @return The platform user id associated with the user, or PLATFORMUSERID_NONE if not found.
	 */
	FPlatformUserId GetPlatformUserIdFromGDKUser(FGDKUserHandle InUser) const;

	/**
	 * Helper method to get a player's nickname from an FGDKUserHandle 
	 *
	 * @param RequestedUser the user to look up
	 *
	 * @return The platform user id associated with the user, or PLATFORMUSERID_NONE if not found.
	 */
	FString GetPlayerNickname(const FGDKUserHandle RequestedUser) const;

PACKAGE_SCOPE:
	/**
	 * Delegate fired when a user signs in or out. Fired from FOnlineSubsystemGDK::OnUserLoginChange
	 */
	void OnUserLoginChange(bool bIsSignIn, int32 UserId, int32 UserIndex);

	/**
	* Refresh cached Gamepads and Users when GDK events fire
	*/
	void RefreshGamepadsAndUsers();

	/**
	 * Helper method to get the current cached list of users
	 *
	 * @return The cached list of users
	 */
	//Windows::Foundation::Collections::IVectorView<FGDKUserHandle*>^ GetCachedUsers() const;
	TArray<FGDKUserHandle>& GetCachedUsers();

private:
	/**
	 * Hook into engine initialization completion to add input delegates
	 */
	void OnEngineInitComplete();

#if WITH_EDITOR
	/**
	 * Hook into the editor's post PIE started event
	 */
	void OnPostPIEStarted(bool bIsSimulatingInEditor);
#endif

	/** Create an FUserOnlineAccount for a GDK User */
	bool AddUserAccount(FGDKUserHandle InUser);

	/** Remove an FUserOnlineAccount for a GDK User */
	bool RemoveUserAccount(FGDKUserHandle InUser);

	/**
	 * Delegate fired when the input system adds a new user
	 *
	 * @param InUserAdded pointer to GDK user added to the system
	 */
	void OnUserAdded(FGDKUserHandle InUserAdded);

	/**
	 * Delegate fired when the input system adds a new user
	 *
	 * @param InUserRemoved pointer to GDK user removed from the system
	 */
	void OnUserRemoved(FGDKUserHandle InUserRemoved);
	
	/**
	 * Callback for handling an Input Device's connection state change.
	 * 
	 * @param NewConnectionState	The new connection state of this device
	 * @param FPlatformUserId		The User ID whose input device has changed
	 * @param FInputDeviceId		The Input Device ID that has changed connection
	 */
	void OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	/**
	 * Delegate fired when the input system notes a controller pairing change
	 * Can be multiple firings for one "action"  (olduser->null) then (null->newuser)
	 * User can be the same user (when connecting the USB cable for instance)
	 * 
	 * @param FInputDeviceId	Input device ID
	 * @param FPlatformUserId	The NewUserPlatformId
	 * @param FPlatformUserId	The OldUserPlatformId
	 */
	void OnInputDevicePairingChange(FInputDeviceId InputDeviceId, FPlatformUserId NewUserPlatformId, FPlatformUserId OldUserPlatformId);

	/**
	 * Callback for querying the bOverallReputationIsBad state for our local users.
	 * If the user's bool is true, they are considered a bad user.
	 * 
	 * @param UserIsBadMap A Map of GDK net ids to a bool if they are bad or not
	 */
	void OnReputationQueryComplete(const TUniqueNetIdMap<bool>& UserIsBadMap);

	/** Cached list of users */
	TArray<FGDKUserHandle > CachedUsers;

	/** Lock for updating/reading CachedUsers vector */
	mutable FCriticalSection CachedUsersLock;

	/** Stored delegate handle to remove the task later */
	FDelegateHandle AppInitComplete;
	FDelegateHandle InputDeviceConnectionChanged;
	FDelegateHandle InputDevicePairingChanged;
	FDelegateHandle UserLoginChanged;
#if WITH_EDITOR
	FDelegateHandle PostPIEStartedHandle;
#endif

PACKAGE_SCOPE:
	FString LoginXSTSEndpoint;

private:

	FDelegateHandle AppResumeDelegateHandle;

	/** Map of online user accounts (using user id as key) */
	typedef TUniqueNetIdMap<TSharedRef<FUserOnlineAccountGDK>> GDKUserAccountMap;
	GDKUserAccountMap OnlineUsers;

	float PrivilegeCheckDelayOnResume = 2.5f;
	float PendingPrivilegeCheckDelay = PrivilegeCheckDelayOnResume;

};

class FUserOnlineAccountGDK :
	public FUserOnlineAccount
{
public:

	// FUserOnlineAccount
	/**
	* @return Access token which is provided to user once authenticated by the online service
	*/
	virtual FString GetAccessToken() const override;
	/**
	* @return Any additional auth data associated with a registered user
	*/
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	/**
	* @return True, if the data has been changed
	*/
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FOnlineUser
	/** Id associated with the user account provided by the online service during registration */
	virtual FUniqueNetIdRef GetUserId() const override;
	/** Real name for the user if known */
	virtual FString GetRealName() const override;
	/** Nickname of the user if known */
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	/** Additional user data associated with a registered user */
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	/** Sets the user's access token, used to verify their authentication */
	void SetAccessToken(const FString& AuthToken);

	/** Set if this user has bad reputation or not */
	void SetBadReputation(const bool bIsBadReputation);

	/** Check if this user has their reputation state set, and if so, what the value is */
	TOptional<bool> GetIsBadReputation() const;

	FGDKUserHandle GetUserHandle() { return UserData; }

	/**
	 * Init/default constructor
	 */
	FUserOnlineAccountGDK(FGDKUserHandle InUser, TSharedPtr<FOnlineIdentityGDK> InIdentityInterface)
		: UserData(InUser)
		, IdentityInterfaceWeakPtr(InIdentityInterface)
	{
		uint64 GDKUserId;
		ensure(SUCCEEDED(XUserGetId(InUser, &GDKUserId)));
		UserId = FUniqueNetIdGDK::Create(GDKUserId);
		// Store our XUID as 'id' for Epic login code purposes
		// On other platforms, this isn't always just our FUniqueNetId.ToString(), so
		// we just follow convention
		UserAttributes.Emplace(USER_ATTR_ID, UserId->ToString());
	}

	/**
	 * Destructor
	 */
	virtual ~FUserOnlineAccountGDK() = default;

private:
	FGDKUserHandle UserData;
	TMap<FString, FString> UserAttributes;
	FUniqueNetIdGDKPtr UserId;
	FString UserXSTSToken;
	const TWeakPtr<FOnlineIdentityGDK> IdentityInterfaceWeakPtr;
};

typedef TSharedPtr<class FOnlineIdentityGDK, ESPMode::ThreadSafe> FOnlineIdentityGDKPtr;
#endif //WITH_GRDK
