// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK
#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemGDKPackage.h"
#include "../Private/OnlineAsyncTaskManagerGDK.h"
#include "HAL/RunnableThread.h"
#include "GDKRuntimeModule.h"
#include "GDKThreadCheck.h"
#include <XGame.h>
#include <XSystem.h>
#include <XNetworking.h>

#define UE_API ONLINESUBSYSTEMGDK_API

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineSessionGDK, ESPMode::ThreadSafe> FOnlineSessionGDKPtr;
typedef TSharedPtr<class FOnlineProfileGDK, ESPMode::ThreadSafe> FOnlineProfileGDKPtr;
typedef TSharedPtr<class FOnlineFriendsGDK, ESPMode::ThreadSafe> FOnlineFriendsGDKPtr;
typedef TSharedPtr<class FMessageSanitizerGDK, ESPMode::ThreadSafe> FMessageSanitizerGDKPtr;
typedef TSharedPtr<class FOnlineUserCloudGDK, ESPMode::ThreadSafe> FOnlineUserCloudGDKPtr;
typedef TSharedPtr<class FOnlineLeaderboardsGDK, ESPMode::ThreadSafe> FOnlineLeaderboardsGDKPtr;
typedef TSharedPtr<class FOnlineVoiceGDK, ESPMode::ThreadSafe> FOnlineVoiceGDKPtr;
typedef TSharedPtr<class FOnlineExternalUIGDK, ESPMode::ThreadSafe> FOnlineExternalUIGDKPtr;
typedef TSharedPtr<class FOnlineIdentityGDK, ESPMode::ThreadSafe> FOnlineIdentityGDKPtr;
typedef TSharedPtr<class FOnlinePurchaseGDK, ESPMode::ThreadSafe> FOnlinePurchaseGDKPtr;
typedef TSharedPtr<class FOnlineStoreGDK, ESPMode::ThreadSafe> FOnlineStoreGDKPtr;
typedef TSharedPtr<class FOnlineAchievementsGDK, ESPMode::ThreadSafe> FOnlineAchievementsGDKPtr;
typedef TSharedPtr<class FOnlineStatsGDK, ESPMode::ThreadSafe> FOnlineStatsGDKPtr;
typedef TSharedPtr<class FOnlineEventsGDK, ESPMode::ThreadSafe> FOnlineEventsGDKPtr;
typedef TSharedPtr<class FOnlinePresenceGDK, ESPMode::ThreadSafe> FOnlinePresenceGDKPtr;
typedef TSharedPtr<class FOnlineUserGDK, ESPMode::ThreadSafe> FOnlineUserGDKPtr;
typedef TSharedPtr<class FOnlineMatchmakingInterfaceGDK, ESPMode::ThreadSafe> FOnlineMatchmakingInterfaceGDKPtr;
typedef TSharedPtr<class FSessionMessageRouter, ESPMode::ThreadSafe> FSessionMessageRouterPtr;

class UWorld;
class FOnlineAsyncTask;
class FOnlineAsyncTaskManagerGDK;

/**
* Delegate fired when we have recieved a DTLS Certificate
*/
DECLARE_DELEGATE_TwoParams(FOnDTLSCertificateReceived, FGuid /*ConnectionId*/, const TArray<uint8> /*Thumbprint*/); //WMM should be shared Ptr

/**
* Delegate fired when network has requested DTLS Cert
*/
DECLARE_DELEGATE_TwoParams(FOnNetworkRequestDTLSCertificate, FGuid /*ConnectionId*/, const FOnDTLSCertificateReceived /*Delegate*/);

/**
* Delegate fired when we have recieved a DTLS Certificate
*/
DECLARE_DELEGATE_TwoParams(FOnNetworkGeneratedDTLSCertificate, FGuid /*ConnectionId*/, const TArray<uint8> /*Thumbprint*/); //WMM should be shared Ptr

template<class FOnlineSubsystemClass> class FOnlineAsyncEvent;

/**
 * Wrapper class representing a GDK user context.
 * Required as XblStatisticChangedHandler doesn't admit variable capture, and we can only pass a single pointer in to the delegate, when we need both OSS and User Id.
 */
class UserContextWrapper : public TSharedFromThis<UserContextWrapper, ESPMode::ThreadSafe>
{
public:
	UserContextWrapper(TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> InSys, uint64 InUserId)
	{
		Sys = InSys;
		UserId = InUserId;
	}

	TWeakPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> Sys;
	uint64 UserId = 0;
};

typedef TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> UserContextWrapperPtr;

/**
 *	OnlineSubsystemGDK - Implementation of the online subsystem for GDK services
 */
class FOnlineSubsystemGDK
	: public FOnlineSubsystemImpl
	, public TSharedFromThis<FOnlineSubsystemGDK, ESPMode::ThreadSafe>
{
public:

	// IOnlineSubsystem
	UE_API virtual IOnlineSessionPtr GetSessionInterface() const override;
	UE_API virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	UE_API virtual IMessageSanitizerPtr GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const override;
	UE_API virtual IOnlinePartyPtr GetPartyInterface() const override;
	UE_API virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	UE_API virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	UE_API virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	UE_API virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	UE_API virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	UE_API virtual IOnlineVoicePtr GetVoiceInterface() const override;
	UE_API virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	UE_API virtual IOnlineTimePtr GetTimeInterface() const override;
	UE_API virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	UE_API virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	UE_API virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	UE_API virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	UE_API virtual IOnlineEventsPtr GetEventsInterface() const override;
	UE_API virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	UE_API virtual IOnlineSharingPtr GetSharingInterface() const override;
	UE_API virtual IOnlineUserPtr GetUserInterface() const override;
	UE_API virtual IOnlineMessagePtr GetMessageInterface() const override;
	UE_API virtual IOnlinePresencePtr GetPresenceInterface() const override;
	UE_API virtual IOnlineChatPtr GetChatInterface() const override;
	UE_API virtual IOnlineStatsPtr GetStatsInterface() const override;
	UE_API virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	UE_API virtual FOnlineMatchmakingInterfaceGDKPtr GetMatchmakingInterface() const;
	UE_API virtual IOnlineTournamentPtr GetTournamentInterface() const override;

	UE_API virtual EOnlineEnvironment::Type GetOnlineEnvironment() const override;

	/** Helpers to get typed Interface shared pointers */
	FOnlineSessionGDKPtr GetSessionInterfaceGDK() const { return SessionInterface; }
	FOnlineIdentityGDKPtr GetIdentityGDK() const { return IdentityInterface; }
	FOnlineStoreGDKPtr GetStoreGDK() const { return StoreInterface; }
	FOnlinePurchaseGDKPtr GetPurchaseGDK() const { return PurchaseInterface; }
	FOnlinePresenceGDKPtr GetPresenceGDK() const { return PresenceInterface; }
	FOnlineLeaderboardsGDKPtr GetLeaderboardsInterfaceGDK() const { return LeaderboardsInterface; }
	FOnlineMatchmakingInterfaceGDKPtr GetMatchmakingInterfaceGDK() const { return MatchmakingInterfaceGDK; }
	FSessionMessageRouterPtr GetSessionMessageRouter() const { return SessionMessageRouterInterface; }
	FOnlineFriendsGDKPtr GetFriendsGDK() const { return FriendInterface; }
	FOnlineUserGDKPtr GetUsersGDK() const { return UserInterface; }
	FOnlineStatsGDKPtr GetStatsGDK() const { return StatsInterface; }
	FOnlineAchievementsGDKPtr GetAchievementsInterfaceGDK() const { return AchievementInterface; }

	UE_API virtual bool Init() override;
	UE_API virtual bool Shutdown() override;
	UE_API virtual bool IsEnabled() const override;
	UE_API virtual FString GetAppId() const override;
	UE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	UE_API virtual FText GetOnlineServiceName() const override;
	UE_API virtual FText GetSocialPlatformName() const override;

	// FTSTickerObjectBase
	UE_API virtual bool Tick(float DeltaTime) override;

	/**
	 * @return true if XBox Live Gold is required for online play
	 */
	static UE_API bool IsXBLGoldRequired();

PACKAGE_SCOPE:
	/** Only the factory makes instances */
	FOnlineSubsystemGDK() = delete;

	explicit FOnlineSubsystemGDK(FName InInstanceName)
		: FOnlineSubsystemImpl(GDK_SUBSYSTEM, InInstanceName)
		, ConvertedNetworkConnectivityLevel(EOnlineServerConnectionStatus::Normal)
		, bHasCalledNetworkStatusChangedAtLeastOnce(false)
		, TitleId(0)
	{
	}

	virtual ~FOnlineSubsystemGDK()
	{
		check(CachedGDKContexts.Num() == 0); // no contexts should be created after shutdown
	}

	/** Get title's product id if one is configured */
	UE_API const FString& GetTitleProductId() const;

	UE_API FOnlineAsyncTaskManagerGDK* GetAsyncTaskManager();

	/** Helpers to manage queuing already created FOnlineAsyncItem */
	UE_API void QueueAsyncTask(FOnlineAsyncTask* const AsyncTask, const bool bCanRunInParallel = false);
	UE_API void QueueAsyncEvent(FOnlineAsyncEvent<FOnlineSubsystemGDK>* const AsyncEvent);

	/** Create a new async task with the provided arguments and queue it to be processed in parallel with other tasks */
	template <typename TOnlineAsyncTask, typename... TArguments>
	inline void CreateAndDispatchAsyncTaskParallel(TArguments&&... Arguments)
	{
		// Ensure our passed-in type is derived from FOnlineAsyncTask
		static_assert(TIsDerivedFrom<TOnlineAsyncTask, FOnlineAsyncTask>::IsDerived, "Argument TOnlineAsyncTask must derive from FOnlineAsyncTask");

		check(OnlineAsyncTaskThreadRunnable.IsValid());

		TOnlineAsyncTask* NewTask = new TOnlineAsyncTask(Forward<TArguments>(Arguments)...);
		OnlineAsyncTaskThreadRunnable->AddToParallelTasks(NewTask);
	}

	/** Create a new async task with the provided arguments and add it to the serial processing queue*/
	template <typename TOnlineAsyncTask, typename... TArguments>
	inline void CreateAndDispatchAsyncTaskSerial(TArguments&&... Arguments)
	{
		// Ensure our passed-in type is derived from FOnlineAsyncTask
		static_assert(TIsDerivedFrom<TOnlineAsyncTask, FOnlineAsyncTask>::IsDerived, "Argument TOnlineAsyncTask must derive from FOnlineAsyncTask");

		check(OnlineAsyncTaskThreadRunnable.IsValid());

		TOnlineAsyncTask* NewTask = new TOnlineAsyncTask(Forward<TArguments>(Arguments)...);
		OnlineAsyncTaskThreadRunnable->AddToInQueue(NewTask);
	}

	/** Create a new async event with the provided arguments and queue it to be processed in the OutQueue */
	template <typename TOnlineAsyncEvent, typename... TArguments>
	inline void CreateAndDispatchAsyncEvent(TArguments&&... Arguments)
	{
		// Ensure our passed-in type is derived from FOnlineAsyncEvent
		static_assert(TIsDerivedFrom<TOnlineAsyncEvent, FOnlineAsyncEvent<FOnlineSubsystemGDK>>::IsDerived, "Argument TOnlineAsyncEvent must derive from FOnlineAsyncEvent");

		check(OnlineAsyncTaskThreadRunnable.IsValid());

		TOnlineAsyncEvent* NewEvent = new TOnlineAsyncEvent(Forward<TArguments>(Arguments)...);
		OnlineAsyncTaskThreadRunnable->AddToOutQueue(NewEvent);
	}

	UE_API FGDKContextHandle CreateGDKContext(FGDKUserHandle GDKUser);
	UE_API void DeleteGDKContext(FGDKUserHandle GDKUser);

	/** Returns the GDK context for the given user, or null if the input could not be found/was invalid. */
	UE_API FGDKContextHandle GetGDKContext(int32 LocalUserNum);
	UE_API FGDKContextHandle GetGDKContext(const FUniqueNetId& UserId);
	UE_API FGDKContextHandle GetGDKContext(FGDKMultiplayerSessionHandle Session);
	UE_API FGDKContextHandle GetGDKContext(uint64 GDKUserId);

	/** Returns the cached GDK context for the given user. Creates and caches a new one if necessary. Session subscriptions require that you preserve the context. */
	UE_API FGDKContextHandle GetGDKContext(FGDKUserHandle GDKUser);

	UE_API FGDKContextHandle GetFirstValidContext();
	
	/** Removes one leading and one trailing curly brace from the input string and returns a new string */
	//Platform::String* RemoveBracesFromGuidString( __in Platform::String* guid );

	/** Updates our local caches with updated session info*/
	UE_API void CacheGDKSession(const FName& SessionName, FGDKMultiplayerSessionHandle LatestSession);
	
	/** Updates our local caches to specify this session was the latest seen */
	UE_API void SetLastDiffedSession(const FName& SessionName, FGDKMultiplayerSessionHandle LatestSession);

	/** Return the last seen session by a provided Name */
	UE_API FGDKMultiplayerSessionHandle GetLastDiffedSession(const FName& SessionName);

	/** Helper to compare two sessions to see if they're the same underlying session */
	static UE_API bool AreSessionReferencesEqual(const XblMultiplayerSessionReference* First, const XblMultiplayerSessionReference* Second);

	UE_API void RefreshNetworkConnectivityLevel();
	UE_API void OnNetworkConnectivityHintChanged(const XNetworkingConnectivityHint& ConnectivityHint);
	UE_API void ApplyNetworkConnectivityLevel(const XNetworkingConnectivityHint& ConnectivityHint);
	UE_API void OnUserLoginChange(bool bIsSignIn, int32 PlatformUserId, int32 UserIndex );
	UE_API void HandleAppResume();
	UE_API void HandleNetworkRequestDTLSCertificate(FGuid ConnectionId, FOnDTLSCertificateReceived Delegate);
	UE_API void HandleNetworkGeneratedDTLSCertificate(FGuid ConnectionId, const TArray<uint8> Thumbprint, bool WriteToService);
	UE_API void HandleNetworkRecievedDTLSCertificate(FGuid ConnectionId, const TArray<uint8> Thumbprint);
	UE_API TMap<FGuid, TArray<uint8>> GetCertificateDictionary() const;

	UE_API void RecreateGDKContextOnSubscriptionLost();

	UE_API void EnableSessionEventHandlers(FGDKContextHandle& GDKContext);

PACKAGE_SCOPE:
	EOnlineServerConnectionStatus::Type ConvertedNetworkConnectivityLevel;

	bool bHasCalledNetworkStatusChangedAtLeastOnce;

	/** Title Id for our current application */
	uint32 TitleId;

private:
	/**
	* Exec function handling for Exec() call above
	*/
	UE_API bool HandleSanitizeStringExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	UE_API bool HandleSanitizeStringsExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	UE_API void InitializeXblByConfig();

	UE_API void CleanupGDKContextForNetworkConnectivityLoss();
	UE_API void ReinitializeGDKContextForNetworkConnectivityRestored();

	/** Interface to the session services */
	FOnlineSessionGDKPtr SessionInterface;

	/** Interface to the external UI services */
	FOnlineExternalUIGDKPtr ExternalUIInterface;

	/** Interface to the identity registration/auth services */
	FOnlineIdentityGDKPtr IdentityInterface;

	/** Interface to the store services */
	FOnlineStoreGDKPtr StoreInterface;

	/** Interface to the purchase services */
	FOnlinePurchaseGDKPtr PurchaseInterface;

	/** Interface to the voice chat services */
	FOnlineVoiceGDKPtr VoiceInterface;

	/** Interface to the events services */
	FOnlineEventsGDKPtr EventsInterface;

	/** Interface to the achievement services */
	FOnlineAchievementsGDKPtr AchievementInterface;

	/** Interface to the stats services */
	FOnlineStatsGDKPtr StatsInterface;

	/** Interface to the rich presence services */
	FOnlinePresenceGDKPtr PresenceInterface;

	/** Interface to the leaderboard services */
	FOnlineLeaderboardsGDKPtr LeaderboardsInterface;

	/** Interface to the matchmaking services */
	FOnlineMatchmakingInterfaceGDKPtr MatchmakingInterfaceGDK;

	/** Interface to the mpsd shouldertap services */
	FSessionMessageRouterPtr SessionMessageRouterInterface;

	/** Interface to the Friends services */
	FOnlineFriendsGDKPtr FriendInterface;

	/** Interface to the message sanitizer */
	FMessageSanitizerGDKPtr MessageSanitizer;

	/** Interface to the Users services */
	FOnlineUserGDKPtr UserInterface;

	/** Online async task runnable */
	TUniquePtr<FOnlineAsyncTaskManagerGDK> OnlineAsyncTaskThreadRunnable;

	/** Online async task thread */
	TUniquePtr<FRunnableThread> OnlineAsyncTaskThread;

	static const XblFunctionContext INVALID_XBL_FUNCTION_CONTEXT = 0;

	// Store single XboxGDKContext per user
	struct FGDKContextInfo
	{
		FGDKContextInfo( FGDKContextHandle& InHandle ) : Handle(InHandle) {}
		FGDKContextHandle Handle;
		XblFunctionContext SessionChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext RelationshipChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext DevicePresenceChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext TitlePresenceChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext StatisticChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext ConnectionIdChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
		XblFunctionContext SubscriptionLostContext = INVALID_XBL_FUNCTION_CONTEXT;
	};
	TMap<uint64, FGDKContextInfo> CachedGDKContexts;

	UE_API void DeleteGDKContextInternal(uint64 Xuid, FGDKContextInfo& GDKContext) const;

	UE_API void EnableSessionEventHandlers(FGDKContextInfo& ContextInfo);
	UE_API void DisableSessionEventHandlers(FGDKContextInfo& ContextInfo) const;

	TArray<UserContextWrapperPtr> UserContextWrappers;

	mutable FCriticalSection GDKContextsLock;

	// task queue used for background Xal and Xbl operations
	TUniquePtr<FGDKAsyncTaskQueue> XblTaskQueue;

	// Windows::Foundation::EventRegistrationToken UserRemovedToken; TODO WMM: find analogue for this

	ANSICHAR SandboxId[XSystemXboxLiveSandboxIdMaxBytes];

	HANDLE NetworkConnectivityChangedHandle;

	XalRegistrationToken OnUserStateChanged;
	XblFunctionContext ServiceCallRoutedHandlerContext = INVALID_XBL_FUNCTION_CONTEXT;

	mutable FCriticalSection DTLSDictionariesLock;
	TMap<FGuid, TArray<uint8>> DTLSCertificateDictionary;
	TMap<FGuid, FOnDTLSCertificateReceived> PendingDTLSCertificateRequests;

	FOnNetworkRequestDTLSCertificate OnNetworkRequestDTLSCertificate;
	FOnNetworkGeneratedDTLSCertificate OnNetworkGeneratedDTLSCertificate;

	/** Stored delegate handle to remove the task later */
	FDelegateHandle AppInitComplete;
	FDelegateHandle UserLoginChanged;
#ifdef UE_PLAYFAB_MATCHMAKING
	FDelegateHandle OnOnlineSubsystemCreated;
#endif

	TOptional<XTaskQueueRegistrationToken> NetworkConnectivityHandle;
	bool bShouldRestoreConnectivity = false;

#if WITH_EDITOR
	bool bIsInitialized = false;
#endif
};

typedef TSharedPtr<FOnlineSubsystemGDK, ESPMode::ThreadSafe> FOnlineSubsystemGDKPtr;

#undef UE_API

#endif //WITH_GRDK
