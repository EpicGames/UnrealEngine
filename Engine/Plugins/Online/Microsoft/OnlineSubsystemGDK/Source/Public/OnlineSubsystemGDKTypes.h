// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK

#if !WITH_XSAPI_C
	#error unexpectedly have GRDK without XSAPI-C... configuration error?
#endif

#include "OnlineSubsystemTypes.h"
#include "Internationalization/BreakIterator.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/multiplayer_c.h>
#include <xsapi-c/profile_c.h>
#include <xsapi-c/presence_c.h>
#include <xsapi-c/xbox_live_global_c.h>
#include <xsapi-c/xbox_live_context_c.h>
#include <xal/xal.h>
#include <XAsyncProvider.h>
#include <XUser.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#include "OnlineSubsystemGDKPackage.h"
#include "OnlineSessionSettings.h"
#include "GDKHandle.h"
#include "GDKRuntimeModule.h"

SETHANDLETYPETRAITS(XblContextHandle, XblContextDuplicateHandle, XblContextCloseHandle);
SETHANDLETYPETRAITS(XblPresenceRecordHandle, XblPresenceRecordDuplicateHandle, XblPresenceRecordCloseHandle);
SETHANDLETYPETRAITS(XblMultiplayerSessionHandle, XblMultiplayerSessionDuplicateHandle, XblMultiplayerSessionCloseHandle);
SETHANDLETYPETRAITS(XblMultiplayerSearchHandle, XblMultiplayerSearchHandleDuplicateHandle, XblMultiplayerSearchHandleCloseHandle);

typedef TGDKHandle<XblContextHandle> FGDKContextHandle;
typedef TGDKHandle<XblPresenceRecordHandle> FGDKPresenceRecordHandle;
typedef TGDKHandle<XblMultiplayerSessionHandle> FGDKMultiplayerSessionHandle;
typedef TGDKHandle<XblMultiplayerSearchHandle> FGDKMultiplayerSearchHandle;

#define COMMUNICATE_USING_TEXT XblPermission::CommunicateUsingText
#define COMMUNICATE_USING_VOICE XblPermission::CommunicateUsingVoice

class FOnlineSessionSearch;

//// This is token used for GDK delegates.  Please make sure to properly remove GDK callbacks on cleanups
//typedef Windows::Foundation::EventRegistrationToken LiveEventToken;

using FUniqueNetIdGDKRef = TSharedRef<const class FUniqueNetIdGDK>;
using FUniqueNetIdGDKPtr = TSharedPtr<const class FUniqueNetIdGDK>;

/** 
 * GDK Unique Id implementation - wrapping XUID
 */
class FUniqueNetIdGDK : public FUniqueNetId
{
protected:
	/** Holds the net id for a player */
	uint64 UniqueNetId;

	/** Hidden on purpose */
	FUniqueNetIdGDK() :
		UniqueNetId(0)
	{
	}

	/**
	* Copy Constructor
	*
	* @param Src the id to copy
	*/
	explicit FUniqueNetIdGDK(const FUniqueNetIdGDK& Src) :
		UniqueNetId(Src.UniqueNetId)
	{
	}

	/**
	* Constructs this object with the specified net id
	*
	* @param InUniqueNetId the id to set ours to
	*/
	explicit FUniqueNetIdGDK(uint64 InUniqueNetId) :
		UniqueNetId(InUniqueNetId)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param String textual representation of an id
	 */
	explicit FUniqueNetIdGDK(const FString& Str) :
		UniqueNetId(FCString::Atoi64(*Str))
	{
		check(Str.IsEmpty() || IsValid());
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdGDK(const FUniqueNetId& InUniqueNetId)
	{
		*this = *Cast(InUniqueNetId);
	}

	/**
	 * Constructs this object for the given user handle
	 *
	 * @param User handle
	 */
	explicit FUniqueNetIdGDK( const FGDKUserHandle& InUserHandle )
	{
		const bool bSucceeded = InUserHandle.IsValid() && SUCCEEDED(XUserGetId(InUserHandle, &UniqueNetId));
		if (!bSucceeded)
		{
			UniqueNetId = 0;
		}
		check(bSucceeded && IsValid());
	}

	/**
	 * Constructs this object for the given context handle
	 *
	 * @param Context handle
	 */
	explicit FUniqueNetIdGDK( const FGDKContextHandle& InContextHandle )
	{
		const bool bSucceeded = InContextHandle.IsValid() && SUCCEEDED(XblContextGetXboxUserId(InContextHandle, &UniqueNetId));
		if (!bSucceeded)
		{
			UniqueNetId = 0;
		}
		check(bSucceeded && IsValid());
	}


public:
	template<typename... TArgs>
	static FUniqueNetIdGDKRef Create(TArgs&&... Args)
	{
		return MakeShareable(new FUniqueNetIdGDK(Forward<TArgs>(Args)...));
	}

	static FUniqueNetIdGDKRef Cast(const FUniqueNetId& InUniqueNetId)
	{
		if (ensure(InUniqueNetId.GetType() == GDK_SUBSYSTEM))
		{
			return StaticCastSharedRef<const FUniqueNetIdGDK>(InUniqueNetId.AsShared());
		}
		return EmptyId();
	}

	virtual FName GetType() const override
	{
		return GDK_SUBSYSTEM;
	}

	/**
	* Get the raw byte representation of this net id
	* This data is platform dependent and shouldn't be manipulated directly
	*
	* @return byte array of size GetSize()
	*/
	virtual const uint8* GetBytes() const override
	{
		return (uint8*)&UniqueNetId;
	}

	/**
	* Get the size of the id
	*
	* @return size in bytes of the id representation
	*/
	virtual int32 GetSize() const override
	{
		return sizeof(uint64);
	}

	/** Is our structure currently pointing to a valid XUID? */
	virtual bool IsValid() const override
	{
		return UniqueNetId != 0;
	}

	/**
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form
	 */
	virtual FString ToString() const override
	{
		return IsValid() ? FString::Printf(TEXT("%llu"), UniqueNetId) : TEXT("<invalid>");
	}

	/**
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in uint64 form
	 */
	virtual uint64 ToUint64() const
	{
		check(IsValid());
		return UniqueNetId;
	}

	/**
	 * Get a human readable representation of the net id
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return id in string form
	 */
	virtual FString ToDebugString() const override
	{
		if(IsValid())
		{
			const FString UniqueNetIdStr = ToString();
			return OSS_UNIQUEID_REDACT(*this, UniqueNetIdStr);
		}
		else
		{
			return TEXT("INVALID");
		}
	}

	virtual uint32 GetTypeHash() const override
	{
		return ::GetTypeHash(UniqueNetId);
	}

	/** global static instance of invalid id */
	static const FUniqueNetIdGDKRef& EmptyId()
	{
		static const FUniqueNetIdGDKRef EmptyId(Create());
		return EmptyId;
	}

	bool operator==(uint64 RawGDKId) const
	{
		return RawGDKId && UniqueNetId == RawGDKId;
	}

	virtual ~FUniqueNetIdGDK() = default;

	friend FArchive& operator<<(FArchive& Ar, FUniqueNetIdGDK& UserId)
	{
		return Ar << UserId.UniqueNetId;
	}
};

/** State of a match ticket */
namespace EOnlineGDKMatchmakingState
{
	enum Type
	{
		None,
		CreatingMatchSession,
		SubmittingInitialTicket,
		WaitingForGameSession,
		JoiningGameSession,
		Active,
		UserCancelled
	};
}

/** 
 * GDK Match ticket wrapper
 */
class FOnlineMatchTicketInfo
{
public:
	FOnlineMatchTicketInfo()
		: EstimatedWaitTimeInSeconds(0.0f) 
		, SessionReference(nullptr)
	{
	}

	FOnlineMatchTicketInfo(
		const FString& InHopperName,
		const FString& InTicketID,
		float InWaitTimeInSeconds
	) 
		: HopperName( InHopperName )
		, TicketId( InTicketID )
		, EstimatedWaitTimeInSeconds( InWaitTimeInSeconds )
		, SessionReference(nullptr)
		, MatchmakingState(EOnlineGDKMatchmakingState::None)
	{
	}

	FString	HopperName;
	FString	TicketId;
	float	EstimatedWaitTimeInSeconds;

private:
	const XblMultiplayerSessionReference* SessionReference;
	FGDKMultiplayerSessionHandle GDKSession;
	FGDKMultiplayerSessionHandle LastDiffedGDKSession;

	/** Easy access to the hosting User of this session (will be null on clients) */
	FGDKUserHandle HostUser;

public:
	FName SessionName;
	FOnlineSessionSettings SessionSettings;

	/** The state of the matchmaking process, will be None for non-matchmaking (custom) sessions */
	EOnlineGDKMatchmakingState::Type MatchmakingState;
	
	/** Search settings are used during match ticket resubmission */
	TSharedPtr<FOnlineSessionSearch> SessionSearch;


	FGDKUserHandle GetHostUser() const
	{
		return HostUser;
	}

	void SetHostUser(FGDKUserHandle InHostUser)
	{
		HostUser = InHostUser;
	}

	/** Returns the Xbox-specific match session pointer. */
	const FGDKMultiplayerSessionHandle GetGDKSession() const
	{
		return GDKSession;
	}

	/** Returns the Xbox-specific match session reference pointer. */
	const XblMultiplayerSessionReference* GetGDKSessionRef() const
	{
		return SessionReference;
	}

	const FGDKMultiplayerSessionHandle GetLastDiffedSession() const
	{
		return LastDiffedGDKSession;
	}

	/** Stores the last game session seen in OnSessionChanged. */
	void SetLastDiffedSession(FGDKMultiplayerSessionHandle LatestSession)
	{
		LastDiffedGDKSession = LatestSession;
	}

	void RefreshGDKInfo(FGDKMultiplayerSessionHandle LatestSession)
	{
		GDKSession = LatestSession;
		if (LatestSession)
		{
			SessionReference = XblMultiplayerSessionSessionReference(LatestSession);
		}

		if (!LastDiffedGDKSession)
		{
			LastDiffedGDKSession = LatestSession;
		}
	}

	void RefreshGDKInfo(const XblMultiplayerSessionReference* LatestSessionRef)
	{
		GDKSession.Clear();
		LastDiffedGDKSession.Clear();
		if (LatestSessionRef)
		{
			SessionReference = LatestSessionRef;
		}
	}
};

typedef TSharedPtr<FOnlineMatchTicketInfo> FOnlineMatchTicketInfoPtr;

/** 
 * GDK Session information wrapper, as well as convenient place to store Host Addr as well as map of xuid-FOnlineAssociateGDK
 */
class FOnlineSessionInfoMpsdGDK : public FOnlineSessionInfo
{

public:
	/**
	 * Constructor
	 */
	FOnlineSessionInfoMpsdGDK()
		: IsReady(false)
		, GDKSession(nullptr)
		, GDKSessionRef(nullptr)
		, LastDiffedGameSession(nullptr)
		, SessionId(FUniqueNetIdString::Create(TEXT(""), GDK_SUBSYSTEM))
	{
	}

	/**
	 * Constructor
	 *
	 * @param InSearchResult The GDK search result corresponding to this object
	 */
	FOnlineSessionInfoMpsdGDK(FGDKMultiplayerSearchHandle InGDKSearchResult)
		: IsReady(false)
		, GDKSession(nullptr)
		, GDKSearchResult(InGDKSearchResult)
		, LastDiffedGameSession(nullptr)
		, SessionId(FUniqueNetIdString::Create(TEXT(""), GDK_SUBSYSTEM))
	{
		if (GDKSearchResult)
		{
			HRESULT result = XblMultiplayerSearchHandleGetSessionReference(GDKSearchResult, &GDKSessionRefInternal);
			GDKSessionRef = &GDKSessionRefInternal;

			FString UriPath;
			UriPath += TEXT("/serviceconfigs/");
			UriPath += GDKSessionRef->Scid;
			UriPath += TEXT("/sessiontemplates/");
			UriPath += GDKSessionRef->SessionTemplateName;
			UriPath += TEXT("/sessions/");
			UriPath += GDKSessionRef->SessionName;

			const_cast<FUniqueNetIdString&>(*SessionId).UniqueNetIdStr = MoveTemp(UriPath);
		}
	}

	/**
	 * Constructor
	 *
	 * @param InGDKSession The session corresponding to this object
	 */
	FOnlineSessionInfoMpsdGDK(FGDKMultiplayerSessionHandle InGDKSession)
		: IsReady(false)
		, GDKSession(InGDKSession)
		, LastDiffedGameSession(InGDKSession)
		, SessionId(FUniqueNetIdString::Create(TEXT(""), GDK_SUBSYSTEM))
	{
		if (InGDKSession)
		{
			GDKSessionRef = XblMultiplayerSessionSessionReference(InGDKSession);
			
			FString UriPath;
			UriPath += TEXT("/serviceconfigs/");
			UriPath += GDKSessionRef->Scid;
			UriPath += TEXT("/sessiontemplates/");
			UriPath += GDKSessionRef->SessionTemplateName;
			UriPath += TEXT("/sessions/");
			UriPath += GDKSessionRef->SessionName;

			const_cast<FUniqueNetIdString&>(*SessionId).UniqueNetIdStr = MoveTemp(UriPath);
		}
	}

	/** Destructor */
	virtual ~FOnlineSessionInfoMpsdGDK() = default;

	virtual const uint8* GetBytes() const override
	{
		check(false);  // NOTIMPL for GDK
		return NULL;
	}

	virtual int32 GetSize() const override
	{
		return sizeof(FOnlineSessionInfoMpsdGDK);
	}

	virtual bool IsValid() const override
	{
		return (GDKSessionRef != nullptr);
	}

	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("[Game session ref: %ls]"),
			GDKSessionRef == nullptr ? L"NULL" : UTF8_TO_TCHAR(GDKSessionRef->SessionName));
	}

	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("[Game session ref: %ls]"),
			GDKSessionRef == nullptr ? L"NULL" : UTF8_TO_TCHAR(GDKSessionRef->SessionName));
	}

	virtual const FUniqueNetId& GetSessionId() const override
	{
		return *SessionId;
	}

	//
	// GDK Platform specific APIs
	//

	/** Returns the GDK-specific session pointer. */
	const FGDKMultiplayerSessionHandle GetGDKMultiplayerSession() const
	{
		return GDKSession;
	}

	/** Returns the GDK-specific search handle. */
	const FGDKMultiplayerSearchHandle GetGDKMultiplayerSearchHandle() const
	{
		return GDKSearchResult;
	}

	/** Returns the GDK-specific session reference pointer. */
	const XblMultiplayerSessionReference* GetGDKMultiplayerSessionRef() const
	{
		return GDKSessionRef;
	}

	/**
	 * Updates the cached session and session reference pointers with a new session.
	 * Useful after WriteSessionAsync operations because the old sessions are no longer valid.
	 *
	 * @param LatestSession The new session to replace the cached version with.
	 */
	void RefreshGDKInfo(FGDKMultiplayerSessionHandle LatestSession)
	{
		GDKSession = LatestSession;
		if (LatestSession.IsValid())
		{
			GDKSessionRef = XblMultiplayerSessionSessionReference(LatestSession);

			FString UriPath;
			UriPath += TEXT("/serviceconfigs/");
			UriPath += GDKSessionRef->Scid;
			UriPath += TEXT("/sessiontemplates/");
			UriPath += GDKSessionRef->SessionTemplateName;
			UriPath += TEXT("/sessions/");
			UriPath += GDKSessionRef->SessionName;

			SessionId = FUniqueNetIdString::Create(UriPath, GDK_SUBSYSTEM);
		}
		else
		{
			GDKSessionRef = nullptr;
			SessionId = FUniqueNetIdString::Create(TEXT(""), GDK_SUBSYSTEM);
		}

		if (!LastDiffedGameSession)
		{
			LastDiffedGameSession = LatestSession;
		}
	}

	// The design below feels kind of ugly, but we need a way to explicitly keep track of the last version
	// of the sessions seen by OnSessionChanged, so we can ensure all changes are detected the next time
	// the session subscription fires. We can't use GetGDKMultiplayerSession/GetGDKMatchSession for this,
	// because processing a session change may itself update these, and cause changes made on the server to
	// be missed when the next diff is run.

	/** Returns the last game session seen in OnSessionChanged. */
	const FGDKMultiplayerSessionHandle GetLastDiffedMultiplayerSession() const
	{
		return LastDiffedGameSession;
	}

	/** Stores the last game session seen in OnSessionChanged. */
	void SetLastDiffedMultiplayerSession(FGDKMultiplayerSessionHandle LatestSession)
	{
		LastDiffedGameSession = LatestSession;
	}

	/** Returns the address of the host of this session. */
	TSharedPtr<class FInternetAddr> GetHostAddr() const
	{
		return HostAddr;
	}

	/** Sets the address of the host of this session. */
	void SetHostAddr(TSharedPtr<class FInternetAddr> InAddr)
	{
		HostAddr = InAddr;
	}

	FGuid GetRoundId() const
	{
		return RoundId;
	}

	void SetRoundId(const FGuid& InRoundId)
	{
		RoundId = InRoundId;
	}

	bool IsSessionReady() const
	{
		return IsReady;
	}

	void SetSessionReady()
	{
		IsReady = true;
	}

	void SetSessionInviteHandle(const XblMultiplayerInviteHandle& InSessionHandle)
	{
		SessionInviteHandle = InSessionHandle;
	}

	const TOptional<XblMultiplayerInviteHandle>& GetSessionInviteHandle() const
	{
		return SessionInviteHandle;
	}

	const TOptional<FString> GetSessionInviteHandleString() const
	{
		if (SessionInviteHandle.IsSet())
		{
			return TOptional<FString>(FString(UTF8_TO_TCHAR(SessionInviteHandle.GetValue().Data)));
		}

		return TOptional<FString>();
	}

private:

	bool IsReady;

	/** Wrapped GDK session */
	FGDKMultiplayerSessionHandle GDKSession;

	/** Wrapped GDK Search result */
	FGDKMultiplayerSearchHandle GDKSearchResult;

	/** pointer to GDK session reference. Should be associated with SessionHandle above or, the internal struct below that is associated with a SearchHandle 
	Pointer target memory is NOT managed by this class*/
	const XblMultiplayerSessionReference* GDKSessionRef;

	XblMultiplayerSessionReference GDKSessionRefInternal;

	/** Last version of GDK game session compared in OnSessionChanged */
	FGDKMultiplayerSessionHandle LastDiffedGameSession;

	/** Host address */
	TSharedPtr<class FInternetAddr> HostAddr;

	/** Placeholder unique id */
	FUniqueNetIdStringRef SessionId;

	/** Stored RoundId, used when triggering Xbox events */
	FGuid RoundId;

	/** Session handle if this session is from an to-be-joined invite */
	TOptional<XblMultiplayerInviteHandle> SessionInviteHandle;
};

typedef TSharedPtr<FOnlineSessionInfoMpsdGDK> FOnlineSessionInfoMpsdGDKPtr;

static const int32 GDK_MAX_PLAYER_NAME_LENGTH = 16;

class FOnlineUserInfoGDK
	: public FOnlineUser
{
public:
	virtual FUniqueNetIdRef GetUserId() const override
	{
		return UserId;
	}

	virtual FString GetRealName() const override
	{
		return FilterPlayerName(UserProfile->gameDisplayName);
	}

	virtual FString GetDisplayName(const FString& Platform = FString()) const override
	{
		return FilterPlayerName(GetGamertag(*UserProfile));
	}

	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override
	{
		const FString* const FoundUserAttribute = UserAttributes.Find(AttrName);
		if (FoundUserAttribute == nullptr)
		{
			OutAttrValue.Empty();
			return false;
		}

		OutAttrValue = *FoundUserAttribute;
		return true;
	}

PACKAGE_SCOPE:
	FOnlineUserInfoGDK(const XblUserProfile* InUserProfile)
		: UserProfile(MakeShared<XblUserProfile>(*InUserProfile))
		, UserId(FUniqueNetIdGDK::Create(InUserProfile->xboxUserId))
	{
		check(UserProfile.IsValid());

		UserAttributes.Emplace(FString(TEXT("Gamerscore")), FString(UserProfile->gamerscore));
		// This is a the URI to a resizeable display image for the user.  For example, &format=png&w=64&h=64
		// Valid Format: png
		// Valid Width/Height: 64/64, 208/208, or 424/424
		UserAttributes.Emplace(FString(TEXT("DisplayPictureUri")), FString(UserProfile->gameDisplayPictureResizeUri));
	}

	virtual ~FOnlineUserInfoGDK() = default;

	static FString FilterPlayerName(const FString& InPlayerName)
	{
		FString OutName(InPlayerName);

		// If our name exceeds the max length, we want to truncate it
		TSharedRef<IBreakIterator> GraphemeBreakIterator = FBreakIterator::CreateCharacterBoundaryIterator();
		GraphemeBreakIterator->SetString(InPlayerName);
		GraphemeBreakIterator->ResetToBeginning();
		int32 GraphemeCount = 0;
		for (int32 CurrentCharIndex = GraphemeBreakIterator->MoveToNext(); CurrentCharIndex != INDEX_NONE; CurrentCharIndex = GraphemeBreakIterator->MoveToNext())
		{
			GraphemeCount++;
		}

		const int32 SizeOverage = GraphemeCount - GDK_MAX_PLAYER_NAME_LENGTH;

		if (SizeOverage > 0)
		{
			// Truncate in-place to max name length
			OutName.RemoveAt(GDK_MAX_PLAYER_NAME_LENGTH, SizeOverage, EAllowShrinking::No);
			// Append ellipsis character to show the name goes on
			OutName.AppendChar(L'\u2026');
		}

		return OutName;
	}

	static FString GetGamertag(const XblUserProfile& InUserProfile)
	{
		FString Gamertag;

		switch (IGDKRuntimeModule::Get().GetDefaultGamertagComponent())
		{
		case XUserGamertagComponent::Classic:
			Gamertag = InUserProfile.gamertag;
			break;
		case XUserGamertagComponent::Modern:
			Gamertag = InUserProfile.modernGamertag;
			break;
		case XUserGamertagComponent::ModernSuffix:
			Gamertag = InUserProfile.modernGamertagSuffix;
			break;
		case XUserGamertagComponent::UniqueModern:
			Gamertag = InUserProfile.uniqueModernGamertag;
			break;
		default:
			checkNoEntry();
			break;
		}

		return Gamertag;
	}

PACKAGE_SCOPE:
	TSharedPtr<XblUserProfile> UserProfile;

	FUniqueNetIdGDKRef UserId;
	TMap<FString, FString> UserAttributes;
};

#endif //WITH_GRDK