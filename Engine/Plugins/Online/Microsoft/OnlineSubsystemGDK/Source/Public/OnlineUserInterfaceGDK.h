// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK
#include "Interfaces/OnlineUserInterface.h"
#include "OnlineSubsystemGDKTypes.h"
#include "OnlineSubsystemGDKPackage.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/privacy_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#include "OnlineError.h"
#define LOCTEXT_NAMESPACE "OnlineSubsystemGDK"
#define ONLINE_ERROR_NAMESPACE "errors.com.epicgames.oss.user"

namespace OnlineUserGDK
{
#include "OnlineErrorMacros.inl"

	namespace Errors
	{
		inline FOnlineError NoGDKContext() { return ONLINE_ERROR(EOnlineErrorResult::InvalidParams, TEXT("no_GDK_context")); }
	}
}


#undef LOCTEXT_NAMESPACE
#undef ONLINE_ERROR_NAMESPACE

class FOnlineSubsystemGDK;
class FOnlineUserInfoGDK;

using FOnlineUserListGDKMap = TUniqueNetIdMap<TSharedRef<FOnlineUserInfoGDK>>;

/** Map of User's permission to communicate to other users */
using FCommunicationPermissionResultsMap = TUniqueNetIdMap<bool>;

/** Map of User's privacy permissions */
using FPrivacyPermissionsResultsMap = TUniqueNetIdMap<TMap<XblPermission, bool>>;

/** Map of anonymous user type's permission to communicate to other users */
using FAnonymousUserCommunicationPermissionResultsMap = TMap<XblAnonymousUserType, bool>;

/** Map of anonymous user type's privacy permissions */
using FAnonymousUserPrivacyPermissionsResultsMap = TMap<XblAnonymousUserType, TMap<XblPermission, bool> >;

/**
 * Delegate used when the user communication permissions query has completed
 *
 * @param RequestStatus If this request was successful or not
 * @param RequestingUser The user who generated this request
 * @param Results A map of users to communication results
 */
DECLARE_DELEGATE_ThreeParams(FOnGDKCommunicationPermissionsQueryComplete, const FOnlineError& /*RequestStatus*/, const FUniqueNetIdRef& /*RequestingUser*/, const FCommunicationPermissionResultsMap& /*Results*/);

/**
 * Delegate used when the class communication permissions query has completed
 *
 * @param RequestStatus If this request was successful or not
 * @param Results A map of user types to communication results
 */
DECLARE_DELEGATE_TwoParams(FOnGDKAnonymousUserCommunicationPermissionsQueryComplete, const FOnlineError& /*RequestStatus*/, const FAnonymousUserCommunicationPermissionResultsMap& /*Results*/);

/**
 * Implements the GDK specific interface for friends
 */
class FOnlineUserGDK
	: public IOnlineUser
	, public TSharedFromThis<FOnlineUserGDK, ESPMode::ThreadSafe>
{
	/** The async task classes require friendship */
	friend class FOnlineAsyncTaskGDKQueryUsers;

public:
	// IOnlineUser
	virtual bool QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& UserIds) override;
	virtual bool GetAllUserInfo(int32 LocalUserNum, TArray<TSharedRef<class FOnlineUser>>& OutUsers) override;
	virtual TSharedPtr<FOnlineUser> GetUserInfo(int32 LocalUserNum, const class FUniqueNetId& UserId) override;
	virtual bool QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate = FOnQueryUserMappingComplete()) override;
	virtual bool QueryExternalIdMappings(const FUniqueNetId& UserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate = FOnQueryExternalIdMappingsComplete()) override;
	virtual void GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds) override;
	virtual FUniqueNetIdPtr GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId) override;

	// FOnlineFriendsGDK
	explicit FOnlineUserGDK(FOnlineSubsystemGDK* const InGDKSubsystem)
		: GDKSubsystem(InGDKSubsystem)
	{
		check(GDKSubsystem);
	}

	virtual ~FOnlineUserGDK() = default;

	/**
	 * Query GDK for permission status between a local user and a list of remote GDK users
	 *
	 * @param UserId local user to query status
	 * @param UsersToQuery array of remote users to query relationship with
	 * @param PermissionsToQuery types of permissions to query
	 * @param CompletionDelegate delegate to fire when query is complete
	 *
	 */
	ONLINESUBSYSTEMGDK_API void QueryUserCommunicationPermissions(const FUniqueNetId& UserId, const TArray<FUniqueNetIdRef >& UsersToQuery, const TArray<XblPermission>& PermissionsToQuery, const FOnGDKCommunicationPermissionsQueryComplete& CompletionDelegate);

	/**
	 * Query GDK for permission status between a local user and types of remote crossplay users
	 *
	 * @param UserId local user to query status
	 * @param UserTypesToQuery array of user types to query relationship with
	 * @param PermissionsToQuery types of permissions to query
	 * @param CompletionDelegate delegate to fire when query is complete
	 *
	 */
	ONLINESUBSYSTEMGDK_API void QueryAnonymousUserCommunicationPermissions(const FUniqueNetId& UserId, const TArray<XblAnonymousUserType>& UserTypesToQuery, const TArray<XblPermission>& PermissionsToQuery, const FOnGDKAnonymousUserCommunicationPermissionsQueryComplete& CompletionDelegate);

private:
	void OnQueryUserCommunicationPermissionsComplete(const FOnlineError& RequestStatus, const FUniqueNetIdRef& RequestingUser, const FPrivacyPermissionsResultsMap& Results, const FOnGDKCommunicationPermissionsQueryComplete CompletionDelegate, const TArray<XblPermission> PermissionsToQuery);

	void OnQueryAnonymousUserCommunicationPermissionsComplete(const FOnlineError& RequestStatus, const FAnonymousUserPrivacyPermissionsResultsMap& Results, const FOnGDKAnonymousUserCommunicationPermissionsQueryComplete CompletionDelegate, const TArray<XblPermission> PermissionsToQuery);

private:
	/** Reference to the main GDK subsystem */
	FOnlineSubsystemGDK* const GDKSubsystem;

	/** Map of known user information */
	FOnlineUserListGDKMap UsersMap;
};

typedef TSharedPtr<FOnlineUserGDK, ESPMode::ThreadSafe> FOnlineUserGDKPtr;

#endif //WITH_GRDK